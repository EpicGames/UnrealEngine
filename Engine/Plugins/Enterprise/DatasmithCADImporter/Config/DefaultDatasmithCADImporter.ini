[CoreRedirects]
+ClassRedirects=(OldName="/Script/DatasmithCoreTechExtension.CoreTechBlueprintLibrary", NewName="/Script/ParametricSurfaceExtension.ParametricSurfaceBlueprintLibrary")
+ClassRedirects=(OldName="/Script/CoreTechExtension.CoreTechBlueprintLibrary", NewName="/Script/ParametricSurfaceExtension.ParametricSurfaceBlueprintLibrary")

+ClassRedirects=(OldName="/Script/DatasmithCoreTechExtension.CoreTechRetessellateAction", NewName="/Script/ParametricSurfaceExtension.ParametricRetessellateAction")
+ClassRedirects=(OldName="/Script/CoreTechExtension.CoreTechRetessellateAction", NewName="/Script/ParametricSurfaceExtension.ParametricRetessellateAction")

+ClassRedirects=(OldName="/Script/DatasmithCoreTechExtension.CoreTechRetessellateActionOptions", NewName="/Script/ParametricSurfaceExtension.ParametricRetessellateActionOptions")
+ClassRedirects=(OldName="/Script/CoreTechExtension.CoreTechRetessellateActionOptions", NewName="/Script/ParametricSurfaceExtension.ParametricRetessellateActionOptions")

+ClassRedirects=(OldName="/Script/DatasmithCoreTechExtension.DataprepTessellationOperation", NewName="/Script/ParametricSurfaceExtension.DataprepTessellationOperation")
+ClassRedirects=(OldName="/Script/CoreTechExtension.DataprepTessellationOperation", NewName="/Script/ParametricSurfaceExtension.DataprepTessellationOperation")

+ClassRedirects=(OldName="/Script/ParametricSurface.UParametricSurfaceData", NewName="/Script/ParametricSurface.UDatasmithParametricSurfaceData")

+PropertyRedirects=(OldName="UDatasmithParametricSurfaceData.RawData",NewName="RawData_DEPRECATED")

[SectionsToSave]
+Section=/Script/DatasmithWireTranslator.DatasmithWireOptions
+Section=/Script/DatasmithOpenNurbsTranslator.DatasmithOpenNurbsImportOptions
+Section=/Script/DatasmithOpenNurbsTranslator.DatasmithOpenNurbsImportOptions


