{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Datasmith CAD Importer",
	"Description" : "Collection of tools to work with CAD files.",
	"Category" : "Importers",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://docs.unrealengine.com/en-US/WorkingWithContent/Importing/Datasmith/",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedTargetPlatforms":
	[
		"Win64",
		"Linux"
	],
	"Modules": [
		{
			"Name": "CADLibrary",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "DatasmithCADTranslator",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DatasmithOpenNurbsTranslator",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "DatasmithDispatcher",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "CADInterfaces",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "CADTools",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DatasmithWireTranslator",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2020",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2021_3",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2022",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2022_1",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2022_2",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2023_0",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2023_1",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2024_1",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2025_0",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "WireInterface2026_0",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "ParametricSurfaceExtension",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "ParametricSurface",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "CADKernelSurface",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DatasmithPLMXMLTranslator",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "DatasmithImporter",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		}
	]
}