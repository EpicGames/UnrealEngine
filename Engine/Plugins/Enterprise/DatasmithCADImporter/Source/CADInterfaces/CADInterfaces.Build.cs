// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class CADInterfaces : ModuleRules
	{
		public CADInterfaces(ReadOnlyTargetRules Target) : base(Target)
		{
			bEnableExceptions = true;
			bLegalToDistributeObjectCode = true;
			bUseUnity = false;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CADKernel",
					"CADLibrary",
					"CADTools",
					"DatasmithCore",
					"Json",
				}
			);

			// CAD library is only available if TechSoft is available too
			bool bHasTechSoft = System.Type.GetType("TechSoft") != null;

			if (Target.Platform == UnrealTargetPlatform.Win64 && bHasTechSoft)
			{
				PublicDependencyModuleNames.Add("TechSoft");
			}
			// System.Type.GetType("TechSoft") does not seem to work on Linux
			// Temporary fix. I will investigate for 5.6
			else if (Target.Platform == UnrealTargetPlatform.Linux)
			{
				string TechSoftPath = Path.Combine(EngineDirectory, "Restricted/NotForLicensees/Source/ThirdParty/Enterprise/TechSoft/TechSoft.Build.cs");
				if(File.Exists(TechSoftPath))
				{
					PublicDependencyModuleNames.Add("TechSoft");
				}
			}
		}
	}
}
