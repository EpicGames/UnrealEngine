// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

enum class ECADType
{
	ACIS,
	CATIA,
	CATIA_CGR,
	CATIA_3DXML,
	CATIAV4,
	CREO,
	IGES,
	INVENTOR,
	JT,
	NX,
	PARASOLID,
	SOLIDWORKS,
	STEP,
	OTHER
};
