// Copyright Epic Games, Inc. All Rights Reserved.
#include "CADFileData.h"

#include "Misc/FileHelper.h"

namespace CADLibrary
{

void FCADFileData::InitCsvFile()
{
	FString CsvData;
	CsvData.Reserve(FCString::Strlen(CSV_HEADER) * 2);
	CsvData += CSV_HEADER;

	const FString& RootPath = FileDescription.GetRootFolder();
	const FString& FilePath = FileDescription.GetSourcePath();

	//Input File
	FString FileName = FilePath.Right(FilePath.Len() - RootPath.Len() - 1);
	FileName.ReplaceCharInline(TEXT(','), TEXT('_'));
	CsvData += FileName + TEXT(",");
	//Input File
	CsvData += SceneGraphArchive.ArchiveFileName + TEXT(",");

	//FileSize
	FFileStatData FileStatData = IFileManager::Get().GetStatData(*FileDescription.GetSourcePath());
	constexpr int64 OneKiloBit = 1024;
	CsvData += FString::Printf(TEXT("%lld,"), FileStatData.FileSize / OneKiloBit);
	CsvData += TEXT("\0\0\0\0\0\0\0\0\0\0\0\0\0\0\0");
	FString CsvPath = GetValidationFilePath();
	FFileHelper::SaveStringToFile(*CsvData, *CsvPath, FFileHelper::EEncodingOptions::ForceUTF8);
}

void FCADFileData::ExportValidationData()
{
	FString CsvData;
	CsvData.Reserve(FCString::Strlen(CSV_HEADER));

	//Time.LoadStep
	CsvData += FString::Printf(TEXT("%f,"), Record.ImportTime);
	//Time.SavePrcTime
	CsvData += FString::Printf(TEXT("%f,"), Record.SavePrcTime);
	//Time.AdaptBRepTime
	CsvData += FString::Printf(TEXT("%f,"), Record.AdaptBRepTime);
	//Time.SewStep
	CsvData += FString::Printf(TEXT("%f,"), Record.SewTime);
	//Time.MeshStep
	CsvData += FString::Printf(TEXT("%f,"), Record.MeshTime);
	//Time.Total
	CsvData += FString::Printf(TEXT("%f,"), Record.LoadProcessTime);

	//MemoryUsage
	constexpr float OneMegaBit = 1024.f * 1024.f;
	uint64 MaxUsedMemory = Record.MaxMemoryUsed - Record.StartMemoryUsed;
	float MaxUsedMemoryF = static_cast<float>(MaxUsedMemory) / OneMegaBit;
	CsvData += FString::Printf(TEXT("%f,"), MaxUsedMemoryF);

	//SceneGrapheData.InstanceNum
	CsvData.AppendInt(SceneGraphArchive.Instances.Num());
	CsvData += TEXT(",");
	//SceneGrapheData.Bodies
	CsvData.AppendInt(SceneGraphArchive.Bodies.Num());
	CsvData += TEXT(",");
	//SceneGrapheData.References
	CsvData.AppendInt(SceneGraphArchive.References.Num());
	CsvData += TEXT(",");
	//SceneGrapheData.UnloadedReferences
	CsvData.AppendInt(SceneGraphArchive.UnloadedReferences.Num());
	CsvData += TEXT(",");
	//SceneGrapheData.ExternalReferenceFiles
	CsvData.AppendInt(SceneGraphArchive.ExternalReferenceFiles.Num());
	CsvData += TEXT(",");

	//SceneGrapheData.ColorHIdToColor
	CsvData.AppendInt(SceneGraphArchive.ColorHIdToColor.Num());
	CsvData += TEXT(",");
	//SceneGrapheData.MaterialHIdToMaterial
	CsvData.AppendInt(SceneGraphArchive.MaterialHIdToMaterial.Num());
	CsvData += TEXT(",");

	int32 FaceCount = 0;
	int32 VertexCount = 0;
	GetMeshStatsToCsv(FaceCount, VertexCount);

	//Meshes.FaceCount
	CsvData.AppendInt(FaceCount);
	CsvData += TEXT(",");
	//Meshes.VertexCount
	CsvData.AppendInt(VertexCount);
	CsvData += TEXT(",");
	CsvData += TEXT('\0');

	FString CsvPath = GetValidationFilePath();
	FFileHelper::SaveStringToFile(*CsvData, *CsvPath, FFileHelper::EEncodingOptions::ForceUTF8, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
}

void FCADFileData::GetMeshStatsToCsv(int32& FaceCount, int32& VertexCount)
{
	
	for (const FBodyMesh& BodyMesh : BodyMeshes)
	{
		FaceCount += BodyMesh.Faces.Num();
		VertexCount += BodyMesh.VertexArray.Num();
	}
}


}