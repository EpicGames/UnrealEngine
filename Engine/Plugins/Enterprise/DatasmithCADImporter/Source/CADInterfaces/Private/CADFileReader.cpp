// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADFileReader.h"

#include "CADFileParser.h"
#include "CADOptions.h"
#include "HAL/FileManager.h"
#include "Templates/TypeHash.h"

#ifdef USE_TECHSOFT_SDK
#include "TechSoftFileParser.h"
#include "TechSoftFileParserCADKernelTessellator.h"
#endif

namespace CADLibrary
{
	FCADFileReader::FCADFileReader(const FImportParameters& ImportParams, FFileDescriptor& InFile, const FString& EnginePluginsPath, const FString& InCachePath)
		: CADFileData(ImportParams, InFile, InCachePath)
	{
#ifdef USE_TECHSOFT_SDK
		if (ImportParams.GetMesher() == EMesher::CADKernel)
		{
			CADParser = MakeUnique<FTechSoftFileParserCADKernelTessellator>(CADFileData, EnginePluginsPath);
		}
		else
		{
			CADParser = MakeUnique<FTechSoftFileParser>(CADFileData, EnginePluginsPath);
		}
#endif
	}

	bool FCADFileReader::FindFile(FFileDescriptor& File)
	{
		const FString& FileName = File.GetFileName();

		FString FilePath = FPaths::GetPath(File.GetSourcePath());
		FString RootFilePath = File.GetRootFolder();

		// Basic case: File exists at the initial path
		if (IFileManager::Get().FileExists(*File.GetSourcePath()))
		{
			return true;
		}

		// Advance case: end of FilePath is in a upper-folder of RootFilePath
		// e.g.
		// FilePath = D:\\data temp\\Unstructured project\\Folder2\\Added_Object.SLDPRT
		//                                                 ----------------------------
		// RootFilePath = D:\\data\\CAD Files\\SolidWorks\\p033 - Unstructured project\\Folder1
		//                ------------------------------------------------------------
		// NewPath = D:\\data\\CAD Files\\SolidWorks\\p033 - Unstructured project\\Folder2\\Added_Object.SLDPRT
		TArray<FString> RootPaths;
		RootPaths.Reserve(30);
		do
		{
			RootFilePath = FPaths::GetPath(RootFilePath);
			RootPaths.Emplace(RootFilePath);
		} while (!FPaths::IsDrive(RootFilePath) && !RootFilePath.IsEmpty());

		TArray<FString> FilePaths;
		FilePaths.Reserve(30);
		FilePaths.Emplace(FileName);
		while (!FPaths::IsDrive(FilePath) && !FilePath.IsEmpty())
		{
			FString FolderName = FPaths::GetCleanFilename(FilePath);
			FilePath = FPaths::GetPath(FilePath);
			FilePaths.Emplace(FPaths::Combine(FolderName, FilePaths.Last()));
		};

		for (int32 IndexFolderPath = 0; IndexFolderPath < RootPaths.Num(); IndexFolderPath++)
		{
			for (int32 IndexFilePath = 0; IndexFilePath < FilePaths.Num(); IndexFilePath++)
			{
				FString NewFilePath = FPaths::Combine(RootPaths[IndexFolderPath], FilePaths[IndexFilePath]);
				if (IFileManager::Get().FileExists(*NewFilePath))
				{
					File.SetSourceFilePath(NewFilePath);
					return true;
				};
			}
		}

		// Last case: the FilePath is elsewhere and the file exist
		// A Warning is launch because the file could be expected to not be loaded
		if (IFileManager::Get().FileExists(*File.GetSourcePath()))
		{
			return true;
		}

		CADFileData.LogWarning(FString::Printf(TEXT("File %s cannot be found."), *File.GetFileName()));
		return false;
	}

	ECADParsingResult FCADFileReader::ProcessFile()
	{
		if (!CADParser.IsValid())
		{
			return ECADParsingResult::ProcessFailed;
		}

		CADFileData.SetArchiveNames();

		if (!FindFile(CADFileData.GetCADFileDescription()))
		{
			return ECADParsingResult::FileNotFound;
		}

		if (FImportParameters::bGEnableCADCache)
		{
			bool bNeedToProceed = true;

			FString CADFileCachePath = CADFileData.GetCADCachePath();
			if (IFileManager::Get().FileExists(*CADFileCachePath))
			{
				if (!FImportParameters::bGOverwriteCache)
				{
					CADFileData.GetCADFileDescription().SetCacheFile(CADFileCachePath);
					FString MeshArchiveFilePath = CADFileData.GetMeshArchiveFilePath();
					if (IFileManager::Get().FileExists(*MeshArchiveFilePath)) // the file has been proceed with same meshing parameters
					{
						bNeedToProceed = false;
					}
				}
			}

			if (!bNeedToProceed)
			{
				// The file has been yet proceed, get ExternalRef
				CADFileData.LoadSceneGraphArchive();
				// update the ExternalRef path according to the current file path
				// Indeed, the loaded file can be in another folder with differents other files
				// Without the update, this is the files in the other folder that will be proceed 
				CADFileData.UpdateExternalRefPath();
				return ECADParsingResult::ProcessOk;
			}
		}

		if (FImportParameters::bValidationProcess)
		{
			CADFileData.InitCsvFile();
		}

		// Process the file
		ECADParsingResult Result = CADParser->Process();

		if (FImportParameters::bGEnableCADCache)
		{
			if (ECADParsingResult::ProcessOk == Result)
			{
				CADFileData.ExportSceneGraphFile();
				CADFileData.ExportMeshArchiveFile();
			}
		}

		if (FImportParameters::bValidationProcess)
		{
			CADFileData.ExportValidationData();
		}

		return Result;
	}
}
