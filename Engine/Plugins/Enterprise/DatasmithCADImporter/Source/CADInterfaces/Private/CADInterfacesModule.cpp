// Copyright Epic Games, Inc. All Rights Reserved.
#include "CADInterfacesModule.h"

#include "CADOptions.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "TechSoftInterface.h"

#define LOCTEXT_NAMESPACE "CADInterfacesModule"

DEFINE_LOG_CATEGORY(LogCADInterfaces);

class FCADInterfacesModule : public ICADInterfacesModule
{
private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static void* KernelIOLibHandle;
	static void* TechSoftLibHandle;
};

void* FCADInterfacesModule::TechSoftLibHandle = nullptr;

ICADInterfacesModule& ICADInterfacesModule::Get()
{
	return FModuleManager::LoadModuleChecked< FCADInterfacesModule >(CADINTERFACES_MODULE_NAME);
}

ECADInterfaceAvailability CADInterfaceAvailability = ECADInterfaceAvailability::Unknown;

ECADInterfaceAvailability ICADInterfacesModule::GetAvailability()
{
	if (FModuleManager::Get().IsModuleLoaded(CADINTERFACES_MODULE_NAME))
	{
		if (CADLibrary::TechSoftInterface::TECHSOFT_InitializeKernel())
		{
			return ECADInterfaceAvailability::Available;
		}
	}

	UE_LOGF(LogCADInterfaces, Warning, "Failed to load CADInterfaces module. Plug-in may not be functional.");
	return ECADInterfaceAvailability::Unavailable;
}

const TCHAR* ICADInterfacesModule::GetLibraryVersion()
{
	return CADLibrary::FTechSoftInterface::Get().GetVersion();
}

void FCADInterfacesModule::StartupModule()
{

#if WITH_EDITOR & defined(USE_TECHSOFT_SDK)
	check(TechSoftLibHandle == nullptr);

	// determine directory paths
	FString TechSoftDllPath = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/TechSoft"), FPlatformProcess::GetBinariesSubdirectory());
	FPlatformProcess::PushDllDirectory(*TechSoftDllPath);

#if PLATFORM_WINDOWS
	FString TechSoftDll = TEXT("A3DLIBS.dll");
#elif PLATFORM_LINUX
	FString TechSoftDll = TEXT("libA3DLIBS.so");
#else
#error Platform not supported
#endif
	TechSoftDll = FPaths::Combine(TechSoftDllPath, TechSoftDll);

	if (!FPaths::FileExists(TechSoftDll))
	{
		UE_LOGF(LogCADInterfaces, Warning, "TechSoft module is missing. Plug-in will not be functional.");
	}
	else
	{
		TechSoftLibHandle = FPlatformProcess::GetDllHandle(*TechSoftDll);
		if (TechSoftLibHandle == nullptr)
		{
			UE_LOGF(LogCADInterfaces, Warning, "Failed to load required library %ls. Plug-in will not be functional.", *TechSoftDll);
		}

	}
	FPlatformProcess::PopDllDirectory(*TechSoftDllPath);
#endif

}

void FCADInterfacesModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FCADInterfacesModule, CADInterfaces);

#undef LOCTEXT_NAMESPACE // "CADInterfacesModule"

