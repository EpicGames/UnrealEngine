// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADSceneGraph.h"

#include "Misc/FileHelper.h"
#include "Serialization/Archive.h"
#include "HAL/FileManager.h"

namespace CADLibrary
{

FArchive& operator<<(FArchive& Ar, FArchiveCADObject& Object)
{
	Ar << Object.Id;
	Ar << Object.Label;
	Ar << Object.MetaData;
	Ar << Object.TransformMatrix;
	Ar << Object.Unit;
	Ar << (FArchiveGraphicProperties&) Object;
	return Ar;
}

bool FArchiveCADObject::SetNameWithAttributeValue(const TCHAR* Key)
{
	FString* LabelPtr = MetaData.Find(Key);
	if (LabelPtr != nullptr)
	{
		Label = *LabelPtr;
		return true;
	}
	return false;
}

FArchive& operator<<(FArchive& Ar, FArchiveInstance& Instance) 
{
	Ar << (FArchiveWithOverridenChildren&) Instance;
	Ar << Instance.ReferenceNodeId;
	Ar << Instance.bIsExternalReference;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FArchiveReference& Component)
{
	Ar << (FArchiveCADObject&) Component;
	Ar << Component.Children;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FArchiveUnloadedReference& Unloaded) 
{
	Ar << (FArchiveCADObject&) Unloaded;
	Ar << Unloaded.ExternalFile;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FArchiveOverrideOccurrence& OverrideOccurrence)
{
	Ar << (FArchiveWithOverridenChildren&) OverrideOccurrence;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FArchiveWithOverridenChildren& WithOverridenChildren)
{
	Ar << (FArchiveCADObject&) WithOverridenChildren;
	Ar << WithOverridenChildren.OverridenChildren;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FArchiveBody& Body)
{
	Ar << (FArchiveCADObject&) Body;
	Ar << Body.MaterialFaceSet;
	Ar << Body.ColorFaceSet;
	Ar << Body.Mesher;
	Ar << Body.ParentId;
	Ar << Body.MeshActorUId;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FArchiveColor& Color) 
{
	Ar << Color.Id;
	Ar << Color.Color;
	Ar << Color.UEMaterialUId;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FArchiveMaterial& Material)
{
	Ar << Material.Id;
	Ar << Material.Material;
	Ar << Material.UEMaterialUId;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FArchiveSceneGraph& SceneGraph)
{
	Ar << SceneGraph.SceneGraphId;
	Ar << SceneGraph.CADFileName;
	Ar << SceneGraph.ArchiveFileName;
	Ar << SceneGraph.FullPath;
	Ar << SceneGraph.ExternalReferenceFiles;

	Ar << SceneGraph.ColorHIdToColor;
	Ar << SceneGraph.MaterialHIdToMaterial;

	Ar << SceneGraph.Instances;
	Ar << SceneGraph.References;
	Ar << SceneGraph.UnloadedReferences;
	Ar << SceneGraph.OverrideOccurrences;
	Ar << SceneGraph.Bodies;

	Ar << SceneGraph.CADIdToIndex;

	return Ar;
}

void FArchiveSceneGraph::SerializeMockUp(const TCHAR* Filename)
{
	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileWriter(Filename));
	if (Archive.IsValid())
	{
		*Archive << *this;
		Archive->Close();
	}
}

void FArchiveSceneGraph::DeserializeMockUpFile(const TCHAR* Filename)
{
	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileReader(Filename));
	if (Archive.IsValid())
	{
		*Archive << *this;
		Archive->Close();
	}
}

FArchiveInstance& FArchiveSceneGraph::AddInstance(const FArchiveCADObject& Parent)
{
	ensure(Instances.Num() < Instances.Max());

	const int32 InstanceId = LastEntityId++;

	int32 Index = Instances.Emplace(InstanceId, Parent);
	CADIdToIndex.Add(Index);
	return Instances[Index];
}

FArchiveInstance& FArchiveSceneGraph::GetInstance(const FCadId CadId)
{
	ensure(CADIdToIndex.IsValidIndex(CadId));
	const int32 Index = CADIdToIndex[CadId];
	ensure(Instances.IsValidIndex(Index));
	return Instances[Index];
}

void FArchiveSceneGraph::RemoveLastInstance()
{
	ensure(!Instances.IsEmpty());
	LastEntityId--;
	ensure(Instances.Last().Id == LastEntityId);
	Instances.SetNum(Instances.Num() - 1, EAllowShrinking::No);
	CADIdToIndex.SetNum(CADIdToIndex.Num() - 1, EAllowShrinking::No);
}

bool FArchiveSceneGraph::IsAInstance(FCadId CadId) const 
{
	if(!CADIdToIndex.IsValidIndex(CadId))
	{
		return false;
	}
	const int32 Index = CADIdToIndex[CadId];

	if(!Instances.IsValidIndex(Index))
	{
		return false;
	}
	return Instances[Index].Id == CadId;
}

FArchiveReference& FArchiveSceneGraph::AddReference(FArchiveUnloadedReference& Reference)
{
	ensure(References.Num() < References.Max());

	int32 Index = References.Emplace(Reference.Id, Reference);

	FArchiveReference& NewReference = References[Index];
	NewReference.MoveTemp(Reference);

	// do not call here RemoveLastUnloadedReference otherwise the Reference.Id will be delete
	UnloadedReferences.SetNum(UnloadedReferences.Num() - 1, EAllowShrinking::No);

	CADIdToIndex[Reference.Id] = Index;

	return NewReference;
}

FArchiveReference& FArchiveSceneGraph::AddReference(FArchiveInstance& Parent)
{
	ensure(References.Num() < References.Max());

	const int32 ReferenceId = LastEntityId++;

	int32 Index = References.Emplace(ReferenceId, Parent);
	CADIdToIndex.Add(Index);

	Parent.bIsExternalReference = false;
	Parent.ReferenceNodeId = ReferenceId;

	return References[Index];
}

FArchiveReference& FArchiveSceneGraph::AddOccurrence(FArchiveReference& Parent)
{
	FArchiveInstance& Instance = AddInstance(Parent);
	Parent.AddChild(Instance.Id);
	FArchiveReference& Reference = AddReference(Instance);
	Instance.ReferenceNodeId = Reference.Id;
	return Reference;
}

void FArchiveSceneGraph::RemoveLastOccurrence()
{
	RemoveLastReference();
	RemoveLastInstance();
}


FArchiveReference& FArchiveSceneGraph::GetReference(const FCadId CadId)
{
	ensure(CADIdToIndex.IsValidIndex(CadId));
	const int32 Index = CADIdToIndex[CadId];
	ensure(References.IsValidIndex(Index));
	return References[Index];
}

bool FArchiveSceneGraph::IsAReference(const FCadId CadId) const
{
	if (!CADIdToIndex.IsValidIndex(CadId))
	{
		return false;
	}
	const int32 Index = CADIdToIndex[CadId];

	if (!References.IsValidIndex(Index))
	{
		return false;
	}
	return References[Index].Id == CadId;
}

void FArchiveSceneGraph::RemoveLastReference()
{
	ensure(!References.IsEmpty());
	LastEntityId--;
	ensure(References.Last().Id == LastEntityId);

	References.SetNum(References.Num() - 1, EAllowShrinking::No);
	CADIdToIndex.SetNum(CADIdToIndex.Num() - 1, EAllowShrinking::No);
}

void FArchiveWithOverridenChildren::AddOverridenChild(const FCadId ChildId)
{
	OverridenChildren.Add(ChildId);
}

void FArchiveReference::AddChild(const FCadId ChildId)
{
	Children.Add(ChildId);
}

void FArchiveReference::RemoveLastChild()
{
	Children.Pop();
}

int32 FArchiveReference::ChildrenCount()
{
	return Children.Num();
}

void FArchiveBody::Delete()
{
	bIsRemoved = true;
	MetaData.Empty();
	ColorFaceSet.Empty();
	MaterialFaceSet.Empty();
	Label.Empty();
	MeshActorUId = 0;
}


void FArchiveReference::MoveTemp(FArchiveUnloadedReference& Reference)
{
	Id = Reference.Id;
	Label = ::MoveTemp(Reference.Label);
	MetaData = ::MoveTemp(Reference.MetaData);
	TransformMatrix = ::MoveTemp(Reference.TransformMatrix);
	ColorUId = Reference.ColorUId;
	MaterialUId = Reference.MaterialUId;
	Unit = Reference.Unit;
}

void FArchiveReference::CopyMetaData(FArchiveUnloadedReference& Reference)
{
	Label = Reference.Label;
	MetaData = Reference.MetaData;
	ColorUId = Reference.ColorUId;
	MaterialUId = Reference.MaterialUId;
	Unit = Reference.Unit;
}

FArchiveUnloadedReference& FArchiveSceneGraph::AddUnloadedReference(FArchiveInstance& Parent)
{
	ensure(UnloadedReferences.Num() < UnloadedReferences.Max());

	const int32 ReferenceId = LastEntityId++;

	int32 Index = UnloadedReferences.Emplace(ReferenceId, Parent);
	CADIdToIndex.Add(Index);

	Parent.bIsExternalReference = true;
	Parent.ReferenceNodeId = ReferenceId;

	return UnloadedReferences[Index];
}

FArchiveUnloadedReference& FArchiveSceneGraph::GetUnloadedReference(const FCadId CadId)
{
	ensure(CADIdToIndex.IsValidIndex(CadId));
	const int32 Index = CADIdToIndex[CadId];
	ensure(UnloadedReferences.IsValidIndex(Index));
	return UnloadedReferences[Index];
}

FArchiveOverrideOccurrence& FArchiveSceneGraph::AddOverrideOccurrence(FArchiveWithOverridenChildren& Parent)
{
	ensure(OverrideOccurrences.Num() < OverrideOccurrences.Max());

	const int32 ReferenceId = LastEntityId++;

	int32 Index = OverrideOccurrences.Emplace(ReferenceId, Parent);
	CADIdToIndex.Add(Index);

	return OverrideOccurrences[Index];
}

FArchiveOverrideOccurrence& FArchiveSceneGraph::GetOverrideOccurrence(const FCadId CadId)
{
	ensure(CADIdToIndex.IsValidIndex(CadId));
	const int32 Index = CADIdToIndex[CadId];
	ensure(OverrideOccurrences.IsValidIndex(Index));
	return OverrideOccurrences[Index];
}

bool FArchiveSceneGraph::IsAUnloadedReference(const FCadId CadId) const
{
	if (!CADIdToIndex.IsValidIndex(CadId))
	{
		return false;
	}
	const int32 Index = CADIdToIndex[CadId];

	if (!UnloadedReferences.IsValidIndex(Index))
	{
		return false;
	}
	return UnloadedReferences[Index].Id == CadId;
}

void FArchiveSceneGraph::RemoveLastUnloadedReference()
{
	ensure(!UnloadedReferences.IsEmpty());
	LastEntityId--;
	ensure(UnloadedReferences.Last().Id == LastEntityId);

	UnloadedReferences.SetNum(UnloadedReferences.Num() - 1, EAllowShrinking::No);
	CADIdToIndex.SetNum(CADIdToIndex.Num() - 1, EAllowShrinking::No);
}

FArchiveBody& FArchiveSceneGraph::AddBody(FArchiveReference& Parent, EMesher InMesher)
{
	ensure(Bodies.Num() < Bodies.Max());

	const int32 BodyId = LastEntityId++;
	int32 Index = Bodies.Emplace(BodyId, Parent, InMesher);
	CADIdToIndex.Add(Index);

	return Bodies[Index];
}

FArchiveBody& FArchiveSceneGraph::GetBody(const FCadId CadId)
{
	ensure(CADIdToIndex.IsValidIndex(CadId));
	const int32 Index = CADIdToIndex[CadId];
	ensure(Bodies.IsValidIndex(Index));
	return Bodies[Index];
}

bool FArchiveSceneGraph::IsABody(const FCadId CadId) const
{
	if (!CADIdToIndex.IsValidIndex(CadId))
	{
		return false;
	}
	const int32 Index = CADIdToIndex[CadId];

	if (!Bodies.IsValidIndex(Index))
	{
		return false;
	}
	return Bodies[Index].Id == CadId;
}

void FArchiveSceneGraph::RemoveLastBody()
{
	ensure(!Bodies.IsEmpty());
	LastEntityId--;
	ensure(Bodies.Last().Id == LastEntityId);
	Bodies.SetNum(Bodies.Num() - 1, EAllowShrinking::No);
	CADIdToIndex.SetNum(CADIdToIndex.Num() - 1, EAllowShrinking::No);
}

void FArchiveSceneGraph::AddExternalReferenceFile(const FArchiveUnloadedReference& Reference)
{
	const int32 Index = CADIdToIndex[Reference.Id];
	ensure(ExternalReferenceFiles.Add(Reference.ExternalFile) == Index);
}

}


