// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechSoftBridge.h"

#ifdef USE_TECHSOFT_SDK

#include "CADSceneGraph.h"
#include "TechSoftInterface.h"
#include "TechSoftFileParser.h"
#include "TechSoftUtils.h"
#include "TUniqueTechSoftObj.h"

#include "Core/Session.h"

#include "Geo/Curves/Curve.h"
#include "Geo/Curves/NURBSCurveData.h"

#include "Geo/Surfaces/NurbsSurfaceData.h"
#include "Geo/Surfaces/Surface.h"

#include "Topo/Body.h"
#include "Topo/Model.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalLink.h"
#include "Topo/TopologicalVertex.h"

#include "Utils/StringUtil.h"

#ifdef CADKERNEL_DEV
#include "CADFileReport.h"
#endif

namespace CADLibrary
{
namespace TechSoftUtils
{

template<typename... InArgTypes>
A3DStatus GetCurveAsNurbs(const A3DCrvBase* A3DCurve, A3DCrvNurbsData* DataPtr, InArgTypes&&... Args)
{
	return TechSoftInterface::GetCurveAsNurbs(A3DCurve, DataPtr, Forward<InArgTypes>(Args)...);
};

template<typename... InArgTypes>
A3DStatus GetSurfaceAsNurbs(const A3DSurfBase* A3DSurface, A3DSurfNurbsData* DataPtr, InArgTypes&&... Args)
{
	return TechSoftInterface::GetSurfaceAsNurbs(A3DSurface, DataPtr, Forward<InArgTypes>(Args)...);
};

UE::CADKernel::FMatrixH CreateCoordinateSystem(const A3DMiscCartesianTransformationData& Transformation, double UnitScale = 1.0)
{
	using namespace UE::CADKernel;

	FVector Origin(Transformation.m_sOrigin.m_dX, Transformation.m_sOrigin.m_dY, Transformation.m_sOrigin.m_dZ);
	FVector Ox(Transformation.m_sXVector.m_dX, Transformation.m_sXVector.m_dY, Transformation.m_sXVector.m_dZ);
	FVector Oy(Transformation.m_sYVector.m_dX, Transformation.m_sYVector.m_dY, Transformation.m_sYVector.m_dZ);

	Ox.Normalize();
	Oy.Normalize();

	if (!FMath::IsNearlyEqual(UnitScale, 1.))
	{
		Origin *= UnitScale;
	}
	FVector Oz = Ox ^ Oy;

	FMatrixH Matrix = FMatrixH(Origin, Ox, Oy, Oz);

	if (!FMath::IsNearlyEqual(Transformation.m_sScale.m_dX, 1.) || !FMath::IsNearlyEqual(Transformation.m_sScale.m_dY, 1.) || !FMath::IsNearlyEqual(Transformation.m_sScale.m_dZ, 1.))
	{
		FMatrixH Scale = FMatrixH::MakeScaleMatrix(Transformation.m_sScale.m_dX, Transformation.m_sScale.m_dY, Transformation.m_sScale.m_dZ);
		Matrix *= Scale;
	}
	return Matrix;
}

void FillInt32Array(const int32 Count, const A3DInt32* Values, TArray<int32>& OutInt32Array)
{
	OutInt32Array.Reserve(Count);
	for (int32 Index = 0; Index < Count; Index++)
	{
		OutInt32Array.Add(Values[Index]);
	}
};

void FillDoubleArray(const int32 Count, const double* Values, TArray<double>& OutDoubleArray)
{
	OutDoubleArray.Reserve(Count);
	for (int32 Index = 0; Index < Count; Index++)
	{
		OutDoubleArray.Add(Values[Index]);
	}
};

void FillDoubleArray(const int32 UCount, const int32 VCount, const double* Values, TArray<double>& OutDoubleArray)
{
	OutDoubleArray.SetNum(UCount * VCount);
	for (int32 Undex = 0, ValueIndex = 0; Undex < UCount; ++Undex)
	{
		int32 Index = Undex;
		for (int32 Vndex = 0; Vndex < VCount; ++Vndex, Index += UCount, ++ValueIndex)
		{
			OutDoubleArray[Index] = Values[ValueIndex];
		}
	}
}
void FillPointArray(const int32 Count, const A3DVector3dData* Points, TArray<FVector>& OutPointsArray, double UnitScale = 1.0)
{
	using namespace UE::CADKernel;

	OutPointsArray.Reserve(Count);
	for (int32 Index = 0; Index < Count; Index++)
	{
		OutPointsArray.Emplace(Points[Index].m_dX, Points[Index].m_dY, Points[Index].m_dZ);
	}

	if (!FMath::IsNearlyEqual(UnitScale, 1.))
	{
		for (FVector& Point : OutPointsArray)
		{
			Point *= UnitScale;
		}
	}
};

void FillPointArray(const int32 UCount, const int32 VCount, const A3DVector3dData* Points, TArray<FVector>& OutPointsArray, double UnitScale = 1.0)
{
	using namespace UE::CADKernel;

	OutPointsArray.SetNum(UCount * VCount);

	for (int32 Undex = 0, PointIndex = 0; Undex < UCount; ++Undex)
	{
		int32 Index = Undex;
		for (int32 Vndex = 0; Vndex < VCount; ++Vndex, Index += UCount, ++PointIndex)
		{
			OutPointsArray[Index].Set(Points[PointIndex].m_dX, Points[PointIndex].m_dY, Points[PointIndex].m_dZ);
		}
	}

	if (!FMath::IsNearlyEqual(UnitScale, 1.))
	{
		for (FVector& Point : OutPointsArray)
		{
			Point *= UnitScale;
		}
	}
};

UE::CADKernel::FSurfacicBoundary GetSurfacicBoundary(A3DDomainData& Domain, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
	using namespace UE::CADKernel;


	FVector2d Min(Domain.m_sMin.m_dX, Domain.m_sMin.m_dY);
	FVector2d Max(Domain.m_sMax.m_dX, Domain.m_sMax.m_dY);

	if (UVReparameterization.GetNeedApply())
	{
		UVReparameterization.Apply(Min);
		UVReparameterization.Apply(Max);
	}

	EIso UIndex = UVReparameterization.GetSwapUV() ? EIso::IsoV : EIso::IsoU;
	EIso VIndex = UVReparameterization.GetSwapUV() ? EIso::IsoU : EIso::IsoV;

	FSurfacicBoundary Boundary;
	Boundary[UIndex].Min = FMath::Min(Min.X, Max.X);
	Boundary[UIndex].Max = FMath::Max(Min.X, Max.X);
	Boundary[VIndex].Min = FMath::Min(Min.Y, Max.Y);
	Boundary[VIndex].Max = FMath::Max(Min.Y, Max.Y);

	return Boundary;
}

UE::CADKernel::FLinearBoundary GetLinearBoundary(A3DIntervalData& A3DDomain)
{
	UE::CADKernel::FLinearBoundary Domain(A3DDomain.m_dMin, A3DDomain.m_dMax);
	return Domain;
}

UE::CADKernel::FLinearBoundary GetLinearBoundary(const A3DCrvBase* A3DCurve)
{
	TUniqueTSObj<A3DIntervalData> A3DDomain(A3DCurve);
	UE::CADKernel::FLinearBoundary Domain(A3DDomain->m_dMin, A3DDomain->m_dMax);
	return Domain;
}

} // namespace TechSoftUtils

FTechSoftBridge::FTechSoftBridge(FTechSoftFileParser& InParser, UE::CADKernel::FSession& InSession)
	: Parser(InParser)
	, Session(InSession)
	, Model(InSession.GetModel())
	, GeometricTolerance(Session.GetGeometricTolerance())
	, EdgeLengthTolerance(Session.GetGeometricTolerance() * 2.)
	, SquareGeometricTolerance(FMath::Square(Session.GetGeometricTolerance()))
	, SquareJoiningVertexTolerance(SquareGeometricTolerance * 2)
{
}

const A3DRiBrepModel* FTechSoftBridge::GetA3DBody(UE::CADKernel::FBody* BRepModel)
{
	const A3DEntity** BodyPtr = CADKernelToTechSoft.Find(BRepModel);
	if (BodyPtr)
	{
		return (A3DRiBrepModel*)*BodyPtr;
	}
	return nullptr;
}

UE::CADKernel::FBody* FTechSoftBridge::GetBody(A3DRiBrepModel* A3DBRepModel)
{
	TSharedPtr<UE::CADKernel::FBody>* BodyPtr = TechSoftToCADKernel.Find(A3DBRepModel);
	if (BodyPtr && !(*BodyPtr)->IsDeleted())
	{
		return (*BodyPtr).Get();
	}
	return nullptr;
}

UE::CADKernel::FBody* FTechSoftBridge::AddBody(A3DRiBrepModel* A3DBRepModel, FArchiveCADObject& ArchiveBody)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().BodyCount++;
#endif

	//UE::CADKernel working unit is mm
	BodyScale = ArchiveBody.Unit * 10.;

	FArchiveCADObject BRepMetaData;
	Parser.ExtractMetaData(A3DBRepModel, BRepMetaData);

	FString* Name = ArchiveBody.MetaData.Find(TEXT("Name"));
	if (Name != nullptr)
	{
		BRepMetaData.MetaData.FindOrAdd(TEXT("Name")) = *Name;
	}

	if (TSharedPtr<UE::CADKernel::FBody>* BodyPtr = TechSoftToCADKernel.Find(A3DBRepModel))
	{
		return (*BodyPtr)->IsDeleted() ? nullptr : (*BodyPtr).Get();
	}

	TSharedRef<UE::CADKernel::FBody> Body = UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FBody>();
	AddMetaData(BRepMetaData, *Body);

	Body->SetDisplayData(ArchiveBody.ColorUId, ArchiveBody.MaterialUId);

	TUniqueTSObj<A3DRiBrepModelData> BRepModelData(A3DBRepModel);
	if (BRepModelData.IsValid())
	{
		TraverseBrepData(BRepModelData->m_pBrepData, Body);
	}

	if (Body->FaceCount() == 0 || bConvertionFailed)
	{
		Body->Delete();
		return nullptr;
	}

	Model.Add(Body);
	TechSoftToCADKernel.Add(A3DBRepModel, Body);
	CADKernelToTechSoft.Add(&*Body, A3DBRepModel);

	return &*Body;
}

void FTechSoftBridge::TraverseBrepData(const A3DTopoBrepData* A3DBrepData, TSharedRef<UE::CADKernel::FBody>& Body)
{
	FArchiveCADObject MetaData;
	Parser.ExtractMetaData(A3DBrepData, MetaData);

	{
		TUniqueTSObj<A3DTopoBodyData> TopoBodyData(A3DBrepData);
		if (TopoBodyData.IsValid())
		{
			if (TopoBodyData->m_pContext)
			{
				TUniqueTSObj<A3DTopoContextData> TopoContextData(TopoBodyData->m_pContext);
				if (TopoContextData.IsValid())
				{
					if (TopoContextData->m_bHaveScale)
					{
						BodyScale *= TopoContextData->m_dScale;
					}
				}
			}
		}
	}

	TUniqueTSObj<A3DTopoBrepDataData> TopoBrepData(A3DBrepData);
	if (TopoBrepData.IsValid())
	{
		for (A3DUns32 Index = 0; Index < TopoBrepData->m_uiConnexSize; ++Index)
		{
			TraverseConnex(TopoBrepData->m_ppConnexes[Index], Body);
			if (bConvertionFailed)
			{
				return;
			}
		}
	}
}

void FTechSoftBridge::TraverseConnex(const A3DTopoConnex* A3DTopoConnex, TSharedRef<UE::CADKernel::FBody>& Body)
{
	FArchiveCADObject MetaData;
	Parser.ExtractMetaData(A3DTopoConnex, MetaData);

	TUniqueTSObj<A3DTopoConnexData> TopoConnexData(A3DTopoConnex);
	if (TopoConnexData.IsValid())
	{
		for (A3DUns32 Index = 0; Index < TopoConnexData->m_uiShellSize; ++Index)
		{
			TraverseShell(TopoConnexData->m_ppShells[Index], Body);
			if (bConvertionFailed)
			{
				return;
			}
		}
	}
}

void FTechSoftBridge::TraverseShell(const A3DTopoShell* A3DShell, TSharedRef<UE::CADKernel::FBody>& Body)
{
	FArchiveCADObject MetaData;
	Parser.ExtractMetaData(A3DShell, MetaData);

	TSharedRef<UE::CADKernel::FShell> Shell = UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FShell>();
	Body->AddShell(Shell);
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().ShellCount++;
#endif

	Shell->SetDisplayData(*Body);
	AddMetaData(MetaData, *Shell);

	TUniqueTSObj<A3DTopoShellData> ShellData(A3DShell);

	if (ShellData.IsValid())
	{
		A3DEdgeToEdge.Empty();
		for (A3DUns32 Index = 0; Index < ShellData->m_uiFaceSize; ++Index)
		{
			AddFace(ShellData->m_ppFaces[Index], ShellData->m_pucOrientationWithShell[Index] == 1 ? UE::CADKernel::Front : UE::CADKernel::Back, Shell, Index);
			if (bConvertionFailed)
			{
				return;
			}
		}
	}
}

static bool bUseCurveAsNurbs = true;
TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurve(const A3DCrvBase* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
	TSharedPtr<UE::CADKernel::FCurve> Curve = TSharedPtr<UE::CADKernel::FCurve>();
	A3DEEntityType eType;
	A3DInt32 Ret = TechSoftInterface::GetEntityType(A3DCurve, &eType);
	if (Ret == A3D_SUCCESS)
	{
#ifdef CADKERNEL_DEV
		UE::CADKernel::FCADFileReport::Get().CurveCount++;
#endif
		switch (eType)
		{
		case kA3DTypeCrvNurbs:
			Curve = AddCurveNurbs(A3DCurve, UVReparameterization);
			break;
		case kA3DTypeCrvLine:
			Curve = AddCurveLine(A3DCurve, UVReparameterization);
			break;
		case kA3DTypeCrvCircle:
			Curve = AddCurveCircle(A3DCurve, UVReparameterization);
			break;
		case kA3DTypeCrvEllipse:
			Curve = AddCurveEllipse(A3DCurve, UVReparameterization);
			break;
		case kA3DTypeCrvParabola:
			Curve = AddCurveParabola(A3DCurve, UVReparameterization);
			break;
		case kA3DTypeCrvHyperbola:
			Curve = AddCurveHyperbola(A3DCurve, UVReparameterization);
			break;
		case kA3DTypeCrvHelix:
			Curve = AddCurveHelix(A3DCurve, UVReparameterization);
			break;
		case kA3DTypeCrvPolyLine:
			Curve = AddCurvePolyLine(A3DCurve, UVReparameterization);
			break;
		case kA3DTypeCrvComposite:
			Curve = AddCurveComposite(A3DCurve, UVReparameterization);
			break;
		default:
			Curve = AddCurveAsNurbs(A3DCurve, UVReparameterization);
			break;
		}
	}

	UE::CADKernel::FLinearBoundary Boundary = TechSoftUtils::GetLinearBoundary(A3DCurve);

	return Curve;
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveLine(const A3DCrvLine* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveLineCount++;
#endif

	if (bUseCurveAsNurbs)
	{
		return AddCurveAsNurbs(A3DCurve, UVReparameterization);
	}


	TUniqueTSObj<A3DCrvLineData> CrvLineData(A3DCurve);
	if (!CrvLineData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	bool bIs2D = (bool)CrvLineData->m_bIs2D;
	// Todo
	return TSharedPtr<UE::CADKernel::FCurve>();
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveCircle(const A3DCrvCircle* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveCircleCount++;
#endif

	if (bUseCurveAsNurbs)
	{
		return AddCurveAsNurbs(A3DCurve, UVReparameterization);
	}

	TUniqueTSObj<A3DCrvCircleData> CrvCircleData(A3DCurve);
	if (!CrvCircleData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	bool bIs2D = (bool)CrvCircleData->m_bIs2D;
	// Todo
	return TSharedPtr<UE::CADKernel::FCurve>();
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveEllipse(const A3DCrvEllipse* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveEllipseCount++;
#endif

	if (bUseCurveAsNurbs)
	{
		return AddCurveAsNurbs(A3DCurve, UVReparameterization);
	}

	TUniqueTSObj<A3DCrvEllipseData> CrvEllipseData(A3DCurve);
	if (!CrvEllipseData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	bool bIs2D = (bool)CrvEllipseData->m_bIs2D;
	// Todo
	return TSharedPtr<UE::CADKernel::FCurve>();
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveParabola(const A3DCrvParabola* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveParabolaCount++;
#endif

	if (bUseCurveAsNurbs)
	{
		return AddCurveAsNurbs(A3DCurve, UVReparameterization);
	}

	TUniqueTSObj<A3DCrvParabolaData> CrvParabolaData(A3DCurve);
	if (!CrvParabolaData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	bool bIs2D = (bool)CrvParabolaData->m_bIs2D;
	// Todo
	return TSharedPtr<UE::CADKernel::FCurve>();
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveHyperbola(const A3DCrvHyperbola* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveHyperbolaCount++;
#endif

	if (bUseCurveAsNurbs)
	{
		return AddCurveAsNurbs(A3DCurve, UVReparameterization);
	}

	TUniqueTSObj<A3DCrvHyperbolaData> CrvHyperbolaData(A3DCurve);
	if (!CrvHyperbolaData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	bool bIs2D = (bool)CrvHyperbolaData->m_bIs2D;
	// Todo
	return TSharedPtr<UE::CADKernel::FCurve>();
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveHelix(const A3DCrvHelix* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveHelixCount++;
#endif

	if (bUseCurveAsNurbs)
	{
		return AddCurveAsNurbs(A3DCurve, UVReparameterization);
	}

	TUniqueTSObj<A3DCrvHelixData> CrvHelixData(A3DCurve);
	if (!CrvHelixData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	bool bIs2D = (bool)CrvHelixData->m_bIs2D;

	return TSharedPtr<UE::CADKernel::FCurve>();
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurvePolyLine(const A3DCrvPolyLine* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurvePolyLineCount++;
#endif

	if (bUseCurveAsNurbs)
	{
		return AddCurveAsNurbs(A3DCurve, UVReparameterization);
	}

	TUniqueTSObj<A3DCrvPolyLineData> CrvPolyLineData(A3DCurve);
	if (!CrvPolyLineData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	bool bIs2D = (bool)CrvPolyLineData->m_bIs2D;
	// Todo
	return TSharedPtr<UE::CADKernel::FCurve>();
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveComposite(const A3DCrvComposite* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveCompositeCount++;
#endif

	if (bUseCurveAsNurbs)
	{
		return AddCurveAsNurbs(A3DCurve, UVReparameterization);
	}

	TUniqueTSObj<A3DCrvCompositeData> CrvCompositeData(A3DCurve);
	if (!CrvCompositeData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	bool bIs2D = (bool)CrvCompositeData->m_bIs2D;
	// Todo
	return TSharedPtr<UE::CADKernel::FCurve>();
}

TSharedPtr<UE::CADKernel::FCurve> AddCurveNurbsFromData(A3DCrvNurbsData& A3DNurbs, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
	UE::CADKernel::FNurbsCurveData Nurbs;
	Nurbs.Dimension = A3DNurbs.m_bIs2D ? 2 : 3;
	Nurbs.bIsRational = (bool)A3DNurbs.m_bRational;
	Nurbs.Degree = A3DNurbs.m_uiDegree;

	TechSoftUtils::FillPointArray(A3DNurbs.m_uiCtrlSize, A3DNurbs.m_pCtrlPts, Nurbs.Poles);
	if (Nurbs.Dimension == 2)
	{
		UVReparameterization.Process(Nurbs.Poles);
	}

	TechSoftUtils::FillDoubleArray(A3DNurbs.m_uiKnotSize, A3DNurbs.m_pdKnots, Nurbs.NodalVector);
	if (Nurbs.bIsRational)
	{
		TechSoftUtils::FillDoubleArray(A3DNurbs.m_uiCtrlSize, A3DNurbs.m_pdWeights, Nurbs.Weights);
	}

	A3DCrvNurbsGet(NULL, &A3DNurbs);

	return UE::CADKernel::FCurve::MakeNurbsCurve(Nurbs);
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveNurbs(const A3DCrvNurbs* A3DNurbs, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveLineCount++;
#endif

	TUniqueTSObj<A3DCrvNurbsData> CrvNurbsData(A3DNurbs);
	if (!CrvNurbsData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	return AddCurveNurbsFromData(*CrvNurbsData, UVReparameterization);
}

TSharedPtr<UE::CADKernel::FCurve> FTechSoftBridge::AddCurveAsNurbs(const A3DCrvBase* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CurveAsNurbsCount++;
#endif

	TUniqueTSObj<A3DCrvNurbsData> NurbsData;

	A3DDouble Tolerance = 1e-3;
	A3DBool bUseSameParameterization = true;
	NurbsData.FillWith(&TechSoftUtils::GetCurveAsNurbs, A3DCurve, Tolerance, bUseSameParameterization);

	if (!NurbsData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FCurve>();
	}

	return AddCurveNurbsFromData(*NurbsData, UVReparameterization);
}

TSharedPtr<UE::CADKernel::FTopologicalEdge> FTechSoftBridge::AddEdge(const A3DTopoCoEdge* A3DCoedge, const TSharedRef<UE::CADKernel::FSurface>& Surface, const TechSoftUtils::FUVReparameterization& UVReparameterization, UE::CADKernel::EOrientation& OutOrientation)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().EdgeCount++;
#endif

	TUniqueTSObj<A3DTopoCoEdgeData> CoEdgeData(A3DCoedge);
	if (!CoEdgeData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FTopologicalEdge>();
	}

	if (CoEdgeData->m_pUVCurve == nullptr)
	{
		bConvertionFailed = true;
		FailureReason = EFailureReason::Curve3D;
		return TSharedPtr<UE::CADKernel::FTopologicalEdge>();
	}

	TSharedPtr<UE::CADKernel::FCurve> Curve = AddCurve(CoEdgeData->m_pUVCurve, UVReparameterization);
	if (!Curve.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FTopologicalEdge>();
	}

	TSharedRef<UE::CADKernel::FRestrictionCurve> RestrictionCurve = UE::CADKernel::FEntity::MakeShared<UE::CADKernel::FRestrictionCurve>(Surface, Curve.ToSharedRef());

	TSharedPtr<UE::CADKernel::FTopologicalEdge> Edge = UE::CADKernel::FTopologicalEdge::Make(RestrictionCurve);
	if (!Edge.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FTopologicalEdge>();
	}

	A3DEdgeToEdge.Emplace(A3DCoedge, Edge);

	OutOrientation = CoEdgeData->m_ucOrientationUVWithLoop > 0 ? UE::CADKernel::EOrientation::Front : UE::CADKernel::EOrientation::Back;

	return Edge;
}

TSharedPtr<UE::CADKernel::FTopologicalLoop> FTechSoftBridge::AddLoop(const A3DTopoLoop* A3DLoop, const TSharedRef<UE::CADKernel::FSurface>& Surface, const TechSoftUtils::FUVReparameterization& UVReparameterization, const bool bIsExternalLoop)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().LoopCount++;
#endif

	TArray<TSharedPtr<UE::CADKernel::FTopologicalEdge>> Edges;
	TArray<UE::CADKernel::EOrientation> Directions;

	TUniqueTSObj<A3DTopoLoopData> TopoLoopData(A3DLoop);
	if (!TopoLoopData.IsValid())
	{
#ifdef CADKERNEL_DEV
		UE::CADKernel::FCADFileReport::Get().DegeneratedLoopCount++;
#endif
		return TSharedPtr<UE::CADKernel::FTopologicalLoop>();
	}

	bool bLoopOrientation = (bool)TopoLoopData->m_ucOrientationWithSurface;
	for (A3DUns32 Index = 0; Index < TopoLoopData->m_uiCoEdgeSize; ++Index)
	{
		UE::CADKernel::EOrientation Orientation;
		TSharedPtr<UE::CADKernel::FTopologicalEdge> Edge = AddEdge(TopoLoopData->m_ppCoEdges[Index], Surface, UVReparameterization, Orientation);
		if (!Edge.IsValid())
		{
#ifdef CADKERNEL_DEV
			UE::CADKernel::FCADFileReport::Get().DegeneratedEdgeCount++;
#endif
			continue;
		}

		Edges.Emplace(Edge);
		Directions.Emplace(Orientation);
	}

	if (Edges.Num() == 0)
	{
#ifdef CADKERNEL_DEV
		UE::CADKernel::FCADFileReport::Get().DegeneratedLoopCount++;
#endif
		return TSharedPtr<UE::CADKernel::FTopologicalLoop>();
	}

	TSharedPtr<UE::CADKernel::FTopologicalLoop> Loop = UE::CADKernel::FTopologicalLoop::Make(Edges, Directions, bIsExternalLoop, GeometricTolerance);

	// Link the edges of the loop with their neighbors if possible
	for (A3DUns32 Index = 0; Index < TopoLoopData->m_uiCoEdgeSize; ++Index)
	{
		const A3DTopoCoEdge* A3DCoedge = TopoLoopData->m_ppCoEdges[Index];
		TSharedPtr<UE::CADKernel::FTopologicalEdge>* Edge = A3DEdgeToEdge.Find(A3DCoedge);
		if (Edge == nullptr || !Edge->IsValid() || (*Edge)->IsDeleted())
		{
			continue;
		}

		TUniqueTSObj<A3DTopoCoEdgeData> CoEdgeData(A3DCoedge);
		if (!CoEdgeData.IsValid())
		{
			continue;
		}

		if (CoEdgeData->m_pNeighbor)
		{
			const A3DTopoCoEdge* Neighbor = CoEdgeData->m_pNeighbor;
			while (Neighbor && Neighbor != A3DCoedge)
			{
				TSharedPtr<UE::CADKernel::FTopologicalEdge>* TwinEdge = A3DEdgeToEdge.Find(Neighbor);
				if (TwinEdge != nullptr && TwinEdge->IsValid() && !(*TwinEdge)->IsDeleted())
				{
					(*Edge)->LinkIfCoincident(*TwinEdge->Get(), EdgeLengthTolerance, SquareJoiningVertexTolerance);
				}

				// Next
				TUniqueTSObj<A3DTopoCoEdgeData> NeighborData(Neighbor);
				if (NeighborData.IsValid())
				{
					Neighbor = NeighborData->m_pNeighbor;
				}
				else
				{
					break;
				}
			}
		}
	}

	return Loop;
}

void FTechSoftBridge::AddFace(const A3DTopoFace* A3DFace, UE::CADKernel::EOrientation Orientation, TSharedRef<UE::CADKernel::FShell>& Shell, uint32 ShellIndex)
{
	using namespace UE::CADKernel;

#ifdef CADKERNEL_DEV
	FCADFileReport::Get().FaceCount++;
#endif

	FArchiveCADObject MetaData;
	Parser.ExtractMetaData(A3DFace, MetaData);

	TUniqueTSObj<A3DTopoFaceData> TopoFaceData(A3DFace);
	if (!TopoFaceData.IsValid())
	{
#ifdef CADKERNEL_DEV
		FCADFileReport::Get().FailedFaceCount++;
#endif
		return;
	}

	const A3DSurfBase* A3DSurface = TopoFaceData->m_pSurface;
	TechSoftUtils::FUVReparameterization UVReparameterization;
	TSharedPtr<FSurface> SurfacePtr = AddSurface(A3DSurface, UVReparameterization);
	if (!SurfacePtr.IsValid())
	{
#ifdef CADKERNEL_DEV
		FCADFileReport::Get().DegeneratedSurfaceCount++;
		FCADFileReport::Get().FailedFaceCount++;
#endif
		return;
	}

	if (UVReparameterization.GetNeedSwapOrientation())
	{
		SwapOrientation(Orientation);
	}

	TSharedRef<FSurface> Surface = SurfacePtr.ToSharedRef();
	TSharedRef<FTopologicalFace> Face = FEntity::MakeShared<FTopologicalFace>(Surface);

	if (TopoFaceData->m_bHasTrimDomain)
	{
		FSurfacicBoundary SurfaceBoundary = TechSoftUtils::GetSurfacicBoundary(TopoFaceData->m_sSurfaceDomain, UVReparameterization);
		Surface->TrimBoundaryTo(SurfaceBoundary);
	}

	if (!TopoFaceData->m_uiLoopSize)
	{
		Face->ApplyNaturalLoops();
	}
	else
	{
		TArray<TSharedPtr<FTopologicalLoop>> Loops;

		const uint32 OuterLoopIndex = TopoFaceData->m_uiOuterLoopIndex;

		for (A3DUns32 Index = 0; Index < TopoFaceData->m_uiLoopSize; ++Index)
		{
			const bool bIsExternalLoop = (Index == OuterLoopIndex);
			TSharedPtr<FTopologicalLoop> Loop = AddLoop(TopoFaceData->m_ppLoops[Index], Surface, UVReparameterization, bIsExternalLoop);
			if (!Loop.IsValid())
			{
				continue;
			}

			TArray<FVector2d> LoopSampling;
			Loop->Get2DSampling(LoopSampling);
			FAABB2D Boundary;
			Boundary += LoopSampling;
			Loop->Boundary.Set(Boundary.GetMin(), Boundary.GetMax());

			// Check if the loop is not composed with only degenerated edge
			bool bDegeneratedLoop = true;
			for (const FOrientedEdge& Edge : Loop->GetEdges())
			{
				if (!Edge.Entity->IsDegenerated())
				{
					bDegeneratedLoop = false;
					break;
				}
			}
			if (bDegeneratedLoop)
			{
				continue;
			}

			Loops.Add(Loop);
		}

		if (Loops.Num() > 1)
		{
			// Check external loop
			TSharedPtr<FTopologicalLoop> ExternalLoop;
			FSurfacicBoundary ExternalBoundary;
			ExternalBoundary.Init();
			for (const TSharedPtr<FTopologicalLoop>& Loop : Loops)
			{
				// fast but not accurate test to check if the loop is inside an other loop based of the bounding box 
				switch (Loop->Boundary.IsInside(ExternalBoundary, Surface->GetIsoTolerances()))
				{
				case ESituation::Undefined:
				{
					// accurate test to check if the loop is inside an other loop 
					if (!Loop->IsInside(*ExternalLoop))
					{
						ExternalBoundary = Loop->Boundary;
						ExternalLoop = Loop;
					}
					break;
				}

				case ESituation::Outside:
				{
					ExternalBoundary = Loop->Boundary;
					ExternalLoop = Loop;
					break;
				}

				default:
					break;
				}
			}

			if (!ExternalLoop->IsExternal())
			{
				for (TSharedPtr<FTopologicalLoop>& Loop : Loops)
				{
					if (Loop->IsExternal())
					{
						Loop->SetInternal();
						break;
					}
				}
				ExternalLoop->SetExternal();
			}
		}

		if (Loops.Num() == 0)
		{
#ifdef CADKERNEL_DEV
			FCADFileReport::Get().FailedFaceCount++;
#endif
			Face->SetAsDegenerated();
			Face->Delete();
			return;
		}
		else
		{
			int32 DoubtfulLoopOrientationCount = 0;
			Face->AddLoops(Loops, DoubtfulLoopOrientationCount);
#ifdef CADKERNEL_DEV
			FCADFileReport::Get().DoubtfulLoopOrientationCount += DoubtfulLoopOrientationCount;
#endif
		}
	}

	if (Face->GetLoops().Num() == 0)
	{
		Face->SetAsDegenerated();
		Face->Delete();
		return;
	}

	AddMetaData(MetaData, *Face);
	Face->CompleteMetaData();

	Face->SetHostId(ShellIndex);
	Shell->Add(Face, Orientation);
}

static bool bUseSurfaceAsNurbs = true;

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddSurface(const A3DSurfBase* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().SurfaceCount++;
#endif

	FArchiveCADObject MetaData;
	Parser.ExtractMetaData(A3DSurface, MetaData);

	A3DEEntityType Type;
	int32 Ret = TechSoftInterface::GetEntityType(A3DSurface, &Type);
	if (Ret == A3D_SUCCESS)
	{
		switch (Type)
		{
		case kA3DTypeSurfBlend01:
			return AddBlend01Surface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfBlend02:
			return AddBlend02Surface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfBlend03:
			return AddBlend03Surface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfNurbs:
			return AddNurbsSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfCone:
			return AddConeSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfCylinder:
			return AddCylinderSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfCylindrical:
			return AddCylindricalSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfOffset:
			return AddOffsetSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfPipe:
			return AddPipeSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfPlane:
			return AddPlaneSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfRuled:
			return AddRuledSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfSphere:
			return AddSphereSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfRevolution:
			return AddRevolutionSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfExtrusion:
			return AddExtrusionSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfFromCurves:
			return AddSurfaceFromCurves(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfTorus:
			return AddTorusSurface(A3DSurface, OutUVReparameterization);

		case kA3DTypeSurfTransform:
			return AddTransformSurface(A3DSurface, OutUVReparameterization);

		default:
			return AddSurfaceAsNurbs(A3DSurface, OutUVReparameterization);
		}
	}
	else if (Ret == A3D_NOT_IMPLEMENTED)
	{
		return AddSurfaceAsNurbs(A3DSurface, OutUVReparameterization);
	}
	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddConeSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().ConeSurfaceCount++;
#endif

	TUniqueTSObj<A3DSurfConeData> A3DConeData(Surface);
	if (!A3DConeData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	OutUVReparameterization.AddUVTransform(A3DConeData->m_sParam);
	OutUVReparameterization.ScaleUVTransform(1, BodyScale);
	if (A3DConeData->m_dSemiAngle < 0)
	{
		OutUVReparameterization.SetNeedSwapOrientation();
	}

	UE::CADKernel::FMatrixH CoordinateSystem = TechSoftUtils::CreateCoordinateSystem(A3DConeData->m_sTrsf, BodyScale);
	UE::CADKernel::FSurfacicBoundary Boundary = TechSoftUtils::GetSurfacicBoundary(A3DConeData->m_sParam.m_sUVDomain, OutUVReparameterization);
	return UE::CADKernel::FSurface::MakeConeSurface(GeometricTolerance, CoordinateSystem, A3DConeData->m_dRadius * BodyScale, A3DConeData->m_dSemiAngle, Boundary);
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddCylinderSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CylinderSurfaceCount++;
#endif

	TUniqueTSObj<A3DSurfCylinderData> A3DCylinderData(Surface);
	if (!A3DCylinderData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	OutUVReparameterization.AddUVTransform(A3DCylinderData->m_sParam);
	OutUVReparameterization.ScaleUVTransform(1, BodyScale);

	UE::CADKernel::FMatrixH CoordinateSystem = TechSoftUtils::CreateCoordinateSystem(A3DCylinderData->m_sTrsf, BodyScale);
	UE::CADKernel::FSurfacicBoundary Boundary = TechSoftUtils::GetSurfacicBoundary(A3DCylinderData->m_sParam.m_sUVDomain, OutUVReparameterization);
	return UE::CADKernel::FSurface::MakeCylinderSurface(GeometricTolerance, CoordinateSystem, A3DCylinderData->m_dRadius * BodyScale, Boundary);
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddLinearTransfoSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().LinearTransfoSurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}
	// Todo
	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddNurbsSurface(const A3DSurfNurbs* Nurbs, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().NurbsSurfaceCount++;
#endif

	TUniqueTSObj<A3DSurfNurbsData> A3DNurbsData(Nurbs);
	if (!A3DNurbsData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return AddSurfaceNurbs(*A3DNurbsData, OutUVReparameterization);
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddOffsetSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().OffsetSurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddPlaneSurface(const A3DSurfPlane* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().PlaneSurfaceCount++;
#endif

	TUniqueTSObj<A3DSurfPlaneData> A3DPlaneData(Surface);
	if (!A3DPlaneData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	OutUVReparameterization.AddUVTransform(A3DPlaneData->m_sParam);
	OutUVReparameterization.ScaleUVTransform(BodyScale, BodyScale);

	UE::CADKernel::FMatrixH CoordinateSystem = TechSoftUtils::CreateCoordinateSystem(A3DPlaneData->m_sTrsf, BodyScale);
	UE::CADKernel::FSurfacicBoundary Boundary = TechSoftUtils::GetSurfacicBoundary(A3DPlaneData->m_sParam.m_sUVDomain, OutUVReparameterization);
	return UE::CADKernel::FSurface::MakePlaneSurface(GeometricTolerance, CoordinateSystem, Boundary);
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddRevolutionSurface(const A3DSurfRevolution* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().RevolutionSurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfRevolutionData> A3DRevolutionData(Surface);
	if (!A3DRevolutionData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddRuledSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().RuledSurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfRuledData> A3DRuledData(Surface);
	if (!A3DRuledData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddSphereSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().SphereSurfaceCount++;
#endif

	TUniqueTSObj<A3DSurfSphereData> A3DSphereData(Surface);
	if (!A3DSphereData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	OutUVReparameterization.AddUVTransform(A3DSphereData->m_sParam);

	UE::CADKernel::FMatrixH CoordinateSystem = TechSoftUtils::CreateCoordinateSystem(A3DSphereData->m_sTrsf, BodyScale);
	UE::CADKernel::FSurfacicBoundary Boundary = TechSoftUtils::GetSurfacicBoundary(A3DSphereData->m_sParam.m_sUVDomain, OutUVReparameterization);
	return UE::CADKernel::FSurface::MakeSphericalSurface(GeometricTolerance, CoordinateSystem, A3DSphereData->m_dRadius * BodyScale, Boundary);
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddTorusSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().TorusSurfaceCount++;
#endif

	TUniqueTSObj<A3DSurfTorusData> A3DTorusData(Surface);
	if (!A3DTorusData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	OutUVReparameterization.AddUVTransform(A3DTorusData->m_sParam);
	UE::CADKernel::FMatrixH CoordinateSystem = TechSoftUtils::CreateCoordinateSystem(A3DTorusData->m_sTrsf, BodyScale);
	UE::CADKernel::FSurfacicBoundary Boundary = TechSoftUtils::GetSurfacicBoundary(A3DTorusData->m_sParam.m_sUVDomain, OutUVReparameterization);
	return UE::CADKernel::FSurface::MakeTorusSurface(GeometricTolerance, CoordinateSystem, A3DTorusData->m_dMajorRadius * BodyScale, A3DTorusData->m_dMinorRadius * BodyScale, Boundary);
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddBlend01Surface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().Blend01SurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfBlend01Data> A3DBlend01Data(Surface);
	if (!A3DBlend01Data.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddBlend02Surface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().Blend02SurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfBlend02Data> A3DBlend02Data(Surface);
	if (!A3DBlend02Data.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddBlend03Surface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().Blend03SurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddCylindricalSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().CylindricalSurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfCylindricalData> A3DCylindricalData(Surface);
	if (!A3DCylindricalData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddPipeSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().PipeSurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfPipeData> A3DPipeData(Surface);
	if (!A3DPipeData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddExtrusionSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().ExtrusionSurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfExtrusionData> A3DExtrusionData(Surface);
	if (!A3DExtrusionData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddSurfaceFromCurves(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().SurfaceFromCurvesCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfFromCurvesData> A3DFromCurvesData(Surface);
	if (!A3DFromCurvesData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddTransformSurface(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().TransformSurfaceCount++;
#endif

	if (bUseSurfaceAsNurbs)
	{
		return AddSurfaceAsNurbs(Surface, OutUVReparameterization);
	}

	TUniqueTSObj<A3DSurfFromCurvesData> A3DTransformData(Surface);
	if (!A3DTransformData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return TSharedPtr<UE::CADKernel::FSurface>();
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddSurfaceNurbs(const A3DSurfNurbsData& A3DNurbsData, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
	using namespace UE::CADKernel;

	FNurbsSurfaceData NurbsData;

	NurbsData.PoleUCount = A3DNurbsData.m_uiUCtrlSize;
	NurbsData.PoleVCount = A3DNurbsData.m_uiVCtrlSize;
	int32 PoleCount = A3DNurbsData.m_uiUCtrlSize * A3DNurbsData.m_uiVCtrlSize;

	NurbsData.UDegree = A3DNurbsData.m_uiUDegree;
	NurbsData.VDegree = A3DNurbsData.m_uiVDegree;

	TechSoftUtils::FillDoubleArray(A3DNurbsData.m_uiUKnotSize, A3DNurbsData.m_pdUKnots, NurbsData.UNodalVector);
	TechSoftUtils::FillDoubleArray(A3DNurbsData.m_uiVKnotSize, A3DNurbsData.m_pdVKnots, NurbsData.VNodalVector);

	TArray<FVector> Poles;
	TechSoftUtils::FillPointArray(NurbsData.PoleUCount, NurbsData.PoleVCount, A3DNurbsData.m_pCtrlPts, NurbsData.Poles);
	if (!FMath::IsNearlyEqual(BodyScale, 1.))
	{
		for (FVector& Point : NurbsData.Poles)
		{
			Point *= BodyScale;
		}
	}

	bool bIsRational = false;
	if (A3DNurbsData.m_pdWeights)
	{
		bIsRational = true;
		TechSoftUtils::FillDoubleArray(NurbsData.PoleUCount, NurbsData.PoleVCount, A3DNurbsData.m_pdWeights, NurbsData.Weights);
	}

	return UE::CADKernel::FSurface::MakeNurbsSurface(GeometricTolerance, NurbsData);
}

TSharedPtr<UE::CADKernel::FSurface> FTechSoftBridge::AddSurfaceAsNurbs(const A3DSurfBase* Surface, TechSoftUtils::FUVReparameterization& OutUVReparameterization)
{
#ifdef CADKERNEL_DEV
	UE::CADKernel::FCADFileReport::Get().SurfaceAsNurbsCount++;
#endif

	TUniqueTSObj<A3DSurfNurbsData> A3DNurbsData;

	A3DDouble Tolerance = 1e-3;
	A3DBool bUseSameParameterization = true;
	A3DNurbsData.FillWith(&TechSoftUtils::GetSurfaceAsNurbs, Surface, Tolerance, bUseSameParameterization);

	if (!A3DNurbsData.IsValid())
	{
		return TSharedPtr<UE::CADKernel::FSurface>();
	}

	return AddSurfaceNurbs(*A3DNurbsData, OutUVReparameterization);

}

void FTechSoftBridge::AddMetaData(FArchiveCADObject& MetaData, UE::CADKernel::FTopologicalShapeEntity& Entity)
{
	FString* Name = MetaData.MetaData.Find(TEXT("Name"));
	if (Name != nullptr)
	{
		Entity.SetName(*Name);
	}
	Entity.SetDisplayData(MetaData.ColorUId, MetaData.MaterialUId);
}

}


#endif // USE_TECHSOFT_SDK
