// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Core/Types.h"

#ifdef USE_TECHSOFT_SDK

#include "Geo/GeoEnum.h"
#include "Math/MatrixH.h"
#include "CADOptions.h"

#include "TechSoftInterface.h"

namespace UE::CADKernel
{

class FBody;
class FCriterion;
class FCurve;
class FEntity;
class FFaceMesh;
class FModel;
class FRestrictionCurve;
class FSession;
class FShell;
class FSurface;
class FSurfacicBoundary;
class FTopologicalEdge;
class FTopologicalLoop;
class FTopologicalShapeEntity;

}

namespace CADLibrary
{

class FArchiveCADObject;
class FTechSoftFileParser;

namespace TechSoftUtils
{

class FUVReparameterization
{
private:
	double Scale[2] = { 1., 1. };
	double Offset[2] = { 0., 0. };
	bool bSwapUV = false;
	bool bNeedApply = false;
	bool bNeedSwapOrientation = false;

public:

	FUVReparameterization()
	{
	}

	void SetCoef(const double InUScale, const double InUOffset, const double InVScale, const double InVOffset)
	{
		Scale[UE::CADKernel::EIso::IsoU] = InUScale;
		Scale[UE::CADKernel::EIso::IsoV] = InVScale;
		Offset[UE::CADKernel::EIso::IsoU] = InUOffset;
		Offset[UE::CADKernel::EIso::IsoV] = InVOffset;
		SetNeedApply();
	}

	bool GetNeedApply() const
	{
		return bNeedApply;
	}

	bool GetSwapUV() const
	{
		return bSwapUV;
	}

	bool GetNeedSwapOrientation() const
	{
		return bNeedSwapOrientation != bSwapUV;
	}

	void SetNeedSwapOrientation()
	{
		bNeedSwapOrientation = true;
	}

	void SetNeedApply()
	{
		if (!FMath::IsNearlyEqual(Scale[UE::CADKernel::EIso::IsoU], 1.) || !FMath::IsNearlyEqual(Scale[UE::CADKernel::EIso::IsoV], 1.) || !FMath::IsNearlyEqual(Offset[UE::CADKernel::EIso::IsoU], 0.) || !FMath::IsNearlyEqual(Offset[UE::CADKernel::EIso::IsoV], 0.))
		{
			bNeedApply = true;
		}
		else
		{
			bNeedApply = false;
		}
	}

	void ScaleUVTransform(double InUScale, double InVScale)
	{
		if (bSwapUV)
		{
			Swap(InUScale, InVScale);
		}
		Scale[UE::CADKernel::EIso::IsoU] *= InUScale;
		Scale[UE::CADKernel::EIso::IsoV] *= InVScale;
		Offset[UE::CADKernel::EIso::IsoU] *= InUScale;
		Offset[UE::CADKernel::EIso::IsoV] *= InVScale;
		SetNeedApply();
	}

	void Process(TArray<FVector>& Poles) const
	{
		using namespace UE::CADKernel;

		if (bNeedApply)
		{
			for (FVector& Point : Poles)
			{
				Apply(Point);
			}
		}
		if (bSwapUV)
		{
			for (FVector& Point : Poles)
			{
				GetSwapUV(Point);
			}
		}
	}

	void AddUVTransform(A3DUVParameterizationData& Transform)
	{
		bSwapUV = (bool) Transform.m_bSwapUV;

		Scale[0] = Scale[0] * Transform.m_dUCoeffA;
		Scale[1] = Scale[1] * Transform.m_dVCoeffA;
		Offset[0] = Offset[0] * Transform.m_dUCoeffA + Transform.m_dUCoeffB;
		Offset[1] = Offset[1] * Transform.m_dVCoeffA + Transform.m_dVCoeffB;
		SetNeedApply();
	}

	void Apply(FVector2d& Point) const
	{
		using namespace UE::CADKernel;

		Point.X = Scale[EIso::IsoU] * Point.X + Offset[EIso::IsoU];
		Point.Y = Scale[EIso::IsoV] * Point.Y + Offset[EIso::IsoV];
	}

private:
	void Apply(FVector& Point) const
	{
		using namespace UE::CADKernel;

		Point.X = Scale[EIso::IsoU] * Point.X + Offset[EIso::IsoU];
		Point.Y = Scale[EIso::IsoV] * Point.Y + Offset[EIso::IsoV];
	}

	void GetSwapUV(FVector& Point) const
	{
		Swap(Point.X, Point.Y);
	}

};
} // ns TechSoftUtils

class FTechSoftBridge
{
private:
	FTechSoftFileParser& Parser;

	UE::CADKernel::FSession& Session;
	UE::CADKernel::FModel& Model;

	const double GeometricTolerance;
	const double EdgeLengthTolerance;
	const double SquareGeometricTolerance;
	const double SquareJoiningVertexTolerance;

	TMap<const A3DEntity*, TSharedPtr<UE::CADKernel::FBody>> TechSoftToCADKernel;
	TMap<UE::CADKernel::FBody*, const A3DEntity*> CADKernelToTechSoft;
	TMap<const A3DTopoCoEdge*, TSharedPtr<UE::CADKernel::FTopologicalEdge>> A3DEdgeToEdge;

	EFailureReason FailureReason = EFailureReason::None;
	bool bConvertionFailed = false;

	double BodyScale = 1;

public:
	FTechSoftBridge(FTechSoftFileParser& InParser, UE::CADKernel::FSession& InSession);
	UE::CADKernel::FBody* AddBody(A3DRiBrepModel* A3DBRepModel, FArchiveCADObject& ArchiveBody);
	UE::CADKernel::FBody* GetBody(A3DRiBrepModel* A3DBRepModel);
	const A3DRiBrepModel* GetA3DBody(UE::CADKernel::FBody* BRepModel);

	EFailureReason GetFailureReason()
	{
		return FailureReason;
	}

private:

	void TraverseBrepData(const A3DTopoBrepData* A3DBrepData, TSharedRef<UE::CADKernel::FBody>& Body);

	void TraverseConnex(const A3DTopoConnex* A3DConnex, TSharedRef<UE::CADKernel::FBody>& Body);
	void TraverseShell(const A3DTopoShell* A3DShell, TSharedRef<UE::CADKernel::FBody>& Body);
	void AddFace(const A3DTopoFace* A3DFace, UE::CADKernel::EOrientation Orientation, TSharedRef<UE::CADKernel::FShell>& Body, uint32 Index);

	TSharedPtr<UE::CADKernel::FTopologicalLoop> AddLoop(const A3DTopoLoop* A3DLoop, const TSharedRef<UE::CADKernel::FSurface>& Surface, const TechSoftUtils::FUVReparameterization& UVReparameterization, const bool bIsExternalLoop);
	TSharedPtr<UE::CADKernel::FTopologicalEdge> AddEdge(const A3DTopoCoEdge* A3DCoedge, const TSharedRef<UE::CADKernel::FSurface>& Surface, const TechSoftUtils::FUVReparameterization& UVReparameterization, UE::CADKernel::EOrientation& OutOrientation);

	TSharedPtr<UE::CADKernel::FSurface> AddSurface(const A3DSurfBase* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddConeSurface(const A3DSurfCone* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddCylinderSurface(const A3DSurfCylinder* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddLinearTransfoSurface(const A3DSurfBase* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddNurbsSurface(const A3DSurfNurbs* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddOffsetSurface(const A3DSurfOffset* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddPlaneSurface(const A3DSurfPlane* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddRevolutionSurface(const A3DSurfRevolution* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddRuledSurface(const A3DSurfRuled* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddSphereSurface(const A3DSurfSphere* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddTorusSurface(const A3DSurfTorus* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);

	TSharedPtr<UE::CADKernel::FSurface> AddBlend01Surface(const A3DSurfBlend01* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddBlend02Surface(const A3DSurfBlend02* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddBlend03Surface(const A3DSurfBlend03* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddCylindricalSurface(const A3DSurfCylindrical* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddPipeSurface(const A3DSurfPipe* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddExtrusionSurface(const A3DSurfExtrusion* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddSurfaceFromCurves(const A3DSurfFromCurves* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddTransformSurface(const A3DSurfTransform* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);

	TSharedPtr<UE::CADKernel::FSurface> AddSurfaceAsNurbs(const A3DSurfBase* A3DSurface, TechSoftUtils::FUVReparameterization& OutUVReparameterization);
	TSharedPtr<UE::CADKernel::FSurface> AddSurfaceNurbs(const A3DSurfNurbsData& A3DNurbsData, TechSoftUtils::FUVReparameterization& OutUVReparameterization);

	TSharedPtr<UE::CADKernel::FRestrictionCurve> AddRestrictionCurve(const A3DCrvBase* A3DCurve, const TSharedRef<UE::CADKernel::FSurface>& Surface);

	TSharedPtr<UE::CADKernel::FCurve> AddCurve(const A3DCrvBase* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);

	TSharedPtr<UE::CADKernel::FCurve> AddCurveCircle(const A3DCrvCircle* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveComposite(const A3DCrvComposite* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveEllipse(const A3DCrvEllipse* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveHelix(const A3DCrvHelix* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveHyperbola(const A3DCrvHyperbola* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveLine(const A3DCrvLine* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveNurbs(const A3DCrvNurbs* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveParabola(const A3DCrvParabola* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurvePolyLine(const A3DCrvPolyLine* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);

	TSharedPtr<UE::CADKernel::FCurve> AddCurveBlend02Boundary(const A3DCrvBlend02Boundary* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveEquation(const A3DCrvEquation* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveIntersection(const A3DCrvIntersection* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveOffset(const A3DCrvOffset* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveOnSurf(const A3DCrvOnSurf* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);
	TSharedPtr<UE::CADKernel::FCurve> AddCurveTransform(const A3DCrvTransform* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);

	TSharedPtr<UE::CADKernel::FCurve> AddCurveAsNurbs(const A3DCrvBase* A3DCurve, const TechSoftUtils::FUVReparameterization& UVReparameterization);

	void AddMetaData(FArchiveCADObject& EntityData, UE::CADKernel::FTopologicalShapeEntity& Entity);
};
}



#endif // USE_TECHSOFT_SDK
