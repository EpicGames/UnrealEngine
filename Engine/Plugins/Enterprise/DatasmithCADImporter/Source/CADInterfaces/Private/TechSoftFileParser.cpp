// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechSoftFileParser.h"

#include "CADFileData.h"
#include "CADOptions.h"
#include "TechSoftUtils.h"
#include "TechSoftUtilsPrivate.h"
#include "TUniqueTechSoftObj.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"

#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Tasks/Task.h"

#if WITH_EDITOR
#include "Async/Async.h"
#include "Misc/FeedbackContext.h"
#endif

#include "Templates/UnrealTemplate.h"

namespace CADLibrary
{

#ifdef USE_TECHSOFT_SDK

namespace ProductOccurrence
{

bool HasNoPartNoChildButHasReference(const A3DAsmProductOccurrenceData& OccurrenceData)
{
	return ((OccurrenceData.m_pPrototype != nullptr || OccurrenceData.m_pExternalData != nullptr) && OccurrenceData.m_pPart == nullptr && OccurrenceData.m_uiPOccurrencesSize == 0);
}

bool HasNoPartButHasReference(const A3DAsmProductOccurrenceData& OccurrenceData)
{
	return ((OccurrenceData.m_pPrototype != nullptr || OccurrenceData.m_pExternalData != nullptr) && OccurrenceData.m_pPart == nullptr);
}

bool HasNoChildButHasReference(const A3DAsmProductOccurrenceData& OccurrenceData)
{
	return ((OccurrenceData.m_pPrototype != nullptr || OccurrenceData.m_pExternalData != nullptr) && OccurrenceData.m_uiPOccurrencesSize == 0);
}

bool HasNoPartNoChild(const A3DAsmProductOccurrenceData& OccurrenceData)
{
	return (OccurrenceData.m_pPart == nullptr && OccurrenceData.m_uiPOccurrencesSize == 0);
}

bool HasPartOrChild(const A3DAsmProductOccurrenceData& OccurrenceData)
{
	return (OccurrenceData.m_pPart != nullptr || OccurrenceData.m_uiPOccurrencesSize != 0);
}

bool PrototypeIsValid(const A3DAsmProductOccurrenceData& OccurrenceData)
{
	return (OccurrenceData.m_pPrototype != nullptr || OccurrenceData.m_pExternalData != nullptr);
}

A3DAsmProductOccurrence* GetReference(const A3DAsmProductOccurrenceData& OccurrenceData)
{
	return OccurrenceData.m_pPrototype ? OccurrenceData.m_pPrototype : OccurrenceData.m_pExternalData;
}


}

namespace TechSoftFileParserImpl
{
#if WITH_EDITOR
	bool bGEnableProgressUpdate = true;
	FAutoConsoleVariableRef GCVarEnableProgressUpdate(
		TEXT("CADImport.Experimental.ProgressUpdate"),
		bGEnableProgressUpdate,
		TEXT("Experimental: If true, enables the display of progress for non-automated CAD import in the editor"),
		ECVF_Default);

	constexpr int32 MaxCounter = 200;
	static const double UpdateFrequency = 1. / 30.;
	class FTaskProgressUpdate
	{
	public:
		FTaskProgressUpdate(const FText& Text)
		: bIsValid(bGEnableProgressUpdate && GWarn && !GIsAutomationTesting)
		{
			if (bIsValid)
			{
				GWarn->BeginSlowTask(Text, true);
			}
		}

		~FTaskProgressUpdate()
		{
			if (bIsValid)
			{
				GWarn->EndSlowTask();
			}
		}

		void SetProgressText(const FText& Text)
		{
			if (bIsValid)
			{
				GWarn->StatusForceUpdate(static_cast<int32>(Counter++), MaxCounter, Text);
				Counter %= MaxCounter;
			}
		}

		void SetComplete(const FText& Text, const FText& Empty, bool bTaskSucceeded)
		{
			if (bIsValid)
			{
				GWarn->StatusForceUpdate(MaxCounter, MaxCounter, Text);
			}
		}

	private:
		int64 Counter = 0;
		bool bIsValid = false;
	};
#endif

// This code is a duplication code of CADFileReader::FindFile 
// This is done in 5.0.3 to avoid public header modification. 
// However this need to be rewrite in the next version. (Jira UE-152626)
bool UpdateFileDescriptor(FFileDescriptor& File)
{
	const FString& FileName = File.GetFileName();

	FString FilePath = FPaths::GetPath(File.GetSourcePath());
	FString RootFilePath = File.GetRootFolder();

	// Basic case: File exists at the initial path
	if (IFileManager::Get().FileExists(*File.GetSourcePath()))
	{
		return true;
	}

	// Advance case: end of FilePath is in a upper-folder of RootFilePath
	// e.g.
	// FilePath = D:\\data temp\\Unstructured project\\Folder2\\Added_Object.SLDPRT
	//                                                 ----------------------------
	// RootFilePath = D:\\data\\CAD Files\\SolidWorks\\p033 - Unstructured project\\Folder1
	//                ------------------------------------------------------------
	// NewPath = D:\\data\\CAD Files\\SolidWorks\\p033 - Unstructured project\\Folder2\\Added_Object.SLDPRT
	TArray<FString> RootPaths;
	RootPaths.Reserve(30);
	do
	{
		RootFilePath = FPaths::GetPath(RootFilePath);
		RootPaths.Emplace(RootFilePath);
	} while (!FPaths::IsDrive(RootFilePath) && !RootFilePath.IsEmpty());

	TArray<FString> FilePaths;
	FilePaths.Reserve(30);
	FilePaths.Emplace(FileName);
	while (!FPaths::IsDrive(FilePath) && !FilePath.IsEmpty())
	{
		FString FolderName = FPaths::GetCleanFilename(FilePath);
		FilePath = FPaths::GetPath(FilePath);
		FilePaths.Emplace(FPaths::Combine(FolderName, FilePaths.Last()));
	};

	for (int32 IndexFolderPath = 0; IndexFolderPath < RootPaths.Num(); IndexFolderPath++)
	{
		for (int32 IndexFilePath = 0; IndexFilePath < FilePaths.Num(); IndexFilePath++)
		{
			FString NewFilePath = FPaths::Combine(RootPaths[IndexFolderPath], FilePaths[IndexFilePath]);
			if (IFileManager::Get().FileExists(*NewFilePath))
			{
				File.SetSourceFilePath(NewFilePath);
				return true;
			};
		}
	}

	return false;
}

// Functions to clean metadata

inline void RemoveUnwantedChar(FString& StringToClean, TCHAR UnwantedChar)
{
	FString NewString;
	NewString.Reserve(StringToClean.Len());
	for (const TCHAR& Char : StringToClean)
	{
		if (Char != UnwantedChar)
		{
			NewString.AppendChar(Char);
		}
	}
	StringToClean = MoveTemp(NewString);
}

// Functions used in traverse model process

void TraverseAttribute(const A3DMiscAttributeData& AttributeData, TMap<FString, FString>& OutMetaData)
{
	FString AttributeFamillyName;
	if (AttributeData.m_bTitleIsInt)
	{
		A3DUns32 UnsignedValue = 0;
		memcpy(&UnsignedValue, AttributeData.m_pcTitle, sizeof(A3DUns32));
		AttributeFamillyName = FString::Printf(TEXT("%u"), UnsignedValue);
	}
	else if (AttributeData.m_pcTitle && AttributeData.m_pcTitle[0] != '\0')
	{
		AttributeFamillyName = UTF8_TO_TCHAR(AttributeData.m_pcTitle);
	}

	for (A3DUns32 Index = 0; Index < AttributeData.m_uiSize; ++Index)
	{
		FString AttributeName = AttributeFamillyName;
		{
			FString AttributeTitle = UTF8_TO_TCHAR(AttributeData.m_asSingleAttributesData[Index].m_pcTitle);
			if (AttributeTitle.Len())
			{
				AttributeName += TEXT(" ") + AttributeTitle;
			}
			else if(Index > 0)
			{
				AttributeName += TEXT(" ") + FString::FromInt((int32)Index);
			}
		}

		FString AttributeValue;
		switch (AttributeData.m_asSingleAttributesData[Index].m_eType)
		{
		case kA3DModellerAttributeTypeTime:
		case kA3DModellerAttributeTypeInt:
		{
			A3DInt32 Value;
			memcpy(&Value, AttributeData.m_asSingleAttributesData[Index].m_pcData, sizeof(A3DInt32));
			AttributeValue = FString::Printf(TEXT("%d"), Value);
			break;
		}

		case kA3DModellerAttributeTypeReal:
		{
			A3DDouble Value;
			memcpy(&Value, AttributeData.m_asSingleAttributesData[Index].m_pcData, sizeof(A3DDouble));
			AttributeValue = FString::Printf(TEXT("%f"), Value);
			break;
		}

		case kA3DModellerAttributeTypeString:
		{
			if (AttributeData.m_asSingleAttributesData[Index].m_pcData && AttributeData.m_asSingleAttributesData[Index].m_pcData[0] != '\0')
			{
				AttributeValue = UTF8_TO_TCHAR(AttributeData.m_asSingleAttributesData[Index].m_pcData);
			}
			break;
		}

		default:
			break;
		}

		if (AttributeName.Len())
		{
			OutMetaData.Emplace(AttributeName, AttributeValue);
		}
	}
}

void SetIOOption(A3DImport& Importer)
{
	using namespace CADLibrary;
	// A3DRWParamsGeneralData Importer.m_sGeneral
	Importer.m_sLoadData.m_sGeneral.m_bReadSolids = A3D_TRUE;
	Importer.m_sLoadData.m_sGeneral.m_bReadSurfaces = A3D_TRUE;
	Importer.m_sLoadData.m_sGeneral.m_bReadWireframes = A3D_FALSE;
	Importer.m_sLoadData.m_sGeneral.m_bReadPmis = A3D_FALSE;
	Importer.m_sLoadData.m_sGeneral.m_bReadAttributes = A3D_TRUE;
	Importer.m_sLoadData.m_sGeneral.m_bReadHiddenObjects = A3D_TRUE;
	Importer.m_sLoadData.m_sGeneral.m_bReadConstructionAndReferences = A3D_FALSE;
	Importer.m_sLoadData.m_sGeneral.m_bReadActiveFilter = A3D_FALSE;
	Importer.m_sLoadData.m_sGeneral.m_eReadingMode2D3D = kA3DRead_3D;

	Importer.m_sLoadData.m_sGeneral.m_eReadGeomTessMode = kA3DReadGeomAndTess/*kA3DReadGeomOnly*/;
	Importer.m_sLoadData.m_sGeneral.m_bReadFeature = A3D_FALSE;

	Importer.m_sLoadData.m_sGeneral.m_bReadConstraints = A3D_FALSE;

	Importer.m_sLoadData.m_sIncremental.m_bLoadNoDependencies = FImportParameters::bGEnableCADCache && GMaxImportThreads != 1;
	Importer.m_sLoadData.m_sIncremental.m_bLoadStructureOnly = false;
}

void UpdateIOOptionAccordingToFormat(const CADLibrary::ECADFormat Format, A3DImport& Importer, bool& OutForceSew)
{
	switch (Format)
	{
	case CADLibrary::ECADFormat::IGES:
	{
		OutForceSew = true;
		break;
	}

	case CADLibrary::ECADFormat::CATIA:
	{
		break;
	}

	case CADLibrary::ECADFormat::SOLIDWORKS:
	{
		Importer.m_sLoadData.m_sSpecifics.m_sSolidworks.m_bLoadAllConfigsData = true;
		break;
	}

	case CADLibrary::ECADFormat::JT:
	{
		Importer.m_sLoadData.m_sIncremental.m_bLoadNoDependencies = false;
		if (FImportParameters::bGPreferJtFileEmbeddedTessellation)
		{
			Importer.m_sLoadData.m_sGeneral.m_eReadGeomTessMode = kA3DReadTessOnly;
			Importer.m_sLoadData.m_sSpecifics.m_sJT.m_eReadTessellationLevelOfDetail = A3DEJTReadTessellationLevelOfDetail::kA3DJTTessLODHigh;
		}
		break;
	}

	case CADLibrary::ECADFormat::N_X:
	{
		Importer.m_sLoadData.m_sGeneral.m_bReadActiveFilter = A3D_TRUE;
		// jira UE-159972
		Importer.m_sLoadData.m_sIncremental.m_bLoadNoDependencies = false;
		break;
	}

	case CADLibrary::ECADFormat::INVENTOR:
	case CADLibrary::ECADFormat::CATIA_3DXML:
	{
		Importer.m_sLoadData.m_sIncremental.m_bLoadNoDependencies = false;
		break;
	}

	default:
		break;
	};
}

double ExtractUniformScale(FVector3d& Scale)
{
	double UniformScale = (Scale.X + Scale.Y + Scale.Z) / 3.;
	double Tolerance = UniformScale * DOUBLE_KINDA_SMALL_NUMBER;

	if (!FMath::IsNearlyEqual(UniformScale, Scale.X, Tolerance) && !FMath::IsNearlyEqual(UniformScale, Scale.Y, Tolerance))
	{
		// non uniform scale 
		// Used in format like IFC or DGN to define pipe by their diameter and their length
		// we remove the diameter component of the scale to have a scale like (Length/diameter, 1, 1) to have a mesh tessellated according the meshing parameters
		if (FMath::IsNearlyEqual((double)Scale.X, (double)Scale.Y, Tolerance) || (FMath::IsNearlyEqual((double)Scale.X, (double)Scale.Z, Tolerance)))
		{
			UniformScale = Scale.X;
		}
		else if (FMath::IsNearlyEqual((double)Scale.Y, (double)Scale.Z, Tolerance))
		{
			UniformScale = Scale.Y;
		}
	}

	Scale.X /= UniformScale;
	Scale.Y /= UniformScale;
	Scale.Z /= UniformScale;

	return UniformScale;
}

} // namespace TechSoftFileParserImpl

#endif

FTechSoftFileParser::FTechSoftFileParser(FCADFileData& InCADData, const FString& EnginePluginsPath)
	: CADFileData(InCADData)
	, SceneGraph(InCADData.GetSceneGraphArchive())
	, TechSoftInterface(FTechSoftInterface::Get())
{
	TechSoftInterface.InitializeKernel(*EnginePluginsPath);
}

#ifdef USE_TECHSOFT_SDK
ECADParsingResult FTechSoftFileParser::Process()
{
	bProcessIsRunning = true;

	TArray<UE::Tasks::FTask> Checkers;
	if(FImportParameters::bValidationProcess)
	{
		Checkers.Emplace(UE::Tasks::Launch(TEXT("MemoryChecker"), [&]() { CheckMemory(); }));
	}

	//FPlatformMemoryStats StartMem = FPlatformMemory::GetStats();
	uint64 StartTime = FPlatformTime::Cycles64();

	const FFileDescriptor& File = CADFileData.GetCADFileDescription();
	FImportRecord& ProcessReport = CADFileData.GetRecord();

	if (File.GetPathOfFileToLoad().IsEmpty())
	{
		return ECADParsingResult::FileNotFound;
	}

	A3DImport Import(TCHAR_TO_UTF8(*File.GetPathOfFileToLoad())); 

	TechSoftFileParserImpl::SetIOOption(Import);

	// Add specific options according to format
	Format = File.GetFileFormat();
	TechSoftFileParserImpl::UpdateIOOptionAccordingToFormat(Format, Import, bForceSew);

#if WITH_EDITOR
	TSharedPtr<TechSoftFileParserImpl::FTaskProgressUpdate> ProgressUpdate;

	auto UpdateNotification = [&ProgressUpdate](const FText& Text)
		{
			if (ProgressUpdate.IsValid())
			{
				ProgressUpdate->SetProgressText(Text);
			}
			else
			{
				ProgressUpdate = MakeShared<TechSoftFileParserImpl::FTaskProgressUpdate>(Text);
			}
		};
#endif

	A3DStatus LoadStatus = A3DStatus::A3D_SUCCESS;
#if WITH_EDITOR
	UpdateNotification(NSLOCTEXT("DatasmithCAD", "LoadModelFileFromFile_Started", "Loading Model"));

	TFuture<bool> LoadResult = Async(
		EAsyncExecution::LargeThreadPool,
		[this, &Import, &LoadStatus]() -> bool
		{
			ModelFile = TechSoftInterface::LoadModelFileFromFile(Import, LoadStatus);
			return true;
		}
	);

	while (!LoadResult.WaitFor(FTimespan::FromSeconds(TechSoftFileParserImpl::UpdateFrequency)))
	{
		UpdateNotification(NSLOCTEXT("DatasmithCAD", "LoadModelFileFromFile_InProgress", "Loading In Progress..."));
	}

	if (ProgressUpdate.IsValid())
	{
		FText ResultText;
		{
			if (ModelFile.IsValid())
			{
				ResultText = NSLOCTEXT("DatasmithCAD", "LoadModelFileFromFile_Done", "Loading Done");
			}
			else
			{
				ResultText = NSLOCTEXT("DatasmithCAD", "LoadModelFileFromFile_Failed", "Loading Failed. See log for details");
			}

			ProgressUpdate->SetProgressText(NSLOCTEXT("DatasmithCAD", "LoadModelFileFromFile_Completed", "Loading Completed"));
		}

		ProgressUpdate->SetComplete(ResultText, FText::GetEmpty(), ModelFile.IsValid());
		ProgressUpdate = nullptr;
	}
#else
	ModelFile = TechSoftInterface::LoadModelFileFromFile(Import, LoadStatus);
#endif

	ProcessReport.ImportTime += FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartTime);

	if (!ModelFile.IsValid())
	{
		switch (LoadStatus)
		{
		case A3DStatus::A3D_LOAD_FILE_TOO_OLD:
		{
			CADFileData.LogWarning(FString::Printf(TEXT("File %s hasn't been loaded because the version is less than the oldest supported version."), *File.GetFileName()));
			break;
		}

		case A3DStatus::A3D_LOAD_FILE_TOO_RECENT:
		{
			CADFileData.LogWarning(FString::Printf(TEXT("File %s hasn't been loaded because the version is more recent than supported version."), *File.GetFileName()));
			break;
		}

		case A3DStatus::A3D_LOAD_CANNOT_ACCESS_CADFILE:
		{
			CADFileData.LogWarning(FString::Printf(TEXT("File %s hasn't been loaded because the input path cannot be opened by the running process for reading."), *File.GetFileName()));
			break;
		}

		case A3DStatus::A3D_LOAD_INVALID_FILE_FORMAT:
		{
			CADFileData.LogWarning(FString::Printf(TEXT("File %s hasn't been loaded because the format is not supported."), *File.GetFileName()));
			break;
		}

		default:
		{
			CADFileData.LogWarning(FString::Printf(TEXT("File %s hasn't been loaded because an error occured while reading the file."), *File.GetFileName()));
			break;
		}
		};
		return ECADParsingResult::ProcessFailed;
	}

	{
		TUniqueTSObj<A3DAsmModelFileData> ModelFileData(ModelFile.Get());
		if (!ModelFileData.IsValid())
		{
			return ECADParsingResult::ProcessFailed;
		}

		ModelerType = (EModelerType) ModelFileData->m_eModellerType;
		FileUnit = TechSoftInterface::GetModelFileUnit(ModelFile.Get());
	}

	// save the file for the next load
	if (CADFileData.IsCacheDefined())
	{
		uint64 StartSaveTime = FPlatformTime::Cycles64();
		FString CacheFilePath = CADFileData.GetCADCachePath();
		if (CacheFilePath != File.GetPathOfFileToLoad())
		{
			TechSoftUtils::SaveModelFileToPrcFile(ModelFile.Get(), CacheFilePath);
		}
		ProcessReport.SavePrcTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartSaveTime);
	}

	// Adapt BRep to UE::CADKernel
	{
		uint64 StartAdaptTime = FPlatformTime::Cycles64();
#if WITH_EDITOR
		UpdateNotification(NSLOCTEXT("DatasmithCAD", "AdaptBRepModel_Started", "Converting Model"));

		TFuture<bool> AdaptResult = Async(
			EAsyncExecution::LargeThreadPool,
			[this]() -> bool
			{
				return AdaptBRepModel() == A3D_SUCCESS;
			}
		);

		while (!AdaptResult.WaitFor(FTimespan::FromSeconds(TechSoftFileParserImpl::UpdateFrequency)))
		{
			UpdateNotification(NSLOCTEXT("DatasmithCAD", "AdaptBRepModel_InProgress", "Conversion In Progress..."));
		}

		bool bAdaptationSucceeded = AdaptResult.Get();
		if (ProgressUpdate.IsValid())
		{
			FText ResultText;
			{
				if (bAdaptationSucceeded)
				{
					ResultText = NSLOCTEXT("DatasmithCAD", "AdaptBRepModel_Done", "Conversion Done");
				}
				else
				{
					ResultText = NSLOCTEXT("DatasmithCAD", "AdaptBRepModel_Failed", "Conversion Failed. See log for details");
				}

				ProgressUpdate->SetProgressText(NSLOCTEXT("DatasmithCAD", "AdaptBRepModel_Completed", "Conversion Completed"));
			}

			ProgressUpdate->SetComplete(ResultText, FText::GetEmpty(), ModelFile.IsValid());
			ProgressUpdate = nullptr;

			if (!bAdaptationSucceeded)
			{
				return ECADParsingResult::ProcessFailed;
			}
		}
#else
		if (AdaptBRepModel() != A3D_SUCCESS)
		{
			return ECADParsingResult::ProcessFailed;
		}
#endif
		ProcessReport.AdaptBRepTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartAdaptTime);
	}

	// Some formats (like IGES) require a sew all the time. In this case, bForceSew = true
	if (bForceSew || CADFileData.GetImportParameters().GetStitchingTechnique() == StitchingSew)
	{
		uint64 StartSewTime = FPlatformTime::Cycles64();
#if WITH_EDITOR
		UpdateNotification(NSLOCTEXT("DatasmithCAD", "Sewing_Started", "Sewing Model"));

		TFuture<bool> SewResult = Async(
			EAsyncExecution::LargeThreadPool,
			[this]() -> bool
			{
				SewModel();
				return true;
			}
		);

		while (!SewResult.WaitFor(FTimespan::FromSeconds(TechSoftFileParserImpl::UpdateFrequency)))
		{
			UpdateNotification(NSLOCTEXT("DatasmithCAD", "Sewing_InProgress", "Sewing In Progress..."));
		}

		if (ProgressUpdate.IsValid())
		{
			ProgressUpdate->SetProgressText(NSLOCTEXT("DatasmithCAD", "Sewing_Completed", "Sewing Completed"));
			ProgressUpdate->SetComplete(NSLOCTEXT("DatasmithCAD", "Sewing_Done", "Sewing Done"), FText::GetEmpty(), true);
			ProgressUpdate = nullptr;
		}
#else
		SewModel();
#endif
		ProcessReport.SewTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartSewTime);
	}

	uint64 StartTraversTime = FPlatformTime::Cycles64();

	ReserveCADFileData();
	ReadMaterialsAndColors();
	ECADParsingResult Result = TraverseModel();

	ProcessReport.ImportTime += FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartTraversTime);

	if (Result == ECADParsingResult::ProcessOk)
	{
#if WITH_EDITOR
		UpdateNotification(NSLOCTEXT("DatasmithCAD", "MeshingBRepModel_Started", "Generating Meshes"));

		TFuture<bool> MeshingResult = Async(
			EAsyncExecution::LargeThreadPool,
			[this]() -> bool
			{
				GenerateBodyMeshes();
				return true;
			}
		);

		while (!MeshingResult.WaitFor(FTimespan::FromSeconds(TechSoftFileParserImpl::UpdateFrequency)))
		{
			UpdateNotification(NSLOCTEXT("DatasmithCAD", "MeshingBRepModel_InProgress", "Generation In Progress..."));
		}

		if (ProgressUpdate.IsValid())
		{
			FText ResultText;
			{
				if (bConvertionFailed)
				{
					ResultText = NSLOCTEXT("DatasmithCAD", "MeshingBRepModel_Done", "Generation Done");
				}
				else
				{
					ResultText = NSLOCTEXT("DatasmithCAD", "MeshingBRepModel_Failed", "Generation Failed. See log for details");
				}

				ProgressUpdate->SetProgressText(NSLOCTEXT("DatasmithCAD", "MeshingBRepModel_Completed", "Generation Completed"));
			}

			ProgressUpdate->SetComplete(ResultText, FText::GetEmpty(), ModelFile.IsValid());
			ProgressUpdate = nullptr;
		}
#else
		GenerateBodyMeshes();
#endif
		if (bConvertionFailed)
		{
			Result = ECADParsingResult::ProcessFailed;
		}

		FString TechSoftVersion = TechSoftInterface::GetTechSoftVersion();
		if (!TechSoftVersion.IsEmpty())
		{
			FArchiveReference* RootReference = SceneGraph.GetRootReference();
			if (ensureMsgf(RootReference, TEXT("FTechSoftFileParser expected root Reference in the Scene Graph")))
			{
				RootReference->MetaData.Add(TEXT("TechsoftVersion"), TechSoftVersion);
			}
		}
	}

	ModelFile.Reset();

	ProcessReport.LoadProcessTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartTime);

	bProcessIsRunning = false;

	UE::Tasks::Wait(Checkers);

	return Result;
}

void FTechSoftFileParser::SewModel()
{
	CADLibrary::TUniqueTSObj<A3DSewOptionsData> SewData;
	SewData->m_bComputePreferredOpenShellOrientation = false;
	
	TechSoftInterface::SewModel(ModelFile.Get(), CADLibrary::FImportParameters::GStitchingTolerance, SewData.GetPtr());
}


void FTechSoftFileParser::GenerateBodyMeshes()
{
	uint64 StartGenerateBodyMeshesTime = FPlatformTime::Cycles64();
	for (TPair<A3DRiRepresentationItem*, FCadId>& Entry : RepresentationItemsCache)
	{
		A3DRiRepresentationItem* RepresentationItemPtr = Entry.Key;
		FArchiveBody& Body = SceneGraph.GetBody(Entry.Value);
		if (!Body.bIsFromCad)
		{
			FTechSoftFileParser::GenerateBodyMesh(RepresentationItemPtr, Body);
		}
		else
		{
			GenerateBodyMesh(RepresentationItemPtr, Body);
		}

		if (bConvertionFailed)
		{
			return;
		}
	}
	CADFileData.GetRecord().MeshTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartGenerateBodyMeshesTime);
}

void FTechSoftFileParser::GenerateBodyMesh(A3DRiRepresentationItem* Representation, FArchiveBody& Body)
{
	FBodyMesh& BodyMesh = CADFileData.AddBodyMesh(Body.Id, Body);

	uint32 NewBRepCount = 0;
	A3DRiBrepModel** NewBReps = nullptr;

	if (CADFileData.GetImportParameters().GetStitchingTechnique() == StitchingHeal)
	{
		TUniqueTSObj<A3DSewOptionsData> SewData;
		SewData->m_bComputePreferredOpenShellOrientation = false;
		const uint32 BRepCount = 1;
		A3DStatus Status = TechSoftInterface::SewBReps(&Representation, BRepCount, CADLibrary::FImportParameters::GStitchingTolerance, FileUnit, SewData.GetPtr(), &NewBReps, NewBRepCount);
		if (Status != A3DStatus::A3D_SUCCESS)
		{
			CADFileData.LogWarning(TEXT("A body healing failed. A body could be missing."));
		}
	}

	if (NewBRepCount > 0)
	{
		for (uint32 Index = 0; Index < NewBRepCount; ++Index)
		{
			TechSoftUtils::FillBodyMesh(NewBReps[Index], CADFileData.GetImportParameters(), Body.Unit, BodyMesh);
		}
	} 
	else
	{
		TechSoftUtils::FillBodyMesh(Representation, CADFileData.GetImportParameters(), Body.Unit, BodyMesh);
	}

	if (BodyMesh.TriangleCount == 0)
	{
		// the mesh of the body is empty, the body is deleted.
		Body.Delete();
	}

	// Convert material
	for (FTessellationData& Tessellation : BodyMesh.Faces)
	{
		// Don't add material of empty face (i.e. face with empty tessellation
		if (Tessellation.VertexIndices.Num() == 0)
		{
			continue;
		}

		// Extract proper color or material based on style index
		FMaterialUId CachedStyleIndex = Tessellation.MaterialUId;
		Tessellation.MaterialUId = 0;

		if (CachedStyleIndex != FTechSoftDefaultValue::Style)
		{
			ExtractGraphStyleProperties(CachedStyleIndex, Tessellation);
		}
		Tessellation.DefineGraphicsPropertiesFromNoOverwrite(Body);

		BodyMesh.AddGraphicPropertiesFrom(Tessellation);
	}

	Body.ColorFaceSet = BodyMesh.ColorSet;
	Body.MaterialFaceSet = BodyMesh.MaterialSet;

	if (Body.ColorUId == 0 && Body.ColorFaceSet.Num())
	{
		Body.ColorUId = *Body.ColorFaceSet.begin();
	}
	if (Body.MaterialUId == 0 && Body.MaterialFaceSet.Num())
	{
		Body.MaterialUId = *Body.MaterialFaceSet.begin();
	}

	// Write part's representation as Prc file if it is a BRep
	A3DEEntityType Type;
	A3DEntityGetType(Representation, &Type);

	if (Type == kA3DTypeRiBrepModel)
	{
		FString FilePath = CADFileData.GetBodyCachePath(Body.MeshActorUId);
		if (!FilePath.IsEmpty())
		{
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

			// Save body unit and default color and material attributes in a json string
			// This will be used when the file is reloaded
			JsonObject->SetNumberField(JSON_ENTRY_BODY_UNIT, Body.Unit);

			if(Body.ColorUId)
			{
				JsonObject->SetNumberField(JSON_ENTRY_COLOR_NAME, Body.ColorUId);
			}
			if(Body.MaterialUId)
			{
				JsonObject->SetNumberField(JSON_ENTRY_MATERIAL_NAME, Body.MaterialUId);
			}

			FString JsonString;
			TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonString);

			FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);
			TechSoftUtils::SaveBodiesToPrcFile(&Representation, 1, FilePath, JsonString);
		}
	}
}

void FTechSoftFileParser::ReserveCADFileData()
{
	// TODO: Could be more accurate
	CountUnderModel();

	CADFileData.ReserveBodyMeshes(ComponentCount[EComponentType::Body]);

	SceneGraph.Reserve(ComponentCount);
	uint32 MaterialNum = CountColorAndMaterial();
	SceneGraph.MaterialHIdToMaterial.Reserve(MaterialNum);
}

void FTechSoftFileParser::CountUnderModel()
{
	TUniqueTSObj<A3DAsmModelFileData> ModelFileData(ModelFile.Get());
	if (!ModelFileData.IsValid())
	{
		return;
	}

	ComponentCount[EComponentType::Reference] ++;

	for (uint32 Index = 0; Index < ModelFileData->m_uiPOccurrencesSize; ++Index)
	{
		if (IsConfigurationSet(ModelFileData->m_ppPOccurrences[Index]))
		{
			CountUnderConfigurationSet(ModelFileData->m_ppPOccurrences[Index]);
		}
		else
		{
			CountUnderOccurrence(ModelFileData->m_ppPOccurrences[Index]);
			CountUnderOverrideOccurrence(ModelFileData->m_ppPOccurrences[Index]);
		}
	}

	ReferenceCache.Empty();
}

ECADParsingResult FTechSoftFileParser::TraverseModel()
{
	TUniqueTSObj<A3DAsmModelFileData> ModelFileData(ModelFile.Get());
	if (!ModelFileData.IsValid())
	{
		return ECADParsingResult::ProcessFailed;
	}

	FArchiveInstance EmptyInstance;
	FArchiveReference& Reference = SceneGraph.AddReference(EmptyInstance);
	ExtractSpecificMetaData(ModelFile.Get(), Reference);
	Reference.Unit = FileUnit;

	if(ModelFileData->m_uiPOccurrencesSize == 0)
	{
		CADFileData.LogWarning(FString::Printf(TEXT("File %s is empty."), *CADFileData.GetCADFileDescription().GetFileName()));
		return ECADParsingResult::ProcessFailed;
	}

	if (ModelFileData->m_uiPOccurrencesSize > 1)
	{
		CADFileData.LogWarning(FString::Printf(TEXT("File %s has many root components, only the first is loaded."), *CADFileData.GetCADFileDescription().GetFileName()));
	}

	if (IsConfigurationSet(ModelFileData->m_ppPOccurrences[0]))
	{
		TraverseConfigurationSet(ModelFileData->m_ppPOccurrences[0], Reference);
	}
	else
	{
		TraverseReference(ModelFileData->m_ppPOccurrences[0], Reference);
	}

	return ECADParsingResult::ProcessOk;
}

void FTechSoftFileParser::TraverseConfigurationSet(const A3DAsmProductOccurrence* ConfigurationSetPtr, FArchiveReference& Reference)
{
	TUniqueTSObj<A3DAsmProductOccurrenceData> ConfigurationSetData(ConfigurationSetPtr);
	if (!ConfigurationSetData.IsValid())
	{
		return;
	}

	ExtractMetaData(ConfigurationSetPtr, Reference);
	ExtractSpecificMetaData(ConfigurationSetPtr, Reference);
	BuildReferenceName(Reference);

	const FString& ConfigurationToLoad = CADFileData.GetCADFileDescription().GetConfiguration();

	A3DMiscTransformation* Transform = ConfigurationSetData->m_pLocation;
	ExtractTransformation(Transform, Reference);

	TUniqueTSObj<A3DAsmProductOccurrenceData> ConfigurationData;
	for (unsigned int Index = 0; Index < ConfigurationSetData->m_uiPOccurrencesSize; ++Index)
	{
		ConfigurationData.FillFrom(ConfigurationSetData->m_ppPOccurrences[Index]);
		if (!ConfigurationData.IsValid())
		{
			continue;
		}

		if (ConfigurationData->m_uiProductFlags & A3D_PRODUCT_FLAG_CONFIG)
		{
			bool bIsConfigurationToLoad = false;
			if (!ConfigurationToLoad.IsEmpty())
			{
				FArchiveCADObject Configuration;
				ExtractMetaData(ConfigurationSetData->m_ppPOccurrences[Index], Configuration);
				if (!Configuration.Label.IsEmpty())
				{
					bIsConfigurationToLoad = (Configuration.Label.Equals(ConfigurationToLoad));
				}
			}
			else
			{
				bIsConfigurationToLoad = ConfigurationData->m_uiProductFlags & A3D_PRODUCT_FLAG_DEFAULT;
			}

			if (bIsConfigurationToLoad)
			{
				TraverseReference(ConfigurationSetData->m_ppPOccurrences[Index], Reference);
				return;
			}
		}
	}

	if (ConfigurationToLoad.IsEmpty())
	{
		// no default configuration, traverse the first occurrence
		for (unsigned int Index = 0; Index < ConfigurationSetData->m_uiPOccurrencesSize; ++Index)
		{
			ConfigurationData.FillFrom(ConfigurationSetData->m_ppPOccurrences[Index]);
			if (!ConfigurationData.IsValid())
			{
				return;
			}

			if (ConfigurationData->m_uiProductFlags & A3D_PRODUCT_FLAG_CONFIG)
			{
				TraverseReference(ConfigurationSetData->m_ppPOccurrences[Index], Reference);
			}
		}
	}
}

void FTechSoftFileParser::CountUnderConfigurationSet(const A3DAsmProductOccurrence* ConfigurationSetPtr)
{
	TUniqueTSObj<A3DAsmProductOccurrenceData> ConfigurationSetData(ConfigurationSetPtr);
	if (!ConfigurationSetData.IsValid())
	{
		return;
	}

	const FString& ConfigurationToLoad = CADFileData.GetCADFileDescription().GetConfiguration();

	TUniqueTSObj<A3DAsmProductOccurrenceData> ConfigurationData;
	for (unsigned int Index = 0; Index < ConfigurationSetData->m_uiPOccurrencesSize; ++Index)
	{
		ConfigurationData.FillFrom(ConfigurationSetData->m_ppPOccurrences[Index]);
		if (!ConfigurationData.IsValid())
		{
			continue;
		}

		if (ConfigurationData->m_uiProductFlags & A3D_PRODUCT_FLAG_CONFIG)
		{
			bool bIsConfigurationToLoad = false;
			if (!ConfigurationToLoad.IsEmpty())
			{
				FArchiveCADObject Configuration;
				ExtractMetaData(ConfigurationSetData->m_ppPOccurrences[Index], Configuration);
				if (!Configuration.Label.IsEmpty())
				{
					bIsConfigurationToLoad = (Configuration.Label.Equals(ConfigurationToLoad));
				}
			}
			else
			{
				bIsConfigurationToLoad = ConfigurationData->m_uiProductFlags & A3D_PRODUCT_FLAG_DEFAULT;
			}

			if (bIsConfigurationToLoad)
			{
				CountUnderOccurrence(ConfigurationSetData->m_ppPOccurrences[Index]);
				CountUnderOverrideOccurrence(ConfigurationSetData->m_ppPOccurrences[Index]);
				return;
			}
		}
	}

	if (ConfigurationToLoad.IsEmpty())
	{
		// no default configuration, traverse the first configuration
		for (unsigned int Index = 0; Index < ConfigurationSetData->m_uiPOccurrencesSize; ++Index)
		{
			ConfigurationData.FillFrom(ConfigurationSetData->m_ppPOccurrences[Index]);
			if (!ConfigurationData.IsValid())
			{
				return;
			}

			if (ConfigurationData->m_uiProductFlags & A3D_PRODUCT_FLAG_CONFIG)
			{
				CountUnderOccurrence(ConfigurationSetData->m_ppPOccurrences[Index]);
				CountUnderOverrideOccurrence(ConfigurationSetData->m_ppPOccurrences[Index]);
			}
		}
	}
}

void FTechSoftFileParser::TraverseReference(const A3DAsmProductOccurrence* A3DReferencePtr, FArchiveReference& Reference)
{
	TUniqueTSObj<A3DAsmProductOccurrenceData> ReferenceData(A3DReferencePtr);
	if (!ReferenceData.IsValid())
	{
		SceneGraph.RemoveLastReference();
		return;
	}

	ExtractMetaData(A3DReferencePtr, Reference);

	if (Reference.bIsRemoved || !Reference.bShow)
	{
		if (Reference.bIsRemoved)
		{
			// Keep node in the SceneGraph if occurrence is not removed but just not visible
			SceneGraph.RemoveLastReference();
		}
		return;
	}

	ExtractSpecificMetaData(A3DReferencePtr, Reference);

	BuildReferenceName(Reference);

	FMatrix ReferenceMatrix = FMatrix::Identity;
	A3DMiscTransformation* Transform = ReferenceData->m_pLocation;
	ExtractTransformation(Transform, Reference);
	
	Reference.TransformMatrix = Reference.TransformMatrix * ReferenceMatrix;

	FArchiveInstance EmptyInstance;
	ProcessReference(A3DReferencePtr, EmptyInstance, Reference);
}

void FTechSoftFileParser::ProcessUnloadedReference(const FArchiveInstance& Instance, FArchiveUnloadedReference& Reference)
{
	// Make sure that the ExternalFile path is right otherwise try to find the file and update.
	TechSoftFileParserImpl::UpdateFileDescriptor(Reference.ExternalFile);

	switch (Format)
	{
	case ECADFormat::SOLIDWORKS:
		if (const FString* ConfigurationName = Instance.MetaData.Find(TEXT("ConfigurationName")))
		{
			Reference.ExternalFile.SetConfiguration(*ConfigurationName);
		}
		break;

	default:
		break;
	}

	SceneGraph.AddExternalReferenceFile(Reference);
}

void FTechSoftFileParser::TraverseOccurrence(const A3DAsmProductOccurrence* OccurrencePtr, FArchiveReference& ParentReference)
{
	// first product occurrence with m_pPart != nullptr || m_uiPOccurrencesSize > 0
	const A3DAsmProductOccurrence* CachedOccurrencePtr = OccurrencePtr;
	TUniqueTSObj<A3DAsmProductOccurrenceData> OccurrenceData(OccurrencePtr);
	if (!OccurrenceData.IsValid())
	{
		return;
	}

	bool bContinueTraverse = OccurrenceData->m_pPrototype
		|| OccurrenceData->m_pExternalData
		|| OccurrenceData->m_pPart
		|| OccurrenceData->m_uiPOccurrencesSize > 0;
	if (!bContinueTraverse)
	{
		return;
	}

	FArchiveInstance& Instance = SceneGraph.AddInstance(ParentReference);
	ExtractMetaData(OccurrencePtr, Instance);
	Instance.DefineGraphicsPropertiesFromNoOverwrite(ParentReference);

	if (Instance.bIsRemoved)
	{
		SceneGraph.RemoveLastInstance();
		return;
	}

	ParentReference.AddChild(Instance.Id);

	ExtractSpecificMetaData(OccurrencePtr, Instance);
	BuildInstanceName(Instance, ParentReference);

	A3DMiscTransformation* Transform = OccurrenceData->m_pLocation;
	
	A3DAsmProductOccurrence* ReferencePtr = ProductOccurrence::GetReference(*OccurrenceData);

	// Is the Reference already processed ?
	if(ReferencePtr)
	{
		FCadId* ReferenceId = ReferenceCache.Find(ReferencePtr);
		if (ReferenceId != nullptr)
		{
			Instance.ReferenceNodeId = *ReferenceId;

			if (SceneGraph.IsAUnloadedReference(Instance.ReferenceNodeId))
			{
				Instance.bIsExternalReference = true;
			}

			ExtractTransformation(Transform, Instance);

			double ReferenceUnit = 1;
			if (SceneGraph.IsAReference(Instance.ReferenceNodeId))
			{
				FArchiveReference& Reference = SceneGraph.GetReference(Instance.ReferenceNodeId);
				ReferenceUnit = Reference.Unit;
			}
			else if (SceneGraph.IsAUnloadedReference(Instance.ReferenceNodeId))
			{
				FArchiveUnloadedReference& Reference = SceneGraph.GetUnloadedReference(Instance.ReferenceNodeId);
				ReferenceUnit = Reference.Unit;
			}

			// check if the instance unit is nearly equal to the existing reference unit, otherwise add a scale component to the instance transform
			double Scale = 1;
			if (!FMath::IsNearlyZero(ReferenceUnit))
			{
				Scale = Instance.Unit / ReferenceUnit;
				if (!FMath::IsNearlyEqual(Scale, 1.))
				{
					// Commented as the result was discarded
					// Instance.TransformMatrix = Instance.TransformMatrix.ApplyScale(Scale);
				}
			}

			return;
		}
	}

	FArchiveUnloadedReference& UnloadedReference = SceneGraph.AddUnloadedReference(Instance);

	bool bIsUnloaded = OccurrenceData->m_uiProductFlags & A3D_PRODUCT_FLAG_EXTERNAL_REFERENCE;

	// Extract metadata and define if it's an unloaded reference or not
	if (ReferencePtr)
	{
		ProcessPrototype(ReferencePtr, UnloadedReference, &Transform);
	}
	else
	{
		UnloadedReference.bIsUnloaded = false;
	}

	ExtractTransformation(Transform, Instance);
	UnloadedReference.Unit = Instance.Unit;

	if (UnloadedReference.bIsUnloaded)
	{
		ProcessUnloadedReference(Instance, UnloadedReference);
	}
	else
	{
		FArchiveReference& NewReference = SceneGraph.AddReference(UnloadedReference);
		Instance.bIsExternalReference = false;
		ProcessReference(OccurrencePtr, Instance, NewReference);
	}

	if(ReferencePtr)
	{
		ReferenceCache.Add(ReferencePtr, Instance.ReferenceNodeId);
	}

	for (uint32 Index = 0; Index < OccurrenceData->m_uiPOccurrencesSize; ++Index)
	{
		ExtractOverrideOccurrenceSubtree(OccurrenceData->m_ppPOccurrences[Index], Instance);
	}
}

void FTechSoftFileParser::ExtractOverrideOccurrenceSubtree(const A3DAsmProductOccurrence* OccurrencePtr, FArchiveWithOverridenChildren& Parent)
{
	TUniqueTSObj<A3DAsmProductOccurrenceData> OccurrenceData(OccurrencePtr);
	if (OccurrenceData->m_uiProductFlags & A3D_PRODUCT_FLAG_INTERNAL)
	{
		return;
	}

	FArchiveOverrideOccurrence& OverrideOccurrence = SceneGraph.AddOverrideOccurrence(Parent);
	ExtractMetaData(OccurrencePtr, OverrideOccurrence);

	Parent.AddOverridenChild(OverrideOccurrence.Id);

	FArchiveReference Reference;
	BuildInstanceName(OverrideOccurrence, Reference);

	A3DMiscTransformation* Transform = OccurrenceData->m_pLocation;
	ExtractTransformation(Transform, OverrideOccurrence);

	for (uint32 Index = 0; Index < OccurrenceData->m_uiPOccurrencesSize; ++Index)
	{
		ExtractOverrideOccurrenceSubtree(OccurrenceData->m_ppPOccurrences[Index], OverrideOccurrence);
	}
}

void FTechSoftFileParser::CountUnderOverrideOccurrence(const A3DAsmProductOccurrence* Occurrence)
{
	TUniqueTSObj<A3DAsmProductOccurrenceData> OccurrenceData(Occurrence);
	if (Occurrence && OccurrenceData.IsValid())
	{
		ComponentCount[EComponentType::OverriddeOccurence]++;

		uint32 ChildrenCount = OccurrenceData->m_uiPOccurrencesSize;
		A3DAsmProductOccurrence** Children = OccurrenceData->m_ppPOccurrences;
		for (uint32 Index = 0; Index < ChildrenCount; ++Index)
		{
			CountUnderOverrideOccurrence(Children[Index]);
		}
	}
}

void FTechSoftFileParser::ProcessReference(const A3DAsmProductOccurrence* OccurrencePtr, FArchiveInstance& Instance, FArchiveReference& Reference)
{
	const A3DAsmProductOccurrence* CachedOccurrencePtr = OccurrencePtr;
	TUniqueTSObj<A3DAsmProductOccurrenceData> OccurrenceData(OccurrencePtr);

	// If the prototype hasn't name, set its name with the name of the instance 
	if (!Reference.IsNameDefined())
	{
		Reference.Label = Instance.Label;
	}

	while (ProductOccurrence::HasNoPartNoChildButHasReference(*OccurrenceData))
	{
		CachedOccurrencePtr = ProductOccurrence::GetReference(*OccurrenceData);
		OccurrenceData.FillFrom(CachedOccurrencePtr);
	}

	if(ProductOccurrence::HasNoPartNoChild(*OccurrenceData))
	{
		return;
	}

	// Add part
	while (ProductOccurrence::HasNoPartButHasReference(*OccurrenceData))
	{
		OccurrenceData.FillFrom(ProductOccurrence::GetReference(*OccurrenceData));
	}
	if (OccurrenceData->m_pPart != nullptr)
	{
		A3DAsmPartDefinition* PartDefinition = OccurrenceData->m_pPart;
		TraversePartDefinition(PartDefinition, Reference);
	}

	// Add Occurrence's Children
	OccurrenceData.FillFrom(CachedOccurrencePtr);
	while (ProductOccurrence::HasNoChildButHasReference(*OccurrenceData))
	{
		OccurrenceData.FillFrom(ProductOccurrence::GetReference(*OccurrenceData));
	}

	uint32 ChildrenCount = OccurrenceData->m_uiPOccurrencesSize;
	A3DAsmProductOccurrence** Children = OccurrenceData->m_ppPOccurrences;
	for (uint32 Index = 0; Index < ChildrenCount; ++Index)
	{
		TraverseOccurrence(Children[Index], Reference);
	}
}

void FTechSoftFileParser::CountUnderOccurrence(const A3DAsmProductOccurrence* Occurrence)
{
	TUniqueTSObj<A3DAsmProductOccurrenceData> OccurrenceData(Occurrence);
	if (Occurrence && OccurrenceData.IsValid())
	{
		ComponentCount[EComponentType::Instance]++;

		A3DAsmProductOccurrence* ReferencePtr = ProductOccurrence::GetReference(*OccurrenceData);

		// Is the Reference already processed ?
		if(ReferencePtr)
		{
			FCadId* ReferenceId = ReferenceCache.Find(ReferencePtr);
			if (ReferenceId != nullptr)
			{
				return;
			}
			ReferenceCache.Add(ReferencePtr, 1);
		}
		ComponentCount[EComponentType::Reference]++;

		const A3DAsmProductOccurrence* CachedOccurrencePtr = Occurrence;
		while (ProductOccurrence::HasNoPartNoChildButHasReference(*OccurrenceData))
		{
			CachedOccurrencePtr = ProductOccurrence::GetReference(*OccurrenceData);
			OccurrenceData.FillFrom(CachedOccurrencePtr);
		}

		if (ProductOccurrence::HasNoPartNoChild(*OccurrenceData))
		{
			return;
		}

		// count under part
		while (ProductOccurrence::HasNoPartButHasReference(*OccurrenceData))
		{
			OccurrenceData.FillFrom(ProductOccurrence::GetReference(*OccurrenceData));
		}
		if (OccurrenceData->m_pPart != nullptr)
		{
			CountUnderPartDefinition(OccurrenceData->m_pPart);
		}

		// count under Occurrence
		OccurrenceData.FillFrom(CachedOccurrencePtr);
		while (ProductOccurrence::HasNoChildButHasReference(*OccurrenceData))
		{
			OccurrenceData.FillFrom(ProductOccurrence::GetReference(*OccurrenceData));
		}

		uint32 ChildrenCount = OccurrenceData->m_uiPOccurrencesSize;
		A3DAsmProductOccurrence** Children = OccurrenceData->m_ppPOccurrences;
		for (uint32 Index = 0; Index < ChildrenCount; ++Index)
		{
			CountUnderOccurrence(Children[Index]);
			CountUnderOverrideOccurrence(Children[Index]);
		}
	}
}

void FTechSoftFileParser::ProcessPrototype(const A3DAsmProductOccurrence* InPrototypePtr, FArchiveUnloadedReference& OutReference, A3DMiscTransformation** OutTransform)
{
	const A3DAsmProductOccurrence* PrototypePtr = InPrototypePtr;
	TUniqueTSObj<A3DAsmProductOccurrenceData> PrototypeData;

	while (PrototypePtr)
	{
		PrototypeData.FillFrom(PrototypePtr);
		if (!PrototypeData.IsValid())
		{
			return;
		}

		ExtractMetaData(PrototypePtr, OutReference);
		ExtractSpecificMetaData(PrototypePtr, OutReference);

		if (OutReference.ExternalFile.IsEmpty())
		{
			TUniqueTSObj<A3DUTF8Char*> FilePathUTF8Ptr;
			FilePathUTF8Ptr.FillWith(&TechSoftInterface::GetFilePathName, PrototypePtr);
			if (!FilePathUTF8Ptr.IsValid() || *FilePathUTF8Ptr == nullptr)
			{
				FilePathUTF8Ptr.FillWith(&TechSoftInterface::GetOriginalFilePathName, PrototypePtr);
			}
			if (FilePathUTF8Ptr.IsValid() && *FilePathUTF8Ptr != nullptr)
			{
				FString FilePath = UTF8_TO_TCHAR(*FilePathUTF8Ptr);
				FPaths::NormalizeFilename(FilePath);
				FString FileName = FPaths::GetCleanFilename(FilePath);
				if (!FileName.IsEmpty() && FileName != CADFileData.GetCADFileDescription().GetFileName())
				{
					OutReference.ExternalFile = FFileDescriptor(*FilePath, nullptr, *CADFileData.GetCADFileDescription().GetRootFolder());
				}
			}
		}
		
		if (ProductOccurrence::HasPartOrChild(*PrototypeData) )
		{
			OutReference.bIsUnloaded = false;
			PrototypePtr = nullptr;
		}
		else
		{
			PrototypePtr = ProductOccurrence::GetReference(*PrototypeData);
		}

		if (!*OutTransform)
		{
			*OutTransform = PrototypeData->m_pLocation;
		}
	}

	if (!*OutTransform)
	{
		while (PrototypeData->m_pLocation == nullptr && ProductOccurrence::PrototypeIsValid(*PrototypeData))
		{
			PrototypeData.FillFrom(ProductOccurrence::GetReference(*PrototypeData));
		}
		if (PrototypeData.IsValid())
		{
			*OutTransform = PrototypeData->m_pLocation;
		}
	}

	if (OutReference.bIsUnloaded)
	{
		if(OutReference.Label.IsEmpty())
		{
			OutReference.Label = OutReference.ExternalFile.GetFileName();
		}
	}
	else
	{
		OutReference.ExternalFile.Empty();
	}

	BuildReferenceName(OutReference);
}

void FTechSoftFileParser::TraversePartDefinition(const A3DAsmPartDefinition* PartDefinitionPtr, FArchiveReference& Part)
{
	ExtractMetaData(PartDefinitionPtr, Part);

	if (Part.bIsRemoved || !Part.bShow)
	{
		return;
	}

	ExtractSpecificMetaData(PartDefinitionPtr, Part);

	BuildPartName(Part);

	TUniqueTSObj<A3DAsmPartDefinitionData> PartData(PartDefinitionPtr);
	if (PartData.IsValid())
	{
		for (A3DUns32 Index = 0; Index < PartData->m_uiRepItemsSize; ++Index)
		{
			TraverseRepresentationItem(PartData->m_ppRepItems[Index], Part);
		}
	}
}

void FTechSoftFileParser::CountUnderPartDefinition(const A3DAsmPartDefinition* PartDefinition)
{
	TUniqueTSObj<A3DAsmPartDefinitionData> PartData(PartDefinition);
	if (PartDefinition && PartData.IsValid())
	{
		ComponentCount[EComponentType::Reference] ++;
		ComponentCount[EComponentType::Instance] ++;

		for (unsigned int Index = 0; Index < PartData->m_uiRepItemsSize; ++Index)
		{
			CountUnderRepresentationItem(PartData->m_ppRepItems[Index]);
		}
	}
}

void FTechSoftFileParser::TraverseRepresentationItem(A3DRiRepresentationItem* RepresentationItem, FArchiveReference& Part)
{
	if (!RepresentationItem)
	{
		return;
	}

	if (FCadId* BodyIndexPtr = RepresentationItemsCache.Find(RepresentationItem))
	{
		Part.AddChild(*BodyIndexPtr);
		return;
	}

	A3DEEntityType Type;
	A3DEntityGetType(RepresentationItem, &Type);

	switch (Type)
	{
	case kA3DTypeRiSet:
		return TraverseRepresentationSet(RepresentationItem, Part);
	case kA3DTypeRiBrepModel:
		return TraverseBRepModel(RepresentationItem, Part);
	case kA3DTypeRiPolyBrepModel:
		return TraversePolyBRepModel(RepresentationItem, Part);
	default:
		break;
	}
}

void FTechSoftFileParser::CountUnderRepresentationItem(const A3DRiRepresentationItem* RepresentationItem)
{
	A3DEEntityType Type;
	A3DEntityGetType(RepresentationItem, &Type);

	switch (Type)
	{
	case kA3DTypeRiSet:
		CountUnderRepresentationSet(RepresentationItem);
		break;
	case kA3DTypeRiBrepModel:
	case kA3DTypeRiPolyBrepModel:
		ComponentCount[EComponentType::Body] ++;
		break;

	default:
		break;
	}
}

void FTechSoftFileParser::TraverseRepresentationSet(const A3DRiSet* RepresentationSetPtr, FArchiveReference& Parent)
{
	TUniqueTSObj<A3DRiSetData> RepresentationSetData(RepresentationSetPtr);
	if (!RepresentationSetData.IsValid())
	{
		return;
	}

	FCadId RepresentationSetId = 0;
	FArchiveReference& RepresentationSet = SceneGraph.AddOccurrence(Parent);
	ExtractMetaData(RepresentationSetPtr, RepresentationSet);
	RepresentationSet.DefineGraphicsPropertiesFromNoOverwrite(Parent);
	BuildRepresentationSetName(RepresentationSet, Parent);

	if (RepresentationSet.bIsRemoved || !RepresentationSet.bShow)
	{
		Parent.RemoveLastChild();
		SceneGraph.RemoveLastOccurrence();
		return;
	}

	for (A3DUns32 Index = 0; Index < RepresentationSetData->m_uiRepItemsSize; ++Index)
	{
		TraverseRepresentationItem(RepresentationSetData->m_ppRepItems[Index], RepresentationSet);
	}
}

void FTechSoftFileParser::CountUnderRepresentationSet(const A3DRiSet* RepresentationSet)
{
	TUniqueTSObj<A3DRiSetData> RepresentationSetData(RepresentationSet);
	if (RepresentationSet && RepresentationSetData.IsValid())
	{
		ComponentCount[EComponentType::Instance] ++;
		ComponentCount[EComponentType::Reference] ++;

		for (A3DUns32 Index = 0; Index < RepresentationSetData->m_uiRepItemsSize; ++Index)
		{
			CountUnderRepresentationItem(RepresentationSetData->m_ppRepItems[Index]);
		}
	}
}

void FTechSoftFileParser::TraverseBRepModel(A3DRiBrepModel* BRepModelPtr, FArchiveReference& Parent)
{
	FArchiveBody& BRep = SceneGraph.AddBody(Parent, CADFileData.GetImportParameters().GetMesher());
	ExtractMetaData(BRepModelPtr, BRep);
	BRep.DefineGraphicsPropertiesFromNoOverwrite(Parent);

	ExtractSpecificMetaData(BRepModelPtr, BRep);

	if (!BRep.bShow || BRep.bIsRemoved)
	{
		SceneGraph.RemoveLastBody();
		return;
	}

	Parent.AddChild(BRep.Id);
	RepresentationItemsCache.Add(BRepModelPtr, BRep.Id);

	TUniqueTSObj<A3DRiBrepModelData> BRepModelData(BRepModelPtr);
	BRep.bIsASolid = (bool)BRepModelData->m_bSolid;
	BuildBodyName(BRep, Parent);

	TUniqueTSObj<A3DRiRepresentationItemData> RepresentationData(BRepModelPtr);
	ExtractCoordinateSystem(RepresentationData->m_pCoordinateSystem, BRep);
}

void FTechSoftFileParser::TraversePolyBRepModel(A3DRiPolyBrepModel* PolygonalPtr, FArchiveReference& Parent)
{
	FArchiveBody& BRep = SceneGraph.AddBody(Parent, EMesher::TechSoft);
	BRep.bIsFromCad = false;

	ExtractMetaData(PolygonalPtr, BRep);
	BRep.DefineGraphicsPropertiesFromNoOverwrite(Parent);

	ExtractSpecificMetaData(PolygonalPtr, BRep);

	if (!BRep.bShow || BRep.bIsRemoved)
	{
		SceneGraph.RemoveLastBody();
		return;
	}

	Parent.AddChild(BRep.Id);
	RepresentationItemsCache.Add(PolygonalPtr, BRep.Id);

	TUniqueTSObj<A3DRiPolyBrepModelData> BRepModelData(PolygonalPtr);
	BRep.bIsASolid = (bool)BRepModelData->m_bIsClosed;
	BuildBodyName(BRep, Parent);

	TUniqueTSObj<A3DRiRepresentationItemData> RepresentationData(PolygonalPtr);
	ExtractCoordinateSystem(RepresentationData->m_pCoordinateSystem, BRep);
}


void FTechSoftFileParser::ExtractMetaData(const A3DEntity* Entity, FArchiveCADObject& OutObject)
{
	TUniqueTSObj<A3DRootBaseData> MetaData(Entity);
	if (MetaData.IsValid())
	{
		if (OutObject.Label.IsEmpty() && MetaData->m_pcName && MetaData->m_pcName[0] != '\0')
		{
			FString Name = UTF8_TO_TCHAR(MetaData->m_pcName);
			if(Name != TEXT("unnamed"))  // "unnamed" is create by Techsoft. This name is ignored 
			{
				Name = TechSoftUtils::CleanLabel(Name);
				OutObject.Label = Name;
			}
		}

		TUniqueTSObj<A3DMiscAttributeData> AttributeData;
		for (A3DUns32 Index = 0; Index < MetaData->m_uiSize; ++Index)
		{
			AttributeData.FillFrom(MetaData->m_ppAttributes[Index]);
			if (AttributeData.IsValid())
			{
				TechSoftFileParserImpl::TraverseAttribute(*AttributeData, OutObject.MetaData);
			}
		}
	}

	if (A3DEntityIsBaseWithGraphicsType(Entity))
	{
		TUniqueTSObj<A3DRootBaseWithGraphicsData> MetaDataWithGraphics(Entity);
		if (MetaDataWithGraphics.IsValid())
		{
			if (MetaDataWithGraphics->m_pGraphics != NULL)
			{
				ExtractGraphicProperties(MetaDataWithGraphics->m_pGraphics, OutObject);
			}
		}
	}
}

void FTechSoftFileParser::BuildReferenceName(FArchiveCADObject& ReferenceData)
{
	TMap<FString, FString>& MetaData = ReferenceData.MetaData;

	FString* NamePtr = MetaData.Find(TEXT("InstanceName"));
	if (NamePtr != nullptr && !NamePtr->IsEmpty())
	{
		if (Format == ECADFormat::CATIA)
		{
			ReferenceData.Label = TechSoftUtils::CleanCatiaReferenceLabel(*NamePtr);
		}
		else
		{
			ReferenceData.Label = *NamePtr;
		}
		return;
	}

	if (ReferenceData.SetNameWithAttributeValue(TEXT("PartNumber")))
	{
		return;
	}

	switch (Format)
	{
	case ECADFormat::CATIA_3DXML:
		ReferenceData.Label = TechSoftUtils::Clean3dxmlReferenceLabel(ReferenceData.Label);
		break;

	case ECADFormat::SOLIDWORKS:
		ReferenceData.Label = TechSoftUtils::CleanSwReferenceLabel(ReferenceData.Label);
		break;

	default:
		break;
	}
}

void FTechSoftFileParser::BuildInstanceName(FArchiveCADObject& InstanceData, const FArchiveReference& Parent)
{
	TMap<FString, FString>& MetaData = InstanceData.MetaData;

	if (InstanceData.SetNameWithAttributeValue(TEXT("InstanceName")))
	{
		return;
	}

	if (InstanceData.IsNameDefined())
	{
		switch (Format)
		{
		case ECADFormat::CATIA:
			InstanceData.Label = TechSoftUtils::CleanCatiaInstanceLabel(InstanceData.Label);
			break;
		
		case ECADFormat::CATIA_3DXML:
			InstanceData.Label = TechSoftUtils::Clean3dxmlInstanceLabel(InstanceData.Label);
			break;

		case ECADFormat::SOLIDWORKS:
			InstanceData.Label = TechSoftUtils::CleanSwInstanceLabel(InstanceData.Label);
			break;

		default:
			break;
		}
		return;
	}

	if (InstanceData.Label.IsEmpty())
	{
		InstanceData.Label = Parent.Label + "_" + FString::FromInt(Parent.Children.Num());
	}
}

void FTechSoftFileParser::BuildPartName(FArchiveCADObject& PartData)
{
	if (PartData.SetNameWithAttributeValue(TEXT("PartNumber")))
	{
		return;
	}
}

void FTechSoftFileParser::BuildBodyName(FArchiveBody& Body, const FArchiveReference& Parent)
{
	if (Format == ECADFormat::CREO)
	{
		Body.Label = TechSoftUtils::CleanCreoLabel(Body.Label);
	}

	if(Body.IsNameDefined())
	{
		return;
	}

	if (Format == ECADFormat::CATIA)
	{
		if(Body.SetNameWithAttributeValue(TEXT("BodyID")))
		{
			return;
		}
	}

	FString& Label = Body.Label;
	if (Parent.IsNameDefined())
	{
		Label = Parent.Label + TEXT("_body");
	}
	else
	{
		if (Body.bIsASolid)
		{
			Label = TEXT("Solid");
		}
		else
		{
			Label = TEXT("Shell");
		}
	}
	Label += FString::FromInt(Parent.Children.Num());
}

void FTechSoftFileParser::BuildRepresentationSetName(FArchiveCADObject& Occurrence, const FArchiveReference& Parent)
{
	if (Occurrence.IsNameDefined())
	{
		return;
	}

	FString& Label = Occurrence.Label;
	if (Parent.IsNameDefined())
	{
		Label = Parent.Label;
	}
	else
	{
		Label = TEXT("Product");
	}
	Label += FString::FromInt(Parent.Children.Num());
}

void FTechSoftFileParser::ExtractSpecificMetaData(const A3DAsmProductOccurrence* Occurrence, FArchiveCADObject& OutMetaData)
{
	//----------- Export Specific information per CAD format -----------
	switch (ModelerType)
	{
	case ModelerSlw:
	{
		TUniqueTSObj<A3DAsmProductOccurrenceDataSLW> SolidWorksSpecificData(Occurrence);
		if (SolidWorksSpecificData.IsValid())
		{
			if (SolidWorksSpecificData->m_psCfgName)
			{
				FString ConfigurationName = UTF8_TO_TCHAR(SolidWorksSpecificData->m_psCfgName);
				OutMetaData.MetaData.Emplace(TEXT("ConfigurationName"), ConfigurationName);
				FString ConfigurationIndex = FString::FromInt(SolidWorksSpecificData->m_iIndexCfg);
				OutMetaData.MetaData.Emplace(TEXT("ConfigurationIndex"), ConfigurationIndex);
			}
		}
		break;
	}
	case ModelerUnigraphics:
	{
#ifdef WIP
		TUniqueTSObj<A3DAsmProductOccurrenceDataUg> UnigraphicsSpecificData(Occurrence);
		if (UnigraphicsSpecificData.IsValid())
		{
			if (UnigraphicsSpecificData->m_psPartUID)
			{
				FString PartUID = UTF8_TO_TCHAR(UnigraphicsSpecificData->m_psPartUID);
				OutMetaData.MetaData.Emplace(TEXT("UnigraphicsPartUID"), PartUID);
			}
			if (UnigraphicsSpecificData->m_psFileName)
			{
				FString FileName = UTF8_TO_TCHAR(UnigraphicsSpecificData->m_psFileName);
				OutMetaData.MetaData.Emplace(TEXT("UnigraphicsFileName"), FileName);
			}
			if (UnigraphicsSpecificData->m_psInstanceFileName)
			{
				FString InstanceFileName = UTF8_TO_TCHAR(UnigraphicsSpecificData->m_psInstanceFileName);
				OutMetaData.MetaData.Emplace(TEXT("UnigraphicsInstanceFileName"), InstanceFileName);
			}
			if (UnigraphicsSpecificData->m_psRefSet)
			{
				FString RefSet = UTF8_TO_TCHAR(UnigraphicsSpecificData->m_psRefSet);
				OutMetaData.MetaData.Emplace(TEXT("UnigraphicsInstanceRefSet"), RefSet);
			}
			if (UnigraphicsSpecificData->m_psPartUID)
			{
				FString PartUID = UTF8_TO_TCHAR(UnigraphicsSpecificData->m_psPartUID);
				OutMetaData.MetaData.Emplace(TEXT("UnigraphicsInstancePartUID"), PartUID);
			}

			if (UnigraphicsSpecificData->m_uiInstanceTag)
			{
				FString InstanceTag = FString::FromInt(UnigraphicsSpecificData->m_uiInstanceTag);
				OutMetaData.MetaData.Emplace(TEXT("UnigraphicsInstanceTag"), InstanceTag);
			}

			for (uint32 Index = 0; Index < UnigraphicsSpecificData->m_uiPromotedBodiesSize; ++Index)
			{
				A3DPromotedBodyUg PromotedBody = UnigraphicsSpecificData->m_asPromotedBodies[Index];
			}

			for (uint32 Index = 0; Index < UnigraphicsSpecificData->m_uiChildrenByRefsetsSize; ++Index)
			{
				A3DElementsByRefsetUg Refset = UnigraphicsSpecificData->m_asChildrenByRefsets[Index];
			}

			if(UnigraphicsSpecificData->m_uiSolidsByRefsetsSize)
			{
				for (uint32 Index = 0; Index < UnigraphicsSpecificData->m_uiSolidsByRefsetsSize; ++Index)
				{
					A3DElementsByRefsetUg Refset = UnigraphicsSpecificData->m_asSolidsByRefsets[Index];

					FString ReferenceSetName = UTF8_TO_TCHAR(Refset.m_psRefset);
					if (ReferenceSetName == CADFileData.GetCADFileDescription().GetConfiguration())
					{
						UnigraphicsReferenceSet.Reserve(Refset.m_uiElementsSize);
						for (uint32 Andex = 0; Andex < Refset.m_uiElementsSize; ++Andex)
						{
							UnigraphicsReferenceSet.Add(Refset.m_auiElements[Andex]);
						}
					}
				}
			}
		}
#endif
		break;
	}

	case ModelerCatiaV5:
	{
		TUniqueTSObj<A3DAsmProductOccurrenceDataCV5> CatiaV5SpecificData(Occurrence);
		if (CatiaV5SpecificData.IsValid())
		{
			if (CatiaV5SpecificData->m_psVersion)
			{
				FString Version = UTF8_TO_TCHAR(CatiaV5SpecificData->m_psVersion);
				OutMetaData.MetaData.Emplace(TEXT("CatiaVersion"), Version);
			}

			if (CatiaV5SpecificData->m_psPartNumber)
			{
				FString PartNumber = UTF8_TO_TCHAR(CatiaV5SpecificData->m_psPartNumber);
				OutMetaData.MetaData.Emplace(TEXT("CatiaPartNumber"), PartNumber);
			}
		}
		break;
	}

	default:
		break;
	}
}

FArchiveColor& FTechSoftFileParser::FindOrAddColor(uint32 ColorIndex, uint8 Alpha)
{
	FMaterialUId ColorHId = BuildColorFastUId(ColorIndex, Alpha);
	if (FArchiveColor* Color = CADFileData.FindColor(ColorHId))
	{
		return *Color;
	}

	FArchiveColor& NewColor = CADFileData.AddColor(ColorHId);
	NewColor.Color = TechSoftUtils::GetColorAt(ColorIndex);
	NewColor.Color.A = Alpha;
	
	NewColor.UEMaterialUId = BuildColorUId(NewColor.Color);
	return NewColor;
}

FArchiveMaterial& FTechSoftFileParser::AddMaterialAt(uint32 MaterialIndexToSave, uint32 GraphMaterialIndex, const A3DGraphStyleData& GraphStyleData)
{
	FArchiveMaterial& NewMaterial = CADFileData.AddMaterial(GraphMaterialIndex);
	FCADMaterial& Material = NewMaterial.Material;

	TUniqueTSObjFromIndex<A3DGraphMaterialData> MaterialData(MaterialIndexToSave);
	if(MaterialData.IsValid())
	{
		Material.Diffuse = TechSoftUtils::GetColorAt(MaterialData->m_uiDiffuse);
		Material.Ambient = TechSoftUtils::GetColorAt(MaterialData->m_uiAmbient);
		Material.Specular = TechSoftUtils::GetColorAt(MaterialData->m_uiSpecular);
		Material.Shininess = MaterialData->m_dShininess;
		if(GraphStyleData.m_bIsTransparencyDefined)
		{
			Material.Transparency = 1. - GraphStyleData.m_ucTransparency/255.;
		}
		// todo: find how to convert Emissive color into ? reflexion coef...
		// Material.Emissive = GetColor(MaterialData->m_uiEmissive);
		// Material.Reflexion;
	}
	NewMaterial.UEMaterialUId = BuildMaterialUId(Material);
	return NewMaterial;
}


// Look at TechSoftUtils::BuildCADMaterial if any loigc changes in this method
// or any of the methods it calls
FArchiveMaterial& FTechSoftFileParser::FindOrAddMaterial(FMaterialUId MaterialIndex, const A3DGraphStyleData& GraphStyleData)
{
	if (FArchiveMaterial* MaterialArchive = CADFileData.FindMaterial(MaterialIndex))
	{
		return *MaterialArchive;
	}

	bool bIsTexture = TechSoftInterface::IsMaterialTexture(MaterialIndex);
	if (bIsTexture)
	{
		TUniqueTSObjFromIndex<A3DGraphTextureApplicationData> TextureData(MaterialIndex);
		if (TextureData.IsValid())
		{
			return AddMaterialAt(TextureData->m_uiMaterialIndex, MaterialIndex, GraphStyleData);
			
#ifdef NOTYETDEFINE
			TUniqueTSObj<A3DGraphTextureDefinitionData> TextureDefinitionData(TextureData->m_uiTextureDefinitionIndex);
			if (TextureDefinitionData.IsValid())
			{
				TUniqueTSObj<A3DGraphPictureData> PictureData(TextureDefinitionData->m_uiPictureIndex);
			}
#endif
		}
		return AddMaterialAt(MaterialIndex, 0, GraphStyleData);
	}
	else
	{
		return AddMaterial(MaterialIndex, GraphStyleData);
	}
}

void FTechSoftFileParser::ExtractGraphicProperties(const A3DGraphics* Graphics, FArchiveCADObject& OutMetaData)
{
	TUniqueTSObj<A3DGraphicsData> GraphicsData(Graphics);
	if (!GraphicsData.IsValid())
	{
		return;
	}

	OutMetaData.bIsRemoved = GraphicsData->m_usBehaviour & kA3DGraphicsRemoved;
	OutMetaData.bShow = GraphicsData->m_usBehaviour & kA3DGraphicsShow;

	if (GraphicsData->m_usBehaviour & kA3DGraphicsFatherHeritColor)
	{
		OutMetaData.Inheritance = ECADGraphicPropertyInheritance::FatherHerit;
	}
	else if (GraphicsData->m_usBehaviour & kA3DGraphicsSonHeritColor)
	{
		OutMetaData.Inheritance = ECADGraphicPropertyInheritance::ChildHerit;
	}

	if (GraphicsData->m_uiStyleIndex == A3D_DEFAULT_STYLE_INDEX)
	{
		return;
	}

	ExtractGraphStyleProperties(GraphicsData->m_uiStyleIndex, OutMetaData);
}

// Please review TechSoftUtils::GetMaterialValues if anything changes
// in this method or the methods it calls
void FTechSoftFileParser::ExtractGraphStyleProperties(uint32 StyleIndex, FArchiveGraphicProperties& OutGraphicProperties)
{
	TUniqueTSObjFromIndex<A3DGraphStyleData> GraphStyleData(StyleIndex);

	if (GraphStyleData.IsValid())
	{
		if (GraphStyleData->m_bMaterial)
		{
			FArchiveMaterial& MaterialArchive = FindOrAddMaterial(GraphStyleData->m_uiRgbColorIndex, *GraphStyleData);
			OutGraphicProperties.MaterialUId = MaterialArchive.UEMaterialUId;
		}
		else
		{
			uint8 Alpha = 255;
			if (GraphStyleData->m_bIsTransparencyDefined)
			{
				Alpha = GraphStyleData->m_ucTransparency;
			}

			FArchiveColor& ColorArchive = FindOrAddColor(GraphStyleData->m_uiRgbColorIndex, Alpha);
			OutGraphicProperties.ColorUId = ColorArchive.UEMaterialUId;
		}
	}
}

void FTechSoftFileParser::ExtractTransformation3D(const A3DMiscTransformation* CartesianTransformation, FArchiveCADObject& Component)
{
	TUniqueTSObj<A3DMiscCartesianTransformationData> CartesianTransformationData(CartesianTransformation);

	if (CartesianTransformationData.IsValid())
	{
		FVector Origin(CartesianTransformationData->m_sOrigin.m_dX, CartesianTransformationData->m_sOrigin.m_dY, CartesianTransformationData->m_sOrigin.m_dZ);
		FVector XVector(CartesianTransformationData->m_sXVector.m_dX, CartesianTransformationData->m_sXVector.m_dY, CartesianTransformationData->m_sXVector.m_dZ);;
		FVector YVector(CartesianTransformationData->m_sYVector.m_dX, CartesianTransformationData->m_sYVector.m_dY, CartesianTransformationData->m_sYVector.m_dZ);;

		FVector ZVector = XVector ^ YVector;

		Origin *= Component.Unit * FImportParameters::GUnitScale;

		const A3DVector3dData& A3DScale = CartesianTransformationData->m_sScale;
		FVector3d Scale(A3DScale.m_dX, A3DScale.m_dY, A3DScale.m_dZ);
		double UniformScale = TechSoftFileParserImpl::ExtractUniformScale(Scale);

		XVector *= Scale.X;
		YVector *= Scale.Y;
		ZVector *= Scale.Z;

		Component.Unit *= UniformScale;

		Component.TransformMatrix = FMatrix(XVector, YVector, ZVector, FVector::ZeroVector);

		if (CartesianTransformationData->m_ucBehaviour & kA3DTransformationMirror)
		{
			Component.TransformMatrix.M[2][0] *= -1;
			Component.TransformMatrix.M[2][1] *= -1;
			Component.TransformMatrix.M[2][2] *= -1;
		}

		Component.TransformMatrix.SetOrigin(Origin);
	}
	else
	{
		Component.TransformMatrix = FMatrix::Identity;
	}
}

void FTechSoftFileParser::ExtractGeneralTransformation(const A3DMiscTransformation* GeneralTransformation, FArchiveCADObject& Component)
{
	TUniqueTSObj<A3DMiscGeneralTransformationData> GeneralTransformationData(GeneralTransformation);
	if (GeneralTransformationData.IsValid())
	{
		FMatrix Matrix = FMatrix::Identity;;
		int32 Index = 0;
		for (int32 Andex = 0; Andex < 4; ++Andex)
		{
			for (int32 Bndex = 0; Bndex < 4; ++Bndex, ++Index)
			{
				Matrix.M[Andex][Bndex] = GeneralTransformationData->m_adCoeff[Index];
			}
		}

		FTransform3d Transform(Matrix);
		FVector3d Scale = Transform.GetScale3D();
		if (Scale.Equals(FVector3d::OneVector, DOUBLE_KINDA_SMALL_NUMBER))
		{
			const double TranslationScale = Component.Unit * FImportParameters::GUnitScale;
			for (Index = 0; Index < 3; ++Index)
			{
				Matrix.M[3][Index] *= TranslationScale;
			}
			Component.TransformMatrix = Matrix;
		}
		else
		{
			FVector3d Translation = Transform.GetTranslation();
			Translation *= Component.Unit * FImportParameters::GUnitScale;

			double UniformScale = TechSoftFileParserImpl::ExtractUniformScale(Scale);
			Component.Unit *= UniformScale;

			FQuat4d Rotation = Transform.GetRotation();

			FTransform3d NewTransform;
			NewTransform.SetScale3D(Scale);
			NewTransform.SetRotation(Rotation);

			Component.TransformMatrix = NewTransform.ToMatrixWithScale();
			Component.TransformMatrix.SetOrigin(Translation);
		}
	}
	else
	{
		Component.TransformMatrix = FMatrix::Identity;
	}
}

void FTechSoftFileParser::ExtractTransformation(const A3DMiscTransformation* Transformation3D, FArchiveCADObject& Component)
{
	if (Transformation3D == NULL)
	{
		return;
	}

	A3DEEntityType Type = kA3DTypeUnknown;
	A3DEntityGetType(Transformation3D, &Type);

	if (Type == kA3DTypeMiscCartesianTransformation)
	{
		ExtractTransformation3D(Transformation3D, Component);
	}
	else if (Type == kA3DTypeMiscGeneralTransformation)
	{
		ExtractGeneralTransformation(Transformation3D, Component);
	}
}

void FTechSoftFileParser::ExtractCoordinateSystem(const A3DRiCoordinateSystem* CoordinateSystem, FArchiveCADObject& OutMetaData)
{
	TUniqueTSObj<A3DRiCoordinateSystemData> CoordinateSystemData(CoordinateSystem);
	if (CoordinateSystemData.IsValid())
	{
		ExtractTransformation(CoordinateSystemData->m_pTransformation, OutMetaData);
	}
	else
	{
		OutMetaData.TransformMatrix = FMatrix::Identity;
	}
}

bool FTechSoftFileParser::IsConfigurationSet(const A3DAsmProductOccurrence* Occurrence)
{
	switch (Format)
	{
	case ECADFormat::CATIAV4:
	case ECADFormat::N_X:
	case ECADFormat::SOLIDWORKS:
	{
		TUniqueTSObj<A3DAsmProductOccurrenceData> OccurrenceData(Occurrence);
		if (!OccurrenceData.IsValid())
		{
			return false;
		}

		return OccurrenceData->m_uiProductFlags & A3D_PRODUCT_FLAG_CONTAINER;
	}

	default : 
		return false;
	}
}

uint32 FTechSoftFileParser::CountColorAndMaterial()
{
	A3DGlobal* GlobalPtr = TechSoftInterface::GetGlobalPointer();
	if (GlobalPtr == nullptr)
	{
		return 0;
	}

	A3DInt32 iRet = A3D_SUCCESS;
	TUniqueTSObj<A3DGlobalData> GlobalData(GlobalPtr);
	if (!GlobalData.IsValid())
	{
		return 0;
	}

	uint32 ColorCount = GlobalData->m_uiColorsSize;
	uint32 MaterialCount = GlobalData->m_uiMaterialsSize;
	uint32 TextureDefinitionCount = GlobalData->m_uiTextureDefinitionsSize;

	return ColorCount + MaterialCount + TextureDefinitionCount;
}

void ExtractTextureDefinition(const A3DGraphTextureDefinitionData& TextureDefinitionData)
{
	// To do
	//TextureDefinitionData.m_uiPictureIndex;
	//TextureDefinitionData.m_ucTextureDimension;
	//TextureDefinitionData.m_eMappingType;
	//TextureDefinitionData.m_eMappingOperator;

	//TextureDefinitionData.m_pOperatorTransfo;

	//TextureDefinitionData.m_uiMappingAttributes;
	//TextureDefinitionData.m_uiMappingAttributesIntensitySize,
	//TextureDefinitionData.m_pdMappingAttributesIntensity;
	//TextureDefinitionData.m_uiMappingAttributesComponentsSize,
	//TextureDefinitionData.m_pucMappingAttributesComponents;
	//TextureDefinitionData.m_eTextureFunction;
	//TextureDefinitionData.m_dRed;
	//TextureDefinitionData.m_dGreen;
	//TextureDefinitionData.m_dBlue;
	//TextureDefinitionData.m_dAlpha;
	//TextureDefinitionData.m_eBlend_src_RGB;
	//TextureDefinitionData.m_eBlend_dst_RGB;
	//TextureDefinitionData.m_eBlend_src_Alpha;
	//TextureDefinitionData.m_eBlend_dst_Alpha;
	//TextureDefinitionData.m_ucTextureApplyingMode;
	//TextureDefinitionData.m_eTextureAlphaTest;
	//TextureDefinitionData.m_dAlphaTestReference;
	//TextureDefinitionData.m_eTextureWrappingModeS;
	//TextureDefinitionData.m_eTextureWrappingModeT;

	//if (TextureDefinitionData.m_pTextureTransfo != nullptr)
	//{
	//	TUniqueTSObj<A3DGraphTextureTransformationData> TransfoData(TextureDefinitionData.m_pTextureTransfo);
	//}
}

void FTechSoftFileParser::ReadMaterialsAndColors()
{
	A3DGlobal* GlobalPtr = TechSoftInterface::GetGlobalPointer();
	if (GlobalPtr == nullptr)
	{
		return;
	}

	A3DInt32 iRet = A3D_SUCCESS;
	TUniqueTSObj<A3DGlobalData> GlobalData(GlobalPtr);
	if (!GlobalData.IsValid())
	{
		return;
	}

	{
		uint32 TextureDefinitionCount = GlobalData->m_uiTextureDefinitionsSize;
		if (TextureDefinitionCount)
		{
			TUniqueTSObjFromIndex<A3DGraphTextureDefinitionData> TextureDefinitionData;
			for (uint32 TextureIndex = 0; TextureIndex < TextureDefinitionCount; ++TextureIndex)
			{
				TextureDefinitionData.FillFrom(TextureIndex);
				ExtractTextureDefinition(*TextureDefinitionData);
			}
		}
	}

	{
		uint32 PictureCount = GlobalData->m_uiPicturesSize;
		if (PictureCount)
		{
			TUniqueTSObjFromIndex<A3DGraphPictureData> PictureData;
			for (uint32 PictureIndex = 0; PictureIndex < PictureCount; ++PictureIndex)
			{
				A3DEntity* PicturePtr = TechSoftInterface::GetPointerFromIndex(PictureIndex, kA3DTypeGraphPicture);
				if (PicturePtr)
				{
					FArchiveCADObject PictureMetaData;
					ExtractMetaData(PicturePtr, PictureMetaData);
				}

				PictureData.FillFrom(PictureIndex);
				// To do
			}
		}
	}
}

void FTechSoftFileParser::CheckMemory()
{
	CADFileData.GetRecord().StartMemoryUsed = FPlatformMemory::GetStats().UsedPhysical;
	uint64& MaxMemoryUsed = CADFileData.GetRecord().MaxMemoryUsed;
	while (bProcessIsRunning)
	{
		FPlatformProcess::Sleep(0.1f);
		const uint64 MemoryUsed = FPlatformMemory::GetStats().UsedPhysical;
		if (MaxMemoryUsed < MemoryUsed)
		{
			MaxMemoryUsed = MemoryUsed;
		}
	}
}

#endif  

} // ns CADLibrary
