// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADFileParser.h"
#include "TechSoftInterface.h"

#include <atomic>

typedef void A3DAsmProductOccurrence;
typedef void A3DRiRepresentationItem;

namespace CADLibrary
{

class FArchiveBody;
class FArchiveCADObject;
class FArchiveColor;
class FArchiveGraphicProperties;
class FArchiveInstance;
class FArchiveMaterial;
class FArchiveReference;
class FArchiveSceneGraph;
class FArchiveUnloadedReference;
class FArchiveWithOverridenChildren;
class FCADFileData;

class FTechSoftFileParser : public ICADFileParser
{
public:

	/**
	 * @param InCADData TODO
	 * @param EnginePluginsPath Full Path of EnginePlugins. Mandatory to set KernelIO to import DWG, or DGN files
	 */
	FTechSoftFileParser(FCADFileData& InCADData, const FString& EnginePluginsPath = TEXT(""));

#ifndef USE_TECHSOFT_SDK
	virtual ECADParsingResult Process() override
	{
		return ECADParsingResult::ProcessFailed;
	}
#else
	virtual ECADParsingResult Process() override;

	double GetFileUnit()
	{
		return FileUnit;
	}

	void ExtractMetaData(const A3DEntity* Entity, FArchiveCADObject& OutMetaData);

protected:

	virtual A3DStatus AdaptBRepModel()
	{
		return A3DStatus::A3D_SUCCESS;
	}

	/**
	 * If the tessellator is TechSoft, SewModel call TechSoftInterface::SewModel
	 * If the tessellator is UE::CADKernel, SewModel do nothing as the file is not yet parsed. In this case, the sew is done in GenerateBodyMeshes.
	 */
	virtual void SewModel();

	// Tessellation methods
	virtual void GenerateBodyMeshes();
	virtual void GenerateBodyMesh(A3DRiRepresentationItem* Representation, FArchiveBody& Body);


private:
	// Methods to parse a first time the file to count the number of components
	// Needed to reserve CADFileData
	// Start with CountUnderModel
	void CountUnderModel();
	void CountUnderConfigurationSet(const A3DAsmProductOccurrence* Occurrence);
	void CountUnderOccurrence(const A3DAsmProductOccurrence* Occurrence);
	void CountUnderPartDefinition(const A3DAsmPartDefinition* PartDefinition);
	void CountUnderRepresentationItem(const A3DRiRepresentationItem* RepresentationItem);
	void CountUnderRepresentationSet(const A3DRiSet* RepresentationSet);
	void CountUnderOverrideOccurrence(const A3DAsmProductOccurrence* Occurrence);

	void ReserveCADFileData();

	// Materials and colors
	uint32 CountColorAndMaterial();
	void ReadMaterialsAndColors();

	// Traverse ASM tree by starting from the model
	ECADParsingResult TraverseModel();

	// Note:
	// Due to dataprep purpose, node without name cannot have a generic name (e.g. Product, Part, Body, Shell, etc...) but must have a name based on its parent's name.
	// To be able to build a name, the name of the parent has to be known.
	// The implementation of the naming policy is done in 5.0.3 with minimal code modification. However the model parsing need to be rewrite in the next version. (Jira UE-152624)

	void TraverseReference(const A3DAsmProductOccurrence* A3DReference, FArchiveReference& Reference);
	bool IsConfigurationSet(const A3DAsmProductOccurrence* Occurrence);
	void TraverseConfigurationSet(const A3DAsmProductOccurrence* ConfigurationSet, FArchiveReference& Reference);
	void TraverseOccurrence(const A3DAsmProductOccurrence* Occurrence, FArchiveReference& ParentReference);
	void ProcessPrototype(const A3DAsmProductOccurrence* InPrototype, FArchiveUnloadedReference& OutReference, A3DMiscTransformation** OutLocation);
	void ProcessReference(const A3DAsmProductOccurrence* OccurrencePtr, FArchiveInstance& Instance, FArchiveReference& Reference);
	void ProcessUnloadedReference(const FArchiveInstance& Instance, FArchiveUnloadedReference& Reference);

	void TraversePartDefinition(const A3DAsmPartDefinition* InPartDefinition, FArchiveReference& Parent);
	void TraverseExternalData(const A3DAsmProductOccurrence* InExternalData, FArchiveReference& Parent);
	void TraverseRepresentationSet(const A3DRiSet* InSetPtr, FArchiveReference& Parent);
	void TraverseRepresentationItem(A3DRiRepresentationItem* RepresentationItem, FArchiveReference& Parent);
	void TraverseBRepModel(A3DRiBrepModel* BrepModel, FArchiveReference& Parent);
	void TraversePolyBRepModel(A3DRiPolyBrepModel* PolygonalBrepModel, FArchiveReference& Parent);

	// MetaData
	void ExtractSpecificMetaData(const A3DAsmProductOccurrence* Occurrence, FArchiveCADObject& OutMetaData);

	void BuildInstanceName(FArchiveCADObject& MetaData, const FArchiveReference& Parent);
	void BuildReferenceName(FArchiveCADObject& Reference);
	void BuildPartName(FArchiveCADObject& Part);
	void BuildBodyName(FArchiveBody& MetaData, const FArchiveReference& Parent);
	void BuildRepresentationSetName(FArchiveCADObject& MetaData, const FArchiveReference& Parent);

	void ExtractGraphicProperties(const A3DGraphics* Graphics, FArchiveCADObject& OutMetaData);

	/**
	 * Recursive method to extract the subtree of override occurrences of the children tree of an instance 
	 */
	void ExtractOverrideOccurrenceSubtree(const A3DAsmProductOccurrence* OccurrencePtr, FArchiveWithOverridenChildren& Parent);

	/**
	 * ColorName and MaterialName have to be initialized before.
	 * This method update the value ColorName or MaterialName accordingly of the GraphStyleData type (material or color)
	 */
	void ExtractGraphStyleProperties(uint32 StyleIndex, FArchiveGraphicProperties& OutGraphicProperties);
	FArchiveColor& FindOrAddColor(uint32 ColorIndex, uint8 Alpha);
	FArchiveMaterial& FindOrAddMaterial(FMaterialUId MaterialId, const A3DGraphStyleData& GraphStyleData);

	/**
	 * @param GraphMaterialIndex is the Techsoft index of the graphic data
	 * @param MaterialIndexToSave is the index of the material really saved (i.e. for texture, at the texture index, with saved the material used by the texture)
	 */
	FArchiveMaterial& AddMaterialAt(uint32 MaterialIndexToSave, uint32 GraphMaterialIndex, const A3DGraphStyleData& GraphStyleData);
	FArchiveMaterial& AddMaterial(uint32 MaterialIndex, const A3DGraphStyleData& GraphStyleData)
	{
		return AddMaterialAt(MaterialIndex, MaterialIndex, GraphStyleData);
	}

	// Transform
	void ExtractCoordinateSystem(const A3DRiCoordinateSystem* CoordinateSystem, FArchiveCADObject& Component);
	void ExtractTransformation(const A3DMiscTransformation* Transformation3d, FArchiveCADObject& Component);
	void ExtractGeneralTransformation(const A3DMiscTransformation* GeneralTransformation, FArchiveCADObject& Component);
	void ExtractTransformation3D(const A3DMiscTransformation* CartesianTransformation, FArchiveCADObject& Component);

#ifndef CADKERNEL_DEV
	void CheckMemory();
#endif  

#endif

protected:

	FUniqueTechSoftModelFile ModelFile;

	TSet<const A3DAsmProductOccurrence*> PrototypeCounted;
	uint32 ComponentCount[EComponentType::LastType] = { 0 };

	FCADFileData& CADFileData;
	FArchiveSceneGraph& SceneGraph;

	FTechSoftInterface& TechSoftInterface;
	ECADFormat Format;
	bool bForceSew = false;

	EModelerType ModelerType;
	double FileUnit = 1;

	TMap<A3DRiRepresentationItem*, FCadId> RepresentationItemsCache;
	TMap<A3DAsmProductOccurrence*, FCadId> ReferenceCache;

	bool bConvertionFailed = false;

	std::atomic<bool> bProcessIsRunning;

};

} // ns CADLibrary