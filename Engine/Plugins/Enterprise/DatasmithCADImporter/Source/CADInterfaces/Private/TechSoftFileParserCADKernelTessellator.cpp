// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechSoftFileParserCADKernelTessellator.h"


#include "CADFileData.h"
#include "CADOptions.h"
#include "CADKernelTools.h"
#include "TechSoftUtils.h"
#include "TechSoftBridge.h"
#include "TUniqueTechSoftObj.h"

#include "Core/CADEntity.h"
#include "Core/Session.h"
#include "Core/Types.h"

#include "Mesh/Meshers/Mesher.h"
#include "Mesh/Structure/ModelMesh.h"

#include "Topo/Body.h"
#include "Topo/Model.h"
#include "Topo/TopologicalShapeEntity.h"
#include "Topo/Topomaker.h"

#include "HAL/PlatformTime.h"

namespace CADLibrary
{

FTechSoftFileParserCADKernelTessellator::FTechSoftFileParserCADKernelTessellator(FCADFileData& InCADData, const FString& EnginePluginsPath)
	: FTechSoftFileParser(InCADData, EnginePluginsPath)
	, LastHostIdUsed(1 << 30)
{
}

#ifdef USE_TECHSOFT_SDK

void FTechSoftFileParserCADKernelTessellator::GenerateBodyMeshes()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTechSoftFileParserCADKernelTessellator::GenerateBodyMeshes);
	if (bForceSew || CADFileData.GetImportParameters().GetStitchingTechnique() == EStitchingTechnique::StitchingSew)
	{
		SewAndGenerateBodyMeshes();
	}
	else
	{
		// If no sew is required, FTechSoftFileParser::GenerateBodyMeshes is called to process all the bodies one by one with the override GenerateBodyMesh
		FTechSoftFileParser::GenerateBodyMeshes();
	}
}

void FTechSoftFileParserCADKernelTessellator::SewAndGenerateBodyMeshes()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTechSoftFileParserCADKernelTessellator::SewAndGenerateBodyMeshes);

	int32 RepresentationCount = RepresentationItemsCache.Num();
	TMap<FCadId, TArray<A3DRiRepresentationItem*>> OccurrenceIdToRepresentations;
	for (TPair<A3DRiRepresentationItem*, FCadId>& Entry : RepresentationItemsCache)
	{
		A3DRiRepresentationItem* RepresentationItemPtr = Entry.Key;
		FArchiveBody& Body = SceneGraph.GetBody(Entry.Value);
		if (!Body.bIsFromCad)
		{
			FTechSoftFileParser::GenerateBodyMesh(RepresentationItemPtr, Body);
			continue;
		}

		TArray<A3DRiRepresentationItem*>& Representations = OccurrenceIdToRepresentations.FindOrAdd(Body.ParentId);
		if (Representations.Max() == 0)
		{
			Representations.Reserve(RepresentationCount);
		}

		Representations.Add(RepresentationItemPtr);
	}

	for (TPair<FCadId, TArray<A3DRiRepresentationItem*>>& Representations : OccurrenceIdToRepresentations)
	{
		SewAndMesh(Representations.Value);
		if (bConvertionFailed)
		{
			return;
		}
	}

	// Delete unused ArchiveBody
	for (FArchiveBody& ArchiveBody : SceneGraph.Bodies)
	{
		if (ArchiveBody.MeshActorUId == 0 && !ArchiveBody.IsDeleted())
		{
			ArchiveBody.Delete();
		}
	}
}

void FTechSoftFileParserCADKernelTessellator::SewAndMesh(TArray<A3DRiRepresentationItem*>& Representations)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTechSoftFileParserCADKernelTessellator::SewAndMesh);

	using namespace UE::CADKernel;

	const double GeometricTolerance = FImportParameters::GStitchingTolerance * 10; // cm to mm

	FSession CADKernelSession(GeometricTolerance);
	CADKernelSession.SetFirstNewHostId(LastHostIdUsed);
	FModel& CADKernelModel = CADKernelSession.GetModel();

	FTechSoftBridge TechSoftBridge(*this, CADKernelSession);

	for (A3DRiRepresentationItem* Representation : Representations)
	{
		FCadId* BodyIndex = RepresentationItemsCache.Find(Representation);
		if (BodyIndex == nullptr)
		{
			continue;
		}
		FArchiveBody& Body = SceneGraph.GetBody(*BodyIndex);

		FBody* CADKernelBody = TechSoftBridge.AddBody(Representation, Body);
		if (CADKernelBody == nullptr)
		{
			const EFailureReason FailureReason = TechSoftBridge.GetFailureReason();
			switch(FailureReason)
			{
				case EFailureReason::Curve3D :
				{
					// The failure of the current body is due to an incompatibility of modeler.
					// Reimport of the full file with another modeler
					bConvertionFailed = true;
					return;
				}

				case EFailureReason::Unknown:
				default:
				{
					// The failure of the current body is not due to an incompatibility of modeler.
					// A reimport of the full file with another modeler is not wanted
					break;
				}
			}
		}
	}

	// Sew if needed
	uint64 StartSewTime = FPlatformTime::Cycles64();
	UE::CADKernel::FTopomakerOptions TopomakerOptions((UE::CADKernel::ESewOption) SewOption::GetFromImportParameters(), GeometricTolerance, FImportParameters::GStitchingForceFactor);

	FTopomaker Topomaker(CADKernelSession, TopomakerOptions);
	Topomaker.Sew();
	Topomaker.SplitIntoConnectedShells();
	Topomaker.OrientShells();

	// The Sew + SplitIntoConnectedShells change the bodies: some are deleted some are create
	// but at the end the count of body is always <= than the initial count
	// We need to found the unchanged bodies to link them to their FArchiveBody
	// The new bodies will be linked to FArchiveBody of deleted bodies but the metadata of these FArchiveBody have to be cleaned

	int32 BodyCount = CADKernelModel.GetBodies().Num();

	TArray<FBody*> NewBodies;
	TArray<FBody*> ExistingBodies;
	NewBodies.Reserve(BodyCount);
	ExistingBodies.Reserve(BodyCount);

	// find new and existing bodies
	for (const TSharedPtr<FBody>& CADKernelBody : CADKernelModel.GetBodies())
	{
		if (!CADKernelBody.IsValid())
		{
			continue;
		}

		const A3DRiRepresentationItem* Representation = TechSoftBridge.GetA3DBody(CADKernelBody.Get());
		if (Representation == nullptr)
		{
			NewBodies.Add(CADKernelBody.Get());
		}
		else
		{
			ExistingBodies.Add(CADKernelBody.Get());
		}
	}

	// find Representation of deleted bodies to find unused FArchiveBody
	TArray<FCadId> ArchiveBodyIdOfDeletedRepresentation;
	int32 DeletedBodyCount = Representations.Num() - BodyCount;
	if (DeletedBodyCount > 0)
	{
		ArchiveBodyIdOfDeletedRepresentation.Reserve(DeletedBodyCount);

		for (A3DRiRepresentationItem* Representation : Representations)
		{
			FBody* Body = TechSoftBridge.GetBody(Representation);
			if (Body == nullptr)
			{
				FCadId* ArchiveBodyId = RepresentationItemsCache.Find(Representation);
				if (ArchiveBodyId == nullptr)
				{
					continue; // should not happen
				}
				ArchiveBodyIdOfDeletedRepresentation.Add(*ArchiveBodyId);
			}
		}
	}

	CADFileData.GetRecord().SewTime += FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartSewTime);

	uint64 StartMeshTime = FPlatformTime::Cycles64();

	// Process existing bodies
	for (FBody* Body : ExistingBodies)
	{
		const A3DRiRepresentationItem* Representation = TechSoftBridge.GetA3DBody(Body);

		FCadId* ArchiveBodyId = RepresentationItemsCache.Find(Representation);
		if (ArchiveBodyId == nullptr)
		{
			continue; // should not append
		}

		FArchiveBody& ArchiveBody = SceneGraph.GetBody(*ArchiveBodyId);
		MeshAndGetTessellation(CADKernelSession, ArchiveBody, *Body);
	}

	if (ArchiveBodyIdOfDeletedRepresentation.Num())
	{
		// Process new bodies
		int32 Index = 0;
		for (FBody* Body : NewBodies)
		{
			FCadId ArchiveBodyId = ArchiveBodyIdOfDeletedRepresentation[Index++];
			FArchiveBody& ArchiveBody = SceneGraph.GetBody(ArchiveBodyId);
			MeshAndGetTessellation(CADKernelSession, ArchiveBody, *Body);
		}

		// Delete unused FArchiveBody
		for (; Index < ArchiveBodyIdOfDeletedRepresentation.Num(); ++Index)
		{
			FCadId ArchiveBodyId = ArchiveBodyIdOfDeletedRepresentation[Index++];
			FArchiveBody& ArchiveBody = SceneGraph.GetBody(ArchiveBodyId);
			ArchiveBody.Delete();
		}
	}

	CADFileData.GetRecord().MeshTime += FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartMeshTime);
}

void FTechSoftFileParserCADKernelTessellator::GenerateBodyMesh(A3DRiRepresentationItem* Representation, FArchiveBody& ArchiveBody)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTechSoftFileParserCADKernelTessellator::GenerateBodyMesh);
	using namespace UE::CADKernel;

	const double GeometricTolerance = FImportParameters::GStitchingTolerance * 10; // cm to mm
	FSession CADKernelSession(GeometricTolerance);
	CADKernelSession.SetFirstNewHostId(LastHostIdUsed);
	FModel& CADKernelModel = CADKernelSession.GetModel();

	FTechSoftBridge TechSoftBridge(*this, CADKernelSession);

	FBody* CADKernelBody = TechSoftBridge.AddBody(Representation, ArchiveBody);
	if (CADKernelBody == nullptr)
	{
		ArchiveBody.Delete();
		const EFailureReason FailureReason = TechSoftBridge.GetFailureReason();
		if(FailureReason == EFailureReason::Curve3D)
		{
			bConvertionFailed = true;
		}
		return;
	}

	CADKernelBody->SetDisplayData(ArchiveBody.ColorUId, ArchiveBody.MaterialUId);

	if (CADFileData.GetImportParameters().GetStitchingTechnique() == StitchingHeal)
	{
		UE::CADKernel::FTopomakerOptions TopomakerOptions((UE::CADKernel::ESewOption)SewOption::GetFromImportParameters(), GeometricTolerance, FImportParameters::GStitchingForceFactor);

		FTopomaker Topomaker(CADKernelSession, TopomakerOptions);
		Topomaker.Sew();
		Topomaker.OrientShells();
	}

	MeshAndGetTessellation(CADKernelSession, ArchiveBody, *CADKernelBody);
}

void FTechSoftFileParserCADKernelTessellator::MeshAndGetTessellation(UE::CADKernel::FSession& CADKernelSession, FArchiveBody& ArchiveBody, UE::CADKernel::FBody& CADKernelBody)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTechSoftFileParserCADKernelTessellator::MeshAndGetTessellation);
	using namespace UE::CADKernel;

	FBodyMesh& BodyMesh = CADFileData.AddBodyMesh(ArchiveBody.Id, ArchiveBody);

	// Save Body in CADKernelArchive file for re-tessellation
	if (CADFileData.IsCacheDefined())
	{
		FString BodyFilePath = CADFileData.GetBodyCachePath(ArchiveBody.MeshActorUId);
		CADKernelSession.SaveDatabase(*BodyFilePath, CADKernelBody);
	}

	// Tessellate the body
	TSharedRef<FModelMesh> CADKernelModelMesh = FEntity::MakeShared<FModelMesh>();

	const double GeometricTolerance = CADKernelSession.GetGeometricTolerance();

	FCADKernelTools::DefineMeshCriteria(CADKernelModelMesh.Get(), CADFileData.GetImportParameters(), GeometricTolerance);

	const bool bActivateThinZoneMeshing = FImportParameters::bGActivateThinZoneMeshing;
	FMesher Mesher(*CADKernelModelMesh, GeometricTolerance, bActivateThinZoneMeshing);
	Mesher.MeshEntity(CADKernelBody);

	FCADKernelTools::GetBodyTessellation(*CADKernelModelMesh, CADKernelBody, BodyMesh);
	if (BodyMesh.Faces.Num() == 0)
	{
		ArchiveBody.Delete();
	}

	ArchiveBody.ColorFaceSet = BodyMesh.ColorSet;
	ArchiveBody.MaterialFaceSet = BodyMesh.MaterialSet;
}

A3DStatus FTechSoftFileParserCADKernelTessellator::AdaptBRepModel()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTechSoftFileParserCADKernelTessellator::AdaptBRepModel);

	const A3DUns32 ValidSurfaceCount = 9;
	A3DUns32 AcceptedSurfaces[ValidSurfaceCount] = {
		//kA3DTypeSurfBlend01,
		//kA3DTypeSurfBlend02,
		//kA3DTypeSurfBlend03,
		kA3DTypeSurfNurbs,
		kA3DTypeSurfCone,
		kA3DTypeSurfCylinder,
		kA3DTypeSurfCylindrical,
		//kA3DTypeSurfOffset,
		//kA3DTypeSurfPipe,
		kA3DTypeSurfPlane,
		kA3DTypeSurfRuled,
		kA3DTypeSurfSphere,
		kA3DTypeSurfRevolution,
		//kA3DTypeSurfExtrusion,
		//kA3DTypeSurfFromCurves,
		kA3DTypeSurfTorus,
		//kA3DTypeSurfTransform,
	};

	const A3DUns32 ValidCurveCount = 7;
	A3DUns32 AcceptedCurves[ValidCurveCount] = {
		//kA3DTypeCrvBase,
		//kA3DTypeCrvBlend02Boundary,
		kA3DTypeCrvNurbs,
		kA3DTypeCrvCircle,
		//kA3DTypeCrvComposite,
		//kA3DTypeCrvOnSurf,
		kA3DTypeCrvEllipse,
		//kA3DTypeCrvEquation,
		//kA3DTypeCrvHelix,
		kA3DTypeCrvHyperbola,
		//kA3DTypeCrvIntersection,
		kA3DTypeCrvLine,
		//kA3DTypeCrvOffset,
		kA3DTypeCrvParabola,
		kA3DTypeCrvPolyLine,
		//kA3DTypeCrvTransform,
	};

	TUniqueTSObj<A3DCopyAndAdaptBrepModelData> CopyAndAdaptBrepModelData;
	CopyAndAdaptBrepModelData->m_bUseSameParam = false;                        // If `A3D_TRUE`, surfaces will keep their parametrization when converted to NURBS.       
	CopyAndAdaptBrepModelData->m_dTol = 1e-3;                                  // Tolerance value of resulting B-rep. The value is relative to the scale of the model.
	CopyAndAdaptBrepModelData->m_bDeleteCrossingUV = false;                    // If `A3D_TRUE`, UV curves that cross seams of periodic surfaces are replaced by 3D curves 
	CopyAndAdaptBrepModelData->m_bSplitFaces = true;                           // If `A3D_TRUE`, the faces with a periodic basis surface are split on parametric seams
	CopyAndAdaptBrepModelData->m_bSplitClosedFaces = false;                    // If `A3D_TRUE`, the faces with a closed basis surface are split into faces at the parametric seam and mid-parameter
	CopyAndAdaptBrepModelData->m_bForceComputeUV = true;                       // If `A3D_TRUE`, UV curves are computed from the B-rep data
	CopyAndAdaptBrepModelData->m_bAllowUVCrossingSeams = true;                 // If `A3D_TRUE` and m_bForceComputeUV is set to `A3D_TRUE`, computed UV curves can cross seams.
	CopyAndAdaptBrepModelData->m_bForceCompute3D = false;                      // If `A3D_TRUE`, 3D curves are computed from the B-rep data
	CopyAndAdaptBrepModelData->m_bContinueOnError = true;                      // Continue processing even if an error occurs. Use \ref A3DCopyAndAdaptBrepModelAdvanced to get the error status.
	CopyAndAdaptBrepModelData->m_bClampTolerantUVCurvesInsideUVDomain = false; // If `A3D_FALSE`, UV curves may stray outside the UV domain as long as the 3D edge tolerance is respected. If set to `A3D_TRUE`, the UV curves will be clamped to the UV domain (if the clamp still leaves them within the edge tolerance). */
	CopyAndAdaptBrepModelData->m_bForceDuplicateGeometries = false;            // If `A3D_TRUE`, break the sharing of surfaces and curves into topologies.*/

	CopyAndAdaptBrepModelData->m_uiAcceptableSurfacesSize = ValidSurfaceCount;
	CopyAndAdaptBrepModelData->m_puiAcceptableSurfaces = &AcceptedSurfaces[0];
	CopyAndAdaptBrepModelData->m_uiAcceptableCurvesSize = ValidCurveCount;
	CopyAndAdaptBrepModelData->m_puiAcceptableCurves = &AcceptedCurves[0];

	int32 ErrorCount = 0;
	A3DCopyAndAdaptBrepModelErrorData* Errors = nullptr;
	A3DStatus Ret = TechSoftInterface::AdaptBRepInModelFile(ModelFile.Get(), *CopyAndAdaptBrepModelData, ErrorCount, &Errors);
	if ((Ret == A3D_SUCCESS || Ret == A3D_TOOLS_CONTINUE_ON_ERROR) && ErrorCount > 0)
	{
		// Add message about non-critical errors during the adaptation
		CADFileData.LogMessage(FString::Printf(TEXT("File %s had %d non-critical error(s) during BRep adaptation step."), *CADFileData.GetCADFileDescription().GetFileName(), ErrorCount));
	}
	else if (Ret != A3D_SUCCESS)
	{
		CADFileData.LogWarning(FString::Printf(TEXT("File %s failed during BRep adaptation step."), *CADFileData.GetCADFileDescription().GetFileName()));
		return A3D_ERROR;
	}
	return A3D_SUCCESS;
}
#endif  

} // ns CADLibrary
