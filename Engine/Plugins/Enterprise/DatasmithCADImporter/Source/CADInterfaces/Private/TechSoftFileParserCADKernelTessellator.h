// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "TechSoftFileParser.h"
#include "CADOptions.h"

namespace UE::CADKernel
{
class FBody;
class FSession;
}

namespace CADLibrary
{

class CADINTERFACES_API FTechSoftFileParserCADKernelTessellator : public FTechSoftFileParser
{
public:

	/**
	 * @param InCADData TODO
	 * @param EnginePluginsPath Full Path of EnginePlugins. Mandatory to set KernelIO to import DWG, or DGN files
	 */
	FTechSoftFileParserCADKernelTessellator(FCADFileData& InCADData, const FString& EnginePluginsPath = TEXT(""));

#ifdef USE_TECHSOFT_SDK

private:

	virtual A3DStatus AdaptBRepModel() override;

	virtual void SewModel() override
	{
		// Do in GenerateBodyMeshes
	}

	// Tessellation methods
	virtual void GenerateBodyMeshes() override;
	virtual void GenerateBodyMesh(A3DRiRepresentationItem* Representation, FArchiveBody& Body) override;
	void MeshAndGetTessellation(UE::CADKernel::FSession& CADKernelSession, FArchiveBody& ArchiveBody, UE::CADKernel::FBody& CADKernelBody);

	void SewAndGenerateBodyMeshes();
	void SewAndMesh(TArray<A3DRiRepresentationItem*>& Representations);

#endif

private:

	int32 LastHostIdUsed = 0;
};

} // ns CADLibrary