// Copyright Epic Games, Inc. All Rights Reserved.

#define INITIALIZE_A3D_API

#include "TechSoftInterface.h"
#ifdef USE_TECHSOFT_SDK
#include "hoops_license.h"
#endif

#include "TUniqueTechSoftObj.h"

#include "CADInterfacesModule.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"

namespace CADLibrary
{

#ifdef USE_TECHSOFT_SDK
const uint32 FTechSoftDefaultValue::Material = A3D_DEFAULT_MATERIAL_INDEX;
const uint32 FTechSoftDefaultValue::Picture = A3D_DEFAULT_PICTURE_INDEX;
const uint32 FTechSoftDefaultValue::RgbColor = A3D_DEFAULT_COLOR_INDEX;
const uint32 FTechSoftDefaultValue::Style = A3D_DEFAULT_STYLE_INDEX;
const uint32 FTechSoftDefaultValue::TextureApplication = A3D_DEFAULT_TEXTURE_APPLICATION_INDEX;
const uint32 FTechSoftDefaultValue::TextureDefinition = A3D_DEFAULT_TEXTURE_DEFINITION_INDEX;
#endif

FTechSoftInterface& FTechSoftInterface::Get()
{
	static FTechSoftInterface TechSoftInterface;
	return TechSoftInterface;
}

const TCHAR* FTechSoftInterface::GetVersion()
{
#ifdef USE_TECHSOFT_SDK
	static const FString Version = TEXT("TechSoft ") + FString::FromInt(A3D_DLL_MAJORVERSION) + TEXT(".") + FString::FromInt(A3D_DLL_MINORVERSION) + TEXT(".") + FString::FromInt(A3D_DLL_UPDATEVERSION);
	return bIsInitialize ? *Version : TEXT("TechSoft uninitialized");
#else
	return TEXT("TechSoft unavailable");
#endif
}

bool FTechSoftInterface::InitializeKernel(const TCHAR* InEngineDir)
{
#ifdef USE_TECHSOFT_SDK
	if (bIsInitialize)
	{
		return true;
	}

	FString EngineDir(InEngineDir);
	if (EngineDir.IsEmpty())
	{
		EngineDir = FPaths::EngineDir();
	}

	static const TCHAR* ThirdPartyDir = TEXT("Binaries/ThirdParty/TechSoft");
	
	FString TechSoftDllPath = FPaths::Combine(EngineDir, ThirdPartyDir, FPlatformProcess::GetBinariesSubdirectory());
	TechSoftDllPath = FPaths::ConvertRelativePathToFull(TechSoftDllPath);

	if (A3DSDKLoadLibraryA(TCHAR_TO_UTF8(*TechSoftDllPath)))
	{
		A3DStatus Status = A3DLicPutUnifiedLicense(HOOPS_LICENSE);

		A3DInt32 iMajorVersion = 0, iMinorVersion = 0;
		Status = A3DDllGetVersion(&iMajorVersion, &iMinorVersion);
		if (Status == A3D_SUCCESS)
		{
			Status = A3DDllInitialize(A3D_DLL_MAJORVERSION, A3D_DLL_MINORVERSION);
			if (Status == A3D_SUCCESS)
			{
				bIsInitialize = true;
				return true;
			}
		}
	}
#ifndef CADKERNEL_DEV
	UE_LOGF(LogCADInterfaces, Warning, "Failed to load required library in %ls. Plug-in will not be functional.", *TechSoftDllPath);
#endif

#endif
	return false;
}

namespace TechSoftInterface
{

bool TECHSOFT_InitializeKernel(const TCHAR* InEnginePluginsPath)
{
	return FTechSoftInterface::Get().InitializeKernel(InEnginePluginsPath);
}

FString GetTechSoftVersion()
{
	return FTechSoftInterface::Get().GetVersion();
}

#ifdef USE_TECHSOFT_SDK

FUniqueTechSoftModelFile LoadModelFileFromFile(const A3DImport& Import, A3DStatus& Status)
{
	A3DAsmModelFile* ModelFile = nullptr;
#if !PLATFORM_EXCEPTIONS_DISABLED
	try
#endif
	{
		Status = A3DAsmModelFileLoadFromFile(Import.GetFilePath(), &Import.m_sLoadData, &ModelFile);
	}
#if !PLATFORM_EXCEPTIONS_DISABLED
	catch (...)
	{
#ifndef CADKERNEL_DEV
		UE_LOGF(LogCADInterfaces, Warning, "Failed to load %s. An exception is thrown.", Import.GetFilePath());
#endif
		return FUniqueTechSoftModelFile();
	}
#endif

	switch (Status)
	{
	case A3D_LOAD_MULTI_MODELS_CADFILE: //if the file contains multiple entries (see A3DRWParamsMultiEntriesData).
	case A3D_LOAD_MISSING_COMPONENTS: //_[I don't know about this one]_
	case A3D_SUCCESS:
		return FUniqueTechSoftModelFile(ModelFile);

	default:
		break;
	}
	return FUniqueTechSoftModelFile();
}

FUniqueTechSoftModelFile LoadModelFileFromPrcFile(const A3DUTF8Char* CADFileName, A3DRWParamsPrcReadHelper** ReadHelper)
{
	A3DAsmModelFile* ModelFile = nullptr;
	if (A3DAsmModelFileLoadFromPrcFile(CADFileName, ReadHelper, &ModelFile) != A3DStatus::A3D_SUCCESS)
	{
		return FUniqueTechSoftModelFile();
	}
	return FUniqueTechSoftModelFile(ModelFile);
}

A3DStatus AdaptBRepInModelFile(A3DAsmModelFile* ModelFile, const A3DCopyAndAdaptBrepModelData& Setting, int32& ErrorCount, A3DCopyAndAdaptBrepModelErrorData** Errors)
{
	A3DUns32 NbErrors;
	A3DStatus Status = A3DAdaptAndReplaceAllBrepInModelFileAdvanced(ModelFile, &Setting, &NbErrors, Errors);
	ErrorCount = NbErrors;
	return Status;
}

A3DStatus DeleteModelFile(A3DAsmModelFile* ModelFile)
{
	return A3DAsmModelFileDelete(ModelFile);
}

A3DStatus DeleteEntity(A3DEntity* EntityPtr)
{
	return A3DEntityDelete(EntityPtr);
}

A3DGlobal* GetGlobalPointer()
{
	A3DGlobal* GlobalPtr;
	if(A3DGlobalGetPointer(&GlobalPtr) == A3DStatus::A3D_SUCCESS)
	{
		return GlobalPtr;
	}
	return nullptr;
}

A3DEntity* GetPointerFromIndex(const uint32 Index, const A3DEEntityType Type)
{
	A3DEntity* EntityPtr = nullptr;
	if (A3DMiscPointerFromIndexGet(Index, Type, &EntityPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return EntityPtr;
}

A3DStatus GetSurfaceAsNurbs(const A3DSurfBase* SurfacePtr, A3DSurfNurbsData* OutDataPtr, A3DDouble Tolerance, A3DBool bUseSameParameterization)
{
	return A3DSurfBaseWithDomainGetAsNurbs(SurfacePtr, nullptr, Tolerance, bUseSameParameterization, OutDataPtr);
}

A3DStatus GetSurfaceDomain(const A3DSurfBase* SurfacePtr, A3DDomainData& OutDomain)
{
	return A3DSurfGetDomain(SurfacePtr, &OutDomain);
}

A3DStatus Evaluate(const A3DSurfBase* SurfacePtr, const A3DVector2dData& UVParameter, A3DUns32 Derivatives, A3DVector3dData* OutPointAndDerivatives)
{
	return A3DSurfEvaluate(SurfacePtr, &UVParameter, Derivatives, OutPointAndDerivatives);
}

A3DStatus GetCurveAsNurbs(const A3DCrvBase* CurvePtr, A3DCrvNurbsData* OutDataPtr, A3DDouble Tolerance, A3DBool bUseSameParameterization)
{
	return A3DCrvBaseGetAsNurbs(CurvePtr, Tolerance, bUseSameParameterization, OutDataPtr);
}

A3DStatus GetOriginalFilePathName(const A3DAsmProductOccurrence* A3DOccurrencePtr, A3DUTF8Char** FilePathUTF8Ptr)
{
	return A3DAsmProductOccurrenceGetOriginalFilePathName(A3DOccurrencePtr, FilePathUTF8Ptr);
}

A3DStatus GetFilePathName(const A3DAsmProductOccurrence* A3DOccurrencePtr, A3DUTF8Char** FilePathUTF8Ptr)
{
	return A3DAsmProductOccurrenceGetFilePathName(A3DOccurrencePtr, FilePathUTF8Ptr);
}

A3DStatus GetEntityType(const A3DEntity* EntityPtr, A3DEEntityType* EntityTypePtr)
{
	return A3DEntityGetType(EntityPtr, EntityTypePtr);
}

bool IsEntityBaseWithGraphicsType(const A3DEntity* EntityPtr)
{
	return (bool)A3DEntityIsBaseWithGraphicsType(EntityPtr);
}

bool IsEntityBaseType(const A3DEntity* EntityPtr)
{
	return (bool)A3DEntityIsBaseType(EntityPtr);
}

bool IsMaterialTexture(const uint32 MaterialIndex)
{
	A3DBool bIsTexture = false;
	return A3DGlobalIsMaterialTexture(MaterialIndex, &bIsTexture) == A3DStatus::A3D_SUCCESS ? bool(bIsTexture) : false;
}

FUniqueTechSoftModelFile CreateAsmModelFile(const A3DAsmModelFileData& ModelFileData)
{
	A3DAsmModelFile* ModelFile = nullptr;
	if (A3DAsmModelFileCreate(&ModelFileData, &ModelFile) != A3DStatus::A3D_SUCCESS)
	{
		return FUniqueTechSoftModelFile();
	}
	return FUniqueTechSoftModelFile(ModelFile);
}

A3DRiBrepModel* CreateRiBRepModel(const A3DRiBrepModelData& RiBRepModelData)
{
	A3DRiBrepModel* RiBrepModelPtr = nullptr;
	if (A3DRiBrepModelCreate(&RiBRepModelData, &RiBrepModelPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return RiBrepModelPtr;
}

A3DAsmPartDefinition* CreateAsmPartDefinition(const A3DAsmPartDefinitionData& PartDefinitionData)
{
	A3DAsmPartDefinition* AsmPartDefinitionPtr = nullptr;
	if (A3DAsmPartDefinitionCreate(&PartDefinitionData, &AsmPartDefinitionPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return AsmPartDefinitionPtr;
}

A3DAsmProductOccurrence* CreateAsmProductOccurrence(const A3DAsmProductOccurrenceData& ProductOccurrenceData)
{
	A3DAsmProductOccurrence* ProductOccurrencePtr = nullptr;
	if (A3DAsmProductOccurrenceCreate(&ProductOccurrenceData, &ProductOccurrencePtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return ProductOccurrencePtr;
}

A3DMiscAttribute* CreateMiscAttribute(const A3DMiscAttributeData& AttributeData)
{
	A3DMiscAttribute* AttributePtr = nullptr;
	if (A3DMiscAttributeCreate(&AttributeData, &AttributePtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return AttributePtr;
}

A3DAsmModelFile* CreateModelFile(const A3DAsmModelFileData& ModelFileData)
{
	A3DAsmModelFile* ModelFilePtr = nullptr;
	if (A3DAsmModelFileCreate(&ModelFileData, &ModelFilePtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return ModelFilePtr;
}

A3DTopoBrepData* CreateTopoBRep(const A3DTopoBrepDataData& TopoBRepDataData)
{
	A3DTopoBrepData* TopoBRepDataPtr = nullptr;
	if (A3DTopoBrepDataCreate(&TopoBRepDataData, &TopoBRepDataPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return TopoBRepDataPtr;
}

A3DTopoCoEdge* CreateTopoCoEdge(const A3DTopoCoEdgeData& TopoCoEdgeData)
{
	A3DTopoCoEdge* TopoCoEdgePtr = nullptr;
	if (A3DTopoCoEdgeCreate(&TopoCoEdgeData, &TopoCoEdgePtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return TopoCoEdgePtr;
}

A3DTopoConnex* CreateTopoConnex(const A3DTopoConnexData& TopoConnexData)
{
	A3DTopoConnex* TopoConnexPtr = nullptr;
	if (A3DTopoConnexCreate(&TopoConnexData, &TopoConnexPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return TopoConnexPtr;
}

A3DTopoEdge* CreateTopoEdge(const A3DTopoEdgeData& TopoEdgeData)
{
	A3DTopoEdge* TopoEdgePtr = nullptr;
	if (A3DTopoEdgeCreate(&TopoEdgeData, &TopoEdgePtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return TopoEdgePtr;
}

A3DTopoFace* CreateTopoFace(const A3DTopoFaceData& TopoFaceData)
{
	A3DTopoFace* TopoFacePtr = nullptr;
	if (A3DTopoFaceCreate(&TopoFaceData, &TopoFacePtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return TopoFacePtr;
}

A3DTopoLoop* CreateTopoLoop(const A3DTopoLoopData& TopoLoopData)
{
	A3DTopoLoop* TopoLoopPtr = nullptr;
	if (A3DTopoLoopCreate(&TopoLoopData, &TopoLoopPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return TopoLoopPtr;
}

A3DCrvNurbs* CreateTopoShell(const A3DTopoShellData& TopoShellData)
{
	A3DTopoShell* TopoShellPtr = nullptr;
	if (A3DTopoShellCreate(&TopoShellData, &TopoShellPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return TopoShellPtr;
}

A3DCrvNurbs* CreateCurveTransform(const A3DCrvTransformData& CurveTransformData)
{
	A3DCrvNurbs* CurveNurbsPtr = nullptr;
	if (A3DCrvTransformCreate(&CurveTransformData, &CurveNurbsPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return CurveNurbsPtr;
}

A3DCrvNurbs* CreateCurveNurbs(const A3DCrvNurbsData& CurveNurbsData)
{
	A3DCrvNurbs* CurveNurbsPtr = nullptr;
	if (A3DCrvNurbsCreate(&CurveNurbsData, &CurveNurbsPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return CurveNurbsPtr;
}

A3DSurfNurbs* CreateSurfaceCylinder(const A3DSurfCylinderData& SurfaceCylinderData)
{
	A3DSurfCylinder* SurfCylinderPtr = nullptr;
	if (A3DSurfCylinderCreate(&SurfaceCylinderData, &SurfCylinderPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return SurfCylinderPtr;
}

A3DSurfNurbs* CreateSurfaceNurbs(const A3DSurfNurbsData& SurfaceNurbsData)
{
	A3DSurfNurbs* SurfaceNurbsPtr = nullptr;
	if (A3DSurfNurbsCreate(&SurfaceNurbsData, &SurfaceNurbsPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return SurfaceNurbsPtr;
}

A3DGraphics* CreateGraphics(const A3DGraphicsData& GraphicsData)
{
	A3DGraphics* GraphicsPtr = nullptr;
	if (A3DGraphicsCreate(&GraphicsData, &GraphicsPtr) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}
	return GraphicsPtr;
}

A3DStatus LinkCoEdges(A3DTopoCoEdge* CoEdgePtr, A3DTopoCoEdge* NeighbourCoEdgePtr)
{
	return A3DTopoCoEdgeSetNeighbour(CoEdgePtr, NeighbourCoEdgePtr);
}

A3DStatus SetRootBase(A3DEntity* EntityPtr, const A3DRootBaseData& RootBaseData)
{
	return A3DRootBaseSet(EntityPtr, &RootBaseData);
}

A3DStatus SetRootBaseWithGraphics(const A3DRootBaseWithGraphicsData& RootBaseWithGraphicsData, A3DRootBaseWithGraphics* RootPtr)
{
	return A3DRootBaseWithGraphicsSet(RootPtr, &RootBaseWithGraphicsData);
}

A3DUns32 InsertGraphMaterial(const A3DGraphMaterialData& InMaterialData)
{
	A3DUns32 OutMaterialGenericIndex = A3D_DEFAULT_MATERIAL_INDEX;
	if (A3DGlobalInsertGraphMaterial(&InMaterialData, &OutMaterialGenericIndex) != A3DStatus::A3D_SUCCESS)
	{
		return A3D_DEFAULT_MATERIAL_INDEX;
	}
	return OutMaterialGenericIndex;
}

A3DUns32 InsertGraphRgbColor(const A3DGraphRgbColorData& InRgbColorData)
{
	A3DUns32 OutIndexRgbColor = A3D_DEFAULT_COLOR_INDEX;
	if (A3DGlobalInsertGraphRgbColor(&InRgbColorData, &OutIndexRgbColor) != A3DStatus::A3D_SUCCESS)
	{
		return A3D_DEFAULT_COLOR_INDEX;
	}
	return OutIndexRgbColor;
}

A3DUns32 InsertGraphStyle(const A3DGraphStyleData& InStyleData)
{
	A3DUns32 OutStyleIndex = A3D_DEFAULT_STYLE_INDEX;
	if (A3DGlobalInsertGraphStyle(&InStyleData, &OutStyleIndex) != A3DStatus::A3D_SUCCESS)
	{
		return A3D_DEFAULT_STYLE_INDEX;
	}
	return OutStyleIndex;
}

A3DStatus ExportModelFileToPrcFile(const A3DAsmModelFile* ModelFile, const A3DRWParamsExportPrcData* ParamsExportData, const A3DUTF8Char* CADFileName, A3DRWParamsPrcWriteHelper** PrcWriteHelper)
{
	return A3DAsmModelFileExportToPrcFile(ModelFile, ParamsExportData, CADFileName, PrcWriteHelper);
}

double GetModelFileUnit(const A3DAsmModelFile* ModelFile)
{
	double FileUnit = 0.1;
	try
	{
		if (A3DAsmModelFileGetUnit(ModelFile, &FileUnit) != A3DStatus::A3D_SUCCESS)
		{
			return 0.1;
		}
	}
	catch (...)
	{
			return 0.1;
	}
	return FileUnit * 0.1;
}

A3DStatus AddAttribute(A3DEntity* EntityPtr, const TCHAR* Title, const TCHAR* Value)
{
	return A3DRootBaseAttributeAdd(EntityPtr, TCHAR_TO_UTF8(Title), TCHAR_TO_UTF8(Value));
}

A3DStatus SewModel(A3DAsmModelFile* ModelPtr, double ToleranceInCM, A3DSewOptionsData const* SewOptions)
{
	const double ToleranceInMM = ToleranceInCM * 10.; // cm => mm
	return A3DAsmModelFileSew(&ModelPtr, ToleranceInMM, SewOptions);
}

A3DStatus SewBReps(A3DRiBrepModel** BRepsToSew, uint32 const BRepCount, double ToleranceInCM, double FileUnit, A3DSewOptionsData const* SewOptions, A3DRiBrepModel*** OutNewBReps, uint32& OutNewBRepCount)
{
	const double CmToMm = 10.;
	const double ToleranceInFileUnit = ToleranceInCM * CmToMm / FileUnit;

	A3DUns32 NewBRepCount = 0;
	A3DStatus Status = A3DSewBrep(&BRepsToSew, BRepCount, ToleranceInFileUnit, SewOptions, OutNewBReps, &NewBRepCount);
	OutNewBRepCount = NewBRepCount;
	return Status;
}

#endif
} // NS TechSoftInterface


#ifdef USE_TECHSOFT_SDK

// TUniqueTSObj InitializeData -----------------------------------
template<>
void TUniqueTSObj<A3DAsmModelFileData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DAsmModelFileData, Data);
}
template<>
void TUniqueTSObj<A3DAsmPartDefinitionData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DAsmPartDefinitionData, Data);
}
template<>
void TUniqueTSObj<A3DAsmProductOccurrenceData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DAsmProductOccurrenceData, Data);
}
template<>
void TUniqueTSObj<A3DAsmProductOccurrenceDataCV5>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DAsmProductOccurrenceDataCV5, Data);
}
template<>
void TUniqueTSObj<A3DAsmProductOccurrenceDataSLW>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DAsmProductOccurrenceDataSLW, Data);
}
template<>
void TUniqueTSObj<A3DAsmProductOccurrenceDataNX>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DAsmProductOccurrenceDataNX, Data);
}
template<>
void TUniqueTSObj<A3DBoundingBoxData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DBoundingBoxData, Data);
}
template<>
void TUniqueTSObj<A3DCopyAndAdaptBrepModelData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCopyAndAdaptBrepModelData, Data);
}
template<>
void TUniqueTSObj<A3DCrvCircleData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvCircleData, Data);
}
template<>
void TUniqueTSObj<A3DCrvCompositeData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvCompositeData, Data);
}
template<>
void TUniqueTSObj<A3DCrvEllipseData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvEllipseData, Data);
}
template<>
void TUniqueTSObj<A3DCrvHelixData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvHelixData, Data);
}
template<>
void TUniqueTSObj<A3DCrvHyperbolaData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvHyperbolaData, Data);
}
template<>
void TUniqueTSObj<A3DCrvLineData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvLineData, Data);
}
template<>
void TUniqueTSObj<A3DCrvNurbsData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvNurbsData, Data);
}
template<>
void TUniqueTSObj<A3DCrvParabolaData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvParabolaData, Data);
}
template<>
void TUniqueTSObj<A3DCrvPolyLineData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvPolyLineData, Data);
}
template<>
void TUniqueTSObj<A3DDomainData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DDomainData, Data);
}
template<>
void TUniqueTSObj<A3DCrvTransformData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DCrvTransformData, Data);
}
template<>
void TUniqueTSObj<A3DGlobalData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DGlobalData, Data);
}
template<>
void TUniqueTSObj<A3DGraphicsData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DGraphicsData, Data);
}
template<>
void TUniqueTSObj<A3DIntervalData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DIntervalData, Data);
}
template<>
void TUniqueTSObj<A3DMiscAttributeData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscAttributeData, Data);
}
template<>
void TUniqueTSObj<A3DMiscCartesianTransformationData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscCartesianTransformationData, Data);
}
template<>
void TUniqueTSObj<A3DMiscEntityReferenceData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscEntityReferenceData, Data);
}
template<>
void TUniqueTSObj<A3DMiscGeneralTransformationData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscGeneralTransformationData, Data);
}
template<>
void TUniqueTSObj<A3DMiscMaterialPropertiesData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscMaterialPropertiesData, Data);
}
template<>
void TUniqueTSObj<A3DMiscReferenceOnCsysItemData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscReferenceOnCsysItemData, Data);
}
template<>
void TUniqueTSObj<A3DMiscReferenceOnTessData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscReferenceOnTessData, Data);
}
template<>
void TUniqueTSObj<A3DMiscReferenceOnTopologyData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscReferenceOnTopologyData, Data);
}
template<>
void TUniqueTSObj<A3DMiscSingleAttributeData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DMiscSingleAttributeData, Data);
}
template<>
void TUniqueTSObj<A3DRWParamsExportPrcData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRWParamsExportPrcData, Data);
}
template<>
void TUniqueTSObj<A3DRiBrepModelData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRiBrepModelData, Data);
}
template<>
void TUniqueTSObj<A3DRiCoordinateSystemData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRiCoordinateSystemData, Data);
}
template<>
void TUniqueTSObj<A3DRiDirectionData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRiDirectionData, Data);
}
template<>
void TUniqueTSObj<A3DRiPolyBrepModelData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRiPolyBrepModelData, Data);
}
template<>
void TUniqueTSObj<A3DRiRepresentationItemData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRiRepresentationItemData, Data);
}
template<>
void TUniqueTSObj<A3DRiSetData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRiSetData, Data);
}
template<>
void TUniqueTSObj<A3DRootBaseData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRootBaseData, Data);
}
template<>
void TUniqueTSObj<A3DRootBaseWithGraphicsData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRootBaseWithGraphicsData, Data);
}
template<>
void TUniqueTSObj<A3DRWParamsTessellationData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DRWParamsTessellationData, Data);
}
template<>
void TUniqueTSObj<A3DSewOptionsData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSewOptionsData, Data);
}
template<>
void TUniqueTSObj<A3DSurfBlend01Data>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfBlend01Data, Data);
}
template<>
void TUniqueTSObj<A3DSurfBlend02Data>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfBlend02Data, Data);
}
template<>
void TUniqueTSObj<A3DSurfBlend03Data>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfBlend03Data, Data);
}
template<>
void TUniqueTSObj<A3DSurfConeData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfConeData, Data);
}
template<>
void TUniqueTSObj<A3DSurfCylinderData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfCylinderData, Data);
}
template<>
void TUniqueTSObj<A3DSurfCylindricalData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfCylindricalData, Data);
}
template<>
void TUniqueTSObj<A3DSurfExtrusionData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfExtrusionData, Data);
}
template<>
void TUniqueTSObj<A3DSurfFromCurvesData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfFromCurvesData, Data);
}
template<>
void TUniqueTSObj<A3DSurfNurbsData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfNurbsData, Data);
}
template<>
void TUniqueTSObj<A3DSurfPipeData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfPipeData, Data);
}
template<>
void TUniqueTSObj<A3DSurfPlaneData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfPlaneData, Data);
}
template<>
void TUniqueTSObj<A3DSurfRevolutionData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfRevolutionData, Data);
}
template<>
void TUniqueTSObj<A3DSurfRuledData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfRuledData, Data);
}
template<>
void TUniqueTSObj<A3DSurfSphereData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfSphereData, Data);
}
template<>
void TUniqueTSObj<A3DSurfTorusData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DSurfTorusData, Data);
}
template<>
void TUniqueTSObj<A3DTess3DData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTess3DData, Data);
}
template<>
void TUniqueTSObj<A3DTessBaseData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTessBaseData, Data);
}
template<>
void TUniqueTSObj<A3DTopoBodyData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoBodyData, Data);
}
template<>
void TUniqueTSObj<A3DTopoBrepDataData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoBrepDataData, Data);
}
template<>
void TUniqueTSObj<A3DTopoCoEdgeData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoCoEdgeData, Data);
}
template<>
void TUniqueTSObj<A3DTopoConnexData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoConnexData, Data);
}
template<>
void TUniqueTSObj<A3DTopoContextData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoContextData, Data);
}
template<>
void TUniqueTSObj<A3DTopoEdgeData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoEdgeData, Data);
}
template<>
void TUniqueTSObj<A3DTopoFaceData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoFaceData, Data);
}
template<>
void TUniqueTSObj<A3DTopoLoopData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoLoopData, Data);
}
template<>
void TUniqueTSObj<A3DTopoShellData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoShellData, Data);
}
template<>
void TUniqueTSObj<A3DTopoUniqueVertexData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoUniqueVertexData, Data);
}
template<>
void TUniqueTSObj<A3DTopoWireEdgeData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DTopoWireEdgeData, Data);
}
template<>
void TUniqueTSObj<A3DUTF8Char*>::InitializeData()
{
	Data = nullptr;
}
template<>
void TUniqueTSObj<A3DVector2dData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DVector2dData, Data);
}
template<>
void TUniqueTSObj<A3DVector3dData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DVector3dData, Data);
}


// TUniqueTSObjFromIndex InitializeData -----------------------------------
template<>
void TUniqueTSObjFromIndex<A3DGraphMaterialData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DGraphMaterialData, Data);
}
template<>
void TUniqueTSObjFromIndex<A3DGraphPictureData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DGraphPictureData, Data);
}
template<>
void TUniqueTSObjFromIndex<A3DGraphRgbColorData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DGraphRgbColorData, Data);
}
template<>
void TUniqueTSObjFromIndex<A3DGraphStyleData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DGraphStyleData, Data);
}
template<>
void TUniqueTSObjFromIndex<A3DGraphTextureApplicationData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DGraphTextureApplicationData, Data);
}
template<>
void TUniqueTSObjFromIndex<A3DGraphTextureDefinitionData>::InitializeData()
{
	A3D_INITIALIZE_DATA(A3DGraphTextureDefinitionData, Data);
}



// TUniqueTSObj GetData -----------------------------------

template<>
A3DStatus TUniqueTSObj<A3DAsmModelFileData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DAsmModelFileGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DAsmPartDefinitionData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DAsmPartDefinitionGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DAsmProductOccurrenceData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DAsmProductOccurrenceGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DAsmProductOccurrenceDataCV5>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DAsmProductOccurrenceGetCV5(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DAsmProductOccurrenceDataSLW>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DAsmProductOccurrenceGetSLW(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DAsmProductOccurrenceDataNX>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DAsmProductOccurrenceGetNX(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DBoundingBoxData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscGetBoundingBox(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCopyAndAdaptBrepModelData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}

template<>
A3DStatus TUniqueTSObj<A3DCrvCircleData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvCircleGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCrvCompositeData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvCompositeGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCrvEllipseData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvEllipseGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCrvHelixData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvHelixGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCrvHyperbolaData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvHyperbolaGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCrvLineData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvLineGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCrvNurbsData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvNurbsGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCrvParabolaData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvParabolaGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DCrvPolyLineData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvPolyLineGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DDomainData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}

template<>
A3DStatus TUniqueTSObj<A3DCrvTransformData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvTransformGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DGlobalData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DGlobalGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DGraphicsData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DGraphicsGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DIntervalData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DCrvGetInterval(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscAttributeData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscAttributeGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscCartesianTransformationData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscCartesianTransformationGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscEntityReferenceData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscEntityReferenceGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscGeneralTransformationData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscGeneralTransformationGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscMaterialPropertiesData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscGetMaterialProperties(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscReferenceOnCsysItemData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscReferenceOnCsysItemGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscReferenceOnTessData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscReferenceOnTessGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscReferenceOnTopologyData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DMiscReferenceOnTopologyGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DMiscSingleAttributeData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}

template<>
A3DStatus TUniqueTSObj<A3DRWParamsExportPrcData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}

template<>
A3DStatus TUniqueTSObj<A3DRiBrepModelData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DRiBrepModelGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DRiCoordinateSystemData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DRiCoordinateSystemGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DRiDirectionData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DRiDirectionGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DRiPolyBrepModelData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DRiPolyBrepModelGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DRiRepresentationItemData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DRiRepresentationItemGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DRiSetData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DRiSetGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DRootBaseData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DRootBaseGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DRootBaseWithGraphicsData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DRootBaseWithGraphicsGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DRWParamsTessellationData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}

template<>
A3DStatus TUniqueTSObj<A3DSewOptionsData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}

template<>
A3DStatus TUniqueTSObj<A3DSurfBlend01Data>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfBlend01Get(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfBlend02Data>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfBlend02Get(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfBlend03Data>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfBlend03Get(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfConeData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfConeGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfCylinderData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfCylinderGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfCylindricalData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfCylindricalGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfExtrusionData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfExtrusionGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfFromCurvesData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfFromCurvesGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfNurbsData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfNurbsGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfPipeData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfPipeGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfPlaneData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfPlaneGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfRevolutionData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfRevolutionGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfRuledData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfRuledGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfSphereData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfSphereGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DSurfTorusData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DSurfTorusGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTess3DData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTess3DGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTessBaseData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTessBaseGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoBodyData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoBodyGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoBrepDataData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoBrepDataGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoCoEdgeData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoCoEdgeGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoConnexData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoConnexGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoContextData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoContextGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoEdgeData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoEdgeGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoFaceData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoFaceGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoLoopData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoLoopGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoShellData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoShellGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoUniqueVertexData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoUniqueVertexGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DTopoWireEdgeData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DTopoWireEdgeGet(InEntityPtr, &Data);
}

template<>
A3DStatus TUniqueTSObj<A3DUTF8Char*>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}

template<>
A3DStatus TUniqueTSObj<A3DVector2dData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}

template<>
A3DStatus TUniqueTSObj<A3DVector3dData>::GetData(const A3DEntity* InEntityPtr)
{
	return A3DStatus::A3D_ERROR;
}


// TUniqueTSObjFromIndex GetData -----------------------------------
template<>
A3DStatus TUniqueTSObjFromIndex<A3DGraphMaterialData>::GetData(const uint32 InEntityIndex)
{
	return A3DGlobalGetGraphMaterialData(InEntityIndex, &Data);
}
template<>
A3DStatus TUniqueTSObjFromIndex<A3DGraphPictureData>::GetData(const uint32 InEntityIndex)
{
	return A3DGlobalGetGraphPictureData(InEntityIndex, &Data);
}
template<>
A3DStatus TUniqueTSObjFromIndex<A3DGraphRgbColorData>::GetData(const uint32 InEntityIndex)
{
	return A3DGlobalGetGraphRgbColorData(InEntityIndex, &Data);
}
template<>
A3DStatus TUniqueTSObjFromIndex<A3DGraphStyleData>::GetData(const uint32 InEntityIndex)
{
	return A3DGlobalGetGraphStyleData(InEntityIndex, &Data);
}
template<>
A3DStatus TUniqueTSObjFromIndex<A3DGraphTextureApplicationData>::GetData(const uint32 InEntityIndex)
{
	return A3DGlobalGetGraphTextureApplicationData(InEntityIndex, &Data);
}
template<>
A3DStatus TUniqueTSObjFromIndex<A3DGraphTextureDefinitionData>::GetData(const uint32 InEntityIndex)
{
	return A3DGlobalGetGraphTextureDefinitionData(InEntityIndex, &Data);
}

// DefaultValue -----------------------------------
template<>
const A3DEntity* TUniqueTSObj<A3DAsmModelFileData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DAsmPartDefinitionData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DAsmProductOccurrenceData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DAsmProductOccurrenceDataCV5>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DAsmProductOccurrenceDataSLW>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DAsmProductOccurrenceDataNX>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DBoundingBoxData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCopyAndAdaptBrepModelData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvCircleData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvCompositeData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvEllipseData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvHelixData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvHyperbolaData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvLineData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvNurbsData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvParabolaData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvPolyLineData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DDomainData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DCrvTransformData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DGlobalData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DGraphicsData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DIntervalData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscAttributeData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscCartesianTransformationData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscEntityReferenceData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscGeneralTransformationData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscMaterialPropertiesData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscReferenceOnCsysItemData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscReferenceOnTessData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscReferenceOnTopologyData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DMiscSingleAttributeData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRWParamsExportPrcData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRiBrepModelData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRiCoordinateSystemData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRiDirectionData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRiPolyBrepModelData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRiRepresentationItemData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRiSetData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRootBaseData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRootBaseWithGraphicsData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DRWParamsTessellationData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSewOptionsData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfBlend01Data>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfBlend02Data>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfBlend03Data>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfConeData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfCylinderData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfCylindricalData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfExtrusionData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfFromCurvesData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfNurbsData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfPipeData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfPlaneData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfRevolutionData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfRuledData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfSphereData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DSurfTorusData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTess3DData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTessBaseData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoBodyData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoBrepDataData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoCoEdgeData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoConnexData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoContextData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoEdgeData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoFaceData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoLoopData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoShellData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoUniqueVertexData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DTopoWireEdgeData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DVector2dData>::GetDefaultIndexerValue() const { return nullptr; }
template<>
const A3DEntity* TUniqueTSObj<A3DVector3dData>::GetDefaultIndexerValue() const { return nullptr; }

template<>
uint32 TUniqueTSObjFromIndex<A3DGraphMaterialData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_MATERIAL_INDEX; }
template<>
uint32 TUniqueTSObjFromIndex<A3DGraphPictureData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_PICTURE_INDEX; }
template<>
uint32 TUniqueTSObjFromIndex<A3DGraphRgbColorData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_COLOR_INDEX; }
template<>
uint32 TUniqueTSObjFromIndex<A3DGraphStyleData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_STYLE_INDEX; }
template<>
uint32 TUniqueTSObjFromIndex<A3DGraphTextureApplicationData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_TEXTURE_APPLICATION_INDEX; }
template<>
uint32 TUniqueTSObjFromIndex<A3DGraphTextureDefinitionData>::GetDefaultIndexerValue() const { return A3D_DEFAULT_TEXTURE_DEFINITION_INDEX; }
template<>
const A3DEntity* TUniqueTSObj<A3DUTF8Char*>::GetDefaultIndexerValue() const { return nullptr; }

#endif

} // NS CADLibrary

