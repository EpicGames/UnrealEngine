// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CADData.h"
#include "CADSceneGraph.h"

#include "TUniqueTechSoftObj.h"
#include "TechSoftInterface.h"

#ifdef USE_TECHSOFT_SDK

namespace TechSoftInterfaceUtils
{

class FTechSoftTessellationExtractor
{
public:
	FTechSoftTessellationExtractor(const A3DRiRepresentationItem* RepresentationItemPtr, const double InBodyUnit)
		: BodyUnit(InBodyUnit)
		, TextureUnit(InBodyUnit)
	{
		CADLibrary::TUniqueTSObj<A3DRiRepresentationItemData> RepresentationItemData(RepresentationItemPtr);
		if (!RepresentationItemData.IsValid())
		{
			return;
		}

		A3DEEntityType Type;
		A3DEntityGetType(RepresentationItemData->m_pTessBase, &Type);
		if (Type != kA3DTypeTess3D)
		{
			return;
		}

		TessellationPtr = RepresentationItemData->m_pTessBase;

		A3DEntityGetType(RepresentationItemPtr, &Type);
		if (Type == kA3DTypeRiBrepModel)
		{
			CADLibrary::TUniqueTSObj<A3DRiBrepModelData> BRepModelData(RepresentationItemPtr);
			if (BRepModelData.IsValid())
			{
				A3DBrepData = BRepModelData->m_pBrepData;
			}
		}
	}

	bool FillBodyMesh(CADLibrary::FBodyMesh& BodyMesh)
	{
		if (TessellationPtr == nullptr)
		{
			return false;
		}

		int32 VertexOffset = BodyMesh.VertexArray.Num();
		FillVertexArray(BodyMesh.VertexArray);

		if (BodyMesh.VertexArray.Num() == VertexOffset)
		{
			return false;
		}

		BodyFaces.Empty();
		GetBRepFaces();
		FillFaceArray(BodyMesh, VertexOffset);

		return BodyMesh.Faces.Num() > 0 ? true : false;
	}

private:
	void FillVertexArray(TArray<FVector3f>& VertexArray)
	{
		using namespace CADLibrary;

		CADLibrary::TUniqueTSObj<A3DTessBaseData> TessellationBaseData(TessellationPtr);

		if (!TessellationBaseData.IsValid() || TessellationBaseData->m_uiCoordSize == 0)
		{
			return;
		}

		int32 VertexCount = TessellationBaseData->m_uiCoordSize / 3;
		VertexArray.Reserve(VertexArray.Num() + VertexCount);

		const double ScaleFactor = BodyUnit * FImportParameters::GUnitScale;
		double* Coordinates = TessellationBaseData->m_pdCoords;
		for (unsigned int Index = 0; Index < TessellationBaseData->m_uiCoordSize; ++Index)
		{
			Coordinates[Index] *= ScaleFactor;
		}

		for (unsigned int Index = 0; Index < TessellationBaseData->m_uiCoordSize; Index += 3)
		{
			VertexArray.Emplace(Coordinates[Index], Coordinates[Index + 1], Coordinates[Index + 2]);
		}
	}

	// #ueent_techsoft: TODO: Make it more in line with the actual implementation of FillFaceArray
	uint32 CountTriangles(const A3DTessFaceData& FaceTessData)
	{
		const int32 TessellationFaceDataWithTriangle = 0x2222;
		const int32 TessellationFaceDataWithFan = 0x4444;
		const int32 TessellationFaceDataWithStripe = 0x8888;
		const int32 TessellationFaceDataWithOneNormal = 0xE0E0;

		uint32 UsedEntitiesFlags = FaceTessData.m_usUsedEntitiesFlags;

		uint32 TriangleCount = 0;
		uint32 FaceSetIndex = 0;
		if (UsedEntitiesFlags & TessellationFaceDataWithTriangle)
		{
			TriangleCount += FaceTessData.m_puiSizesTriangulated[FaceSetIndex];
			FaceSetIndex++;
		}

		if (FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex)
		{
			if (UsedEntitiesFlags & TessellationFaceDataWithFan)
			{
				uint32 LastFanIndex = 1 + FaceSetIndex + FaceTessData.m_puiSizesTriangulated[FaceSetIndex];
				FaceSetIndex++;
				for (; FaceSetIndex < LastFanIndex; FaceSetIndex++)
				{
					uint32 FanSize = (FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalMask);
					TriangleCount += (FanSize - 2);
				}
			}
		}

		if (FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex)
		{
			FaceSetIndex++;
			for (; FaceSetIndex < FaceTessData.m_uiSizesTriangulatedSize; FaceSetIndex++)
			{
				uint32 StripeSize = (FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalMask);
				TriangleCount += (StripeSize - 2);
			}
		}
		return TriangleCount;
	}

	void FillFaceArray(CADLibrary::FBodyMesh& BodyMesh, int32 VertexOffset = 0)
	{
		using namespace CADLibrary;

		TArray<FTessellationData>& Faces = BodyMesh.Faces;

		CADLibrary::TUniqueTSObj<A3DTess3DData> A3DTessellationData(TessellationPtr);

		if (!A3DTessellationData.IsValid() || A3DTessellationData->m_uiFaceTessSize == 0)
		{
			return;
		}

		TessellationNormals = A3DTessellationData->m_pdNormals;
		TessellationTexCoords = A3DTessellationData->m_pdTextureCoords;
		TriangulatedIndexes = A3DTessellationData->m_puiTriangulatedIndexes;

		if (A3DTessellationData->m_uiFaceTessSize != BodyFaces.Num())
		{
			BodyFaces.Empty();
			A3DBrepData = nullptr;
		}

		for (unsigned int Index = 0; Index < A3DTessellationData->m_uiFaceTessSize; ++Index)
		{
			const A3DTessFaceData& FaceTessData = A3DTessellationData->m_psFaceTessData[Index];

			uint32 FaceSetIndex = 0;
			bool bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			// don't create empty TessellationData
			if (!bMustProcess)
			{
				continue;
			}

			FTessellationData& Tessellation = Faces.Emplace_GetRef();

			// there is a bijection between A3DTess3DData->m_psFaceTessData and A3DTopoShellData->m_ppFaces
			Tessellation.PatchId = Index;

			Tessellation.MaterialUId = 0;
			if (FaceTessData.m_uiStyleIndexesSize != 0)
			{
				// Store the StyleIndex on the MaterialName. It will be processed after tessellation
				Tessellation.MaterialUId = FaceTessData.m_puiStyleIndexes[0];
			}

			// Pre-allocate memory for triangles' data
			uint32 TriangleCount = CountTriangles(FaceTessData);
			Tessellation.PositionIndices.Reserve(3 * TriangleCount);
			Tessellation.VertexIndices.Reserve(3 * TriangleCount);
			Tessellation.NormalArray.Reserve(3 * TriangleCount);
			if (FaceTessData.m_uiTextureCoordIndexesSize > 0)
			{
				Tessellation.TexCoordArray.Reserve(3 * TriangleCount);
			}

			uint32 UsedEntitiesFlags = FaceTessData.m_usUsedEntitiesFlags;

			uint32 LastTrianguleIndex = FaceTessData.m_uiStartTriangulated;
			LastVertexIndex = 0;

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangle)
			{
				AddFaceTriangle(Tessellation, FaceTessData.m_puiSizesTriangulated[FaceSetIndex++], LastTrianguleIndex, LastVertexIndex);
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleOneNormal)
			{
				AddFaceTriangleWithUniqueNormal(Tessellation, FaceTessData.m_puiSizesTriangulated[FaceSetIndex++], LastTrianguleIndex, LastVertexIndex);
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleTextured)
			{
				AddFaceTriangleWithTexture(Tessellation, FaceTessData.m_puiSizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize, LastTrianguleIndex, LastVertexIndex);
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleOneNormalTextured)
			{
				AddFaceTriangleWithUniqueNormalAndTexture(Tessellation, FaceTessData.m_puiSizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize, LastTrianguleIndex, LastVertexIndex);
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleFan)
			{
				uint32 FanCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
				for (uint32 FanIndex = 0; FanIndex < FanCount; ++FanIndex)
				{
					uint32 VertexCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
					AddFaceTriangleFan(Tessellation, VertexCount, LastTrianguleIndex, LastVertexIndex);
				}
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleFanOneNormal)
			{
				uint32 FanCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
				for (uint32 FanIndex = 0; FanIndex < FanCount; ++FanIndex)
				{
					ensure((FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalSingle) != 0);

					uint32 VertexCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
					AddFaceTriangleFanWithUniqueNormal(Tessellation, VertexCount, LastTrianguleIndex, LastVertexIndex);
				}
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleFanTextured)
			{
				uint32 FanCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
				for (uint32 FanIndex = 0; FanIndex < FanCount; ++FanIndex)
				{
					uint32 VertexCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
					AddFaceTriangleFanWithTexture(Tessellation, VertexCount, FaceTessData.m_uiTextureCoordIndexesSize, LastTrianguleIndex, LastVertexIndex);
				}
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleFanOneNormalTextured)
			{
				uint32 FanCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
				for (uint32 FanIndex = 0; FanIndex < FanCount; ++FanIndex)
				{
					ensure((FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalSingle) != 0);

					uint32 VertexCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
					AddFaceTriangleFanWithUniqueNormalAndTexture(Tessellation, VertexCount, FaceTessData.m_uiTextureCoordIndexesSize, LastTrianguleIndex, LastVertexIndex);
				}
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleStripe)
			{
				A3DUns32 StripeSize = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
				for (A3DUns32 StripeIndex = 0; StripeIndex < StripeSize; ++StripeIndex)
				{
					A3DUns32 PointCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
					AddFaceTriangleStripe(Tessellation, PointCount, LastTrianguleIndex, LastVertexIndex);
				}
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeOneNormal)
			{
				A3DUns32 StripeSize = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
				for (A3DUns32 StripeIndex = 0; StripeIndex < StripeSize; ++StripeIndex)
				{
					bool bIsOneNormal = (FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalSingle) != 0;
					A3DUns32 PointCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;

					// Is there only one normal for the entire stripe?
					if (bIsOneNormal == false)
					{
						AddFaceTriangleStripe(Tessellation, PointCount, LastTrianguleIndex, LastVertexIndex);
						continue;
					}

					AddFaceTriangleStripeWithUniqueNormal(Tessellation, PointCount, LastTrianguleIndex, LastVertexIndex);
				}
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeTextured)
			{
				A3DUns32 StripeSize = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
				for (A3DUns32 StripeIndex = 0; StripeIndex < StripeSize; ++StripeIndex)
				{
					A3DUns32 PointCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++];
					AddFaceTriangleStripeWithTexture(Tessellation, PointCount, FaceTessData.m_uiTextureCoordIndexesSize, LastTrianguleIndex, LastVertexIndex);
				}
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (bMustProcess && UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeOneNormalTextured)
			{
				A3DUns32 StripeSize = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
				for (A3DUns32 StripeIndex = 0; StripeIndex < StripeSize; ++StripeIndex)
				{
					A3DUns32 PointCount = FaceTessData.m_puiSizesTriangulated[FaceSetIndex++] & kA3DTessFaceDataNormalMask;
					AddFaceTriangleStripeWithUniqueNormalAndTexture(Tessellation, PointCount, FaceTessData.m_uiTextureCoordIndexesSize, LastTrianguleIndex, LastVertexIndex);
				}
				bMustProcess = FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex;
			}

			if (VertexOffset > 0)
			{
				for (int32& PositionIndex : Tessellation.PositionIndices)
				{
					PositionIndex += VertexOffset;
				}
			}

			ensure(!bMustProcess);

			if (!BodyFaces.IsEmpty())
			{
				const A3DTopoFace* TopoFace = BodyFaces[Index];
				ScaleUV(TopoFace, Tessellation.TexCoordArray);
			}
			BodyMesh.TriangleCount += (Tessellation.VertexIndices.Num() / 3.);
		}
	}

	typedef double A3DDouble;
	bool AddFace(int32 FaceIndex[3], CADLibrary::FTessellationData& Tessellation, int32& InOutVertexIndex)
	{
		if (FaceIndex[0] == FaceIndex[1] || FaceIndex[0] == FaceIndex[2] || FaceIndex[1] == FaceIndex[2])
		{
			return false;
		}

		for (int32 Index = 0; Index < 3; ++Index)
		{
			Tessellation.VertexIndices.Add(InOutVertexIndex++);
		}
		Tessellation.PositionIndices.Append(FaceIndex, 3);
		return true;
	};

	void AddNormals(const A3DDouble* Normals, const int32 Indices[3], TArray<FVector3f>& NormalsArray)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			int32 NormalIndex = Indices[Index];
			NormalsArray.Emplace(Normals[NormalIndex], Normals[NormalIndex + 1], Normals[NormalIndex + 2]);
		}
	};

	void AddTextureCoordinates(const A3DDouble* TextureCoords, const int32 Indices[3], TArray<FVector2f>& TessellationTextures)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			int32 TextureIndex = Indices[Index];
			TessellationTextures.Emplace(TextureCoords[TextureIndex], TextureCoords[TextureIndex + 1]);
		}
	};

	void AddFaceTriangle(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, uint32& InOutStartIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		// Get Triangles
		for (uint32 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[0] = TriangulatedIndexes[InOutStartIndex++];
			FaceIndex[0] = TriangulatedIndexes[InOutStartIndex++] / 3;
			NormalIndex[1] = TriangulatedIndexes[InOutStartIndex++];
			FaceIndex[1] = TriangulatedIndexes[InOutStartIndex++] / 3;
			NormalIndex[2] = TriangulatedIndexes[InOutStartIndex++];
			FaceIndex[2] = TriangulatedIndexes[InOutStartIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}
		}
	}

	void AddFaceTriangleWithUniqueNormal(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, uint32& InOutLastTriangleIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		// Get Triangles
		for (uint32 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
			NormalIndex[1] = NormalIndex[0];
			NormalIndex[2] = NormalIndex[0];

			FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;
			FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (!AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				continue;
			}

			AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
		}
	}

	void AddFaceTriangleWithUniqueNormalAndTexture(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, const uint32 TextureCount, uint32& InOutStartIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		// Get Triangles
		for (uint32 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[0] = TriangulatedIndexes[InOutStartIndex++];
			NormalIndex[1] = NormalIndex[0];
			NormalIndex[2] = NormalIndex[0];

			TextureIndex[0] = TriangulatedIndexes[InOutStartIndex];
			InOutStartIndex += TextureCount;
			FaceIndex[0] = TriangulatedIndexes[InOutStartIndex++] / 3;
			TextureIndex[1] = TriangulatedIndexes[InOutStartIndex];
			InOutStartIndex += TextureCount;
			FaceIndex[1] = TriangulatedIndexes[InOutStartIndex++] / 3;
			TextureIndex[2] = TriangulatedIndexes[InOutStartIndex];
			InOutStartIndex += TextureCount;
			FaceIndex[2] = TriangulatedIndexes[InOutStartIndex++] / 3;

			if (!AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				continue;
			}

			AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			AddTextureCoordinates(TessellationTexCoords, TextureIndex, Tessellation.TexCoordArray);
		}
	}

	void AddFaceTriangleWithTexture(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, const uint32 InTextureCount, uint32& InOutStartIndex, int32& inOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		// Get Triangles
		for (uint64 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[0] = TriangulatedIndexes[InOutStartIndex++];
			TextureIndex[0] = TriangulatedIndexes[InOutStartIndex];
			InOutStartIndex += InTextureCount;
			FaceIndex[0] = TriangulatedIndexes[InOutStartIndex++] / 3;
			NormalIndex[1] = TriangulatedIndexes[InOutStartIndex++];
			TextureIndex[1] = TriangulatedIndexes[InOutStartIndex];
			InOutStartIndex += InTextureCount;
			FaceIndex[1] = TriangulatedIndexes[InOutStartIndex++] / 3;
			NormalIndex[2] = TriangulatedIndexes[InOutStartIndex++];
			TextureIndex[2] = TriangulatedIndexes[InOutStartIndex];
			InOutStartIndex += InTextureCount;
			FaceIndex[2] = TriangulatedIndexes[InOutStartIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, inOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(TessellationTexCoords, TextureIndex, Tessellation.TexCoordArray);
			}
		}
	}

	void AddFaceTriangleFan(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, uint32& InOutLastTriangleIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
		FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;
		NormalIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++];
		FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		for (unsigned long TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++];
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}

			NormalIndex[1] = NormalIndex[2];
			FaceIndex[1] = FaceIndex[2];
		}
	}

	void AddFaceTriangleFanWithUniqueNormal(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, uint32& InOutLastTriangleIndex, int32& /*LastVertexIndex*/)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
		NormalIndex[1] = NormalIndex[0];
		NormalIndex[2] = NormalIndex[0];

		FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;
		FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		// Get Triangles
		for (uint32 TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}

			FaceIndex[1] = FaceIndex[2];
		}
	}

	void AddFaceTriangleFanWithUniqueNormalAndTexture(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, const uint32 TextureCount, uint32& InOutLastTriangleIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
		NormalIndex[1] = NormalIndex[0];
		NormalIndex[2] = NormalIndex[0];

		TextureIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex];
		InOutLastTriangleIndex += TextureCount;
		FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		TextureIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex];
		InOutLastTriangleIndex += TextureCount;
		FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		for (uint32 TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			TextureIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex];
			InOutLastTriangleIndex += TextureCount;
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(TessellationTexCoords, TextureIndex, Tessellation.TexCoordArray);
			}

			FaceIndex[1] = FaceIndex[2];
			TextureIndex[1] = TextureIndex[2];
		}
	}

	void AddFaceTriangleFanWithTexture(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, const uint32 TextureCount, uint32& InOutLastTriangleIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
		TextureIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex];
		InOutLastTriangleIndex += TextureCount;
		FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		NormalIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++];
		TextureIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex];
		InOutLastTriangleIndex += TextureCount;
		FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		for (uint32 TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++];
			TextureIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex];
			InOutLastTriangleIndex += TextureCount;
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(TessellationTexCoords, TextureIndex, Tessellation.TexCoordArray);
			}

			NormalIndex[1] = NormalIndex[2];
			TextureIndex[1] = TextureIndex[2];
			FaceIndex[1] = FaceIndex[2];
		}
	}

	void AddFaceTriangleStripe(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, uint32& InOutLastTriangleIndex, int32& /*LastVertexIndex*/)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
		FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;
		NormalIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++];
		FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		for (unsigned long TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++];
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}

			TriangleIndex++;
			if (TriangleIndex == InTriangleCount)
			{
				break;
			}

			Swap(FaceIndex[1], FaceIndex[2]);
			Swap(NormalIndex[1], NormalIndex[2]);

			NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
			FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}

			Swap(FaceIndex[0], FaceIndex[1]);
			Swap(NormalIndex[0], NormalIndex[1]);
		}
	}

	void AddFaceTriangleStripeWithTexture(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, const uint32 TextureCount, uint32& InOutLastTriangleIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
		TextureIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex];
		InOutLastTriangleIndex += TextureCount;
		FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;
		NormalIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++];
		TextureIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex];
		InOutLastTriangleIndex += TextureCount;
		FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		for (unsigned long TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++];
			TextureIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex];
			InOutLastTriangleIndex += TextureCount;
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(TessellationTexCoords, TextureIndex, Tessellation.TexCoordArray);
			}

			TriangleIndex++;
			if (TriangleIndex == InTriangleCount)
			{
				break;
			}

			Swap(FaceIndex[1], FaceIndex[2]);
			Swap(NormalIndex[1], NormalIndex[2]);
			Swap(TextureIndex[1], TextureIndex[2]);

			NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
			FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}

			Swap(FaceIndex[0], FaceIndex[1]);
			Swap(NormalIndex[0], NormalIndex[1]);
			Swap(TextureIndex[0], TextureIndex[1]);
		}
	}

	void AddFaceTriangleStripeWithUniqueNormal(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, uint32& InOutLastTriangleIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
		NormalIndex[1] = NormalIndex[0];
		NormalIndex[2] = NormalIndex[0];

		FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;
		FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		for (unsigned long TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}

			TriangleIndex++;
			if (TriangleIndex == InTriangleCount)
			{
				break;
			}

			Swap(FaceIndex[1], FaceIndex[2]);

			FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}

			Swap(FaceIndex[0], FaceIndex[1]);
		}
	}

	void AddFaceTriangleStripeWithUniqueNormalAndTexture(CADLibrary::FTessellationData& Tessellation, const uint32 InTriangleCount, const uint32 TextureCount, uint32& InOutLastTriangleIndex, int32& InOutLastVertexIndex)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++];
		NormalIndex[1] = NormalIndex[0];
		NormalIndex[2] = NormalIndex[0];

		TextureIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex];
		InOutLastTriangleIndex += TextureCount;
		FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		TextureIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex];
		InOutLastTriangleIndex += TextureCount;
		FaceIndex[1] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

		for (unsigned long TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			TextureIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex];
			InOutLastTriangleIndex += TextureCount;
			FaceIndex[2] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(TessellationTexCoords, TextureIndex, Tessellation.TexCoordArray);
			}

			TriangleIndex++;
			if (TriangleIndex == InTriangleCount)
			{
				break;
			}

			Swap(FaceIndex[1], FaceIndex[2]);
			Swap(TextureIndex[1], TextureIndex[2]);

			TextureIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex];
			InOutLastTriangleIndex += TextureCount;
			FaceIndex[0] = TriangulatedIndexes[InOutLastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, InOutLastVertexIndex))
			{
				AddNormals(TessellationNormals, NormalIndex, Tessellation.NormalArray);
			}

			Swap(FaceIndex[0], FaceIndex[1]);
			Swap(TextureIndex[0], TextureIndex[1]);
		}
	}

	void ScaleUV(const A3DTopoFace* TopoFace, TArray<FVector2f>& TexCoordArray)
	{
		CADLibrary::TUniqueTSObj<A3DTopoFaceData> TopoFaceData(TopoFace);
		if (!TopoFaceData.IsValid())
		{
			return;
		}

		CADLibrary::TUniqueTSObj<A3DDomainData> Domain;
		if (TopoFaceData->m_bHasTrimDomain)
		{
			*Domain = TopoFaceData->m_sSurfaceDomain;
		}
		else
		{
			CADLibrary::TechSoftInterface::GetSurfaceDomain(TopoFaceData->m_pSurface, *Domain);
		}

		const int32 IsoCurveCount = 7;
		const double DeltaU = (Domain->m_sMax.m_dX - Domain->m_sMin.m_dX) / (IsoCurveCount - 1.);
		const double DeltaV = (Domain->m_sMax.m_dY - Domain->m_sMin.m_dY) / (IsoCurveCount - 1.);

		const A3DSurfBase* A3DSurface = TopoFaceData->m_pSurface;

		FVector NodeMatrix[IsoCurveCount * IsoCurveCount];

		CADLibrary::TUniqueTSObj<A3DVector3dData> Point3D;
		CADLibrary::TUniqueTSObj<A3DVector2dData> CoordinateObj;
		A3DVector2dData& Coordinate = *CoordinateObj;
		Coordinate.m_dX = Domain->m_sMin.m_dX;
		Coordinate.m_dY = Domain->m_sMin.m_dY;

		for (int32 IndexI = 0; IndexI < IsoCurveCount; IndexI++)
		{
			for (int32 IndexJ = 0; IndexJ < IsoCurveCount; IndexJ++)
			{
				CADLibrary::TechSoftInterface::Evaluate(A3DSurface, Coordinate, 0, Point3D.GetPtr());
				NodeMatrix[IndexI * IsoCurveCount + IndexJ].X = Point3D->m_dX;
				NodeMatrix[IndexI * IsoCurveCount + IndexJ].Y = Point3D->m_dY;
				NodeMatrix[IndexI * IsoCurveCount + IndexJ].Z = Point3D->m_dZ;
				Coordinate.m_dY += DeltaV;
			}
			Coordinate.m_dX += DeltaU;
			Coordinate.m_dY = Domain->m_sMin.m_dY;
		}

		// Compute length of 7 iso V line
		double LengthU[IsoCurveCount];
		double LengthUMax = 0;
		double LengthUMed = 0;

		for (int32 IndexJ = 0; IndexJ < IsoCurveCount; IndexJ++)
		{
			LengthU[IndexJ] = 0;
			for (int32 IndexI = 0; IndexI < (IsoCurveCount - 1); IndexI++)
			{
				LengthU[IndexJ] += FVector::Distance(NodeMatrix[IndexI * IsoCurveCount + IndexJ], NodeMatrix[(IndexI + 1) * IsoCurveCount + IndexJ]);
			}
			LengthUMed += LengthU[IndexJ];
			LengthUMax = FMath::Max(LengthU[IndexJ], LengthUMax);
		}
		LengthUMed /= IsoCurveCount;
		LengthUMed = LengthUMed * 2 / 3 + LengthUMax / 3;

		// Compute length of 7 iso U line
		double LengthV[IsoCurveCount];
		double LengthVMax = 0;
		double LengthVMed = 0;

		for (int32 IndexI = 0; IndexI < IsoCurveCount; IndexI++)
		{
			LengthV[IndexI] = 0;
			for (int32 IndexJ = 0; IndexJ < (IsoCurveCount - 1); IndexJ++)
			{
				LengthV[IndexI] += FVector::Distance(NodeMatrix[IndexI * IsoCurveCount + IndexJ], NodeMatrix[IndexI * IsoCurveCount + IndexJ + 1]);
			}
			LengthVMed += LengthV[IndexI];
			LengthVMax = FMath::Max(LengthV[IndexI], LengthVMax);
		}
		LengthVMed /= IsoCurveCount;
		LengthVMed = LengthVMed * 2 / 3 + LengthVMax / 3;

		// Texture unit is meter, Coord unit from TechSoft is mm, so TextureScale = 0.001 to convert mm into m
		const double TextureScale = 0.01;
		const float UScale = TextureUnit * TextureScale * LengthUMed / (Domain->m_sMax.m_dX - Domain->m_sMin.m_dX);
		const float VScale = TextureUnit * TextureScale * LengthVMed / (Domain->m_sMax.m_dY - Domain->m_sMin.m_dY);

		for (FVector2f& TexCoord : TexCoordArray)
		{
			TexCoord[0] *= UScale;
			TexCoord[1] *= VScale;
		}
	}

	void GetBRepFaces()
	{
		if (A3DBrepData == nullptr)
		{
			return;
		}

		CADLibrary::TUniqueTSObj<A3DTopoBodyData> TopoBodyData(A3DBrepData);
		if (TopoBodyData.IsValid())
		{
			if (TopoBodyData->m_pContext)
			{
				CADLibrary::TUniqueTSObj<A3DTopoContextData> TopoContextData(TopoBodyData->m_pContext);
				if (TopoContextData.IsValid())
				{
					if (TopoContextData->m_bHaveScale)
					{
						TextureUnit *= TopoContextData->m_dScale;
					}
				}
			}
		}

		CADLibrary::TUniqueTSObj<A3DTopoBrepDataData> TopoBrepData(A3DBrepData);
		if (TopoBrepData.IsValid())
		{
			for (A3DUns32 Index = 0; Index < TopoBrepData->m_uiConnexSize; ++Index)
			{
				CADLibrary::TUniqueTSObj<A3DTopoConnexData> TopoConnexData(TopoBrepData->m_ppConnexes[Index]);
				if (TopoConnexData.IsValid())
				{
					for (A3DUns32 Sndex = 0; Sndex < TopoConnexData->m_uiShellSize; ++Sndex)
					{
						CADLibrary::TUniqueTSObj<A3DTopoShellData> ShellData(TopoConnexData->m_ppShells[Sndex]);
						if (ShellData.IsValid())
						{
							for (A3DUns32 Fndex = 0; Fndex < ShellData->m_uiFaceSize; ++Fndex)
							{
								const A3DTopoFace* A3DFace = ShellData->m_ppFaces[Fndex];
								BodyFaces.Add(A3DFace);
							}
						}
					}
				}
			}
		}
	}

private:
	const A3DTess3D* TessellationPtr = nullptr;
	const A3DTopoBrepData* A3DBrepData = nullptr;
	const double BodyUnit = 1;
	double TextureUnit = 1;

	TArray<const A3DTopoFace*> BodyFaces;

	int32 LastVertexIndex = 0;
	A3DUns32* TriangulatedIndexes = nullptr;
	A3DDouble* TessellationNormals;
	A3DDouble* TessellationTexCoords;				/*!< Array of \ref A3DDouble, as texture coordinates. */
};

} // ns TechSoftUtils

#endif