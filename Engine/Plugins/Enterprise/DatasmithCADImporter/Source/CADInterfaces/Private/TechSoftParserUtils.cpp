// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechSoftParserUtils.h"

#ifdef USE_TECHSOFT_SDK

namespace TechSoftParserUtils
{
	bool FTechSoftMeshConverter::UpdateBodyMesh()
	{
		using namespace CADLibrary;

		if (!Tessellation3DData.IsValid() || !TessellationBaseData.IsValid() || TessellationBaseData->m_uiCoordSize == 0)
		{
			return false;
		}

		// Vertex's positions are global to the BodYMesh. Normals and UVs are not.
		{
			const int32 VertexCount = TessellationBaseData->m_uiCoordSize / 3;
			BodyMesh.VertexArray.Reserve(VertexCount);

			double* Coordinates = TessellationBaseData->m_pdCoords;
			for (unsigned int Index = 0; Index < TessellationBaseData->m_uiCoordSize; ++Index)
			{
				Coordinates[Index] *= ScaleFactor;
			}

			for (unsigned int Index = 0; Index < TessellationBaseData->m_uiCoordSize; Index += 3)
			{
				BodyMesh.VertexArray.Emplace(Coordinates[Index], Coordinates[Index + 1], Coordinates[Index + 2]);
			}
		}

		BodyMesh.Faces.Reserve(BodyMesh.Faces.Num() + Tessellation3DData->m_uiFaceTessSize);
		for (unsigned int Index = 0; Index < Tessellation3DData->m_uiFaceTessSize; ++Index)
		{
			const A3DTessFaceData& FaceTessData = Tessellation3DData->m_psFaceTessData[Index];
			if (FaceTessData.m_uiSizesTriangulatedSize > 0)
			{
				ConvertFace(FaceTessData, BodyMesh.Faces.Emplace_GetRef());
			}
		}
		
		return true;
	}

	void FTechSoftMeshConverter::ConvertFace(const A3DTessFaceData& FaceTessData, CADLibrary::FTessellationData& Tessellation)
	{
		uint32 TriangleCount = CountTriangles(FaceTessData);
		Reserve(Tessellation, TriangleCount, /*bWithTexture*/ FaceTessData.m_uiTextureCoordIndexesSize > 0);

		uint32 UsedEntitiesFlags = FaceTessData.m_usUsedEntitiesFlags;
		const A3DUns32* SizesTriangulated = FaceTessData.m_puiSizesTriangulated;

		uint32 FaceSetIndex = 0;

		LastTriangleIndex = FaceTessData.m_uiStartTriangulated;
		LastVertexIndex = 0;

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangle)
		{
			AddFaceTriangle(Tessellation, SizesTriangulated[FaceSetIndex++]);
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleFan)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 FanCount = SizesTriangulated[FaceSetIndex++];
			for (uint32 Index = 0; Index < FanCount; ++Index)
			{
				ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
				AddFaceTriangleFan(Tessellation, SizesTriangulated[FaceSetIndex++]);
			}
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleStripe)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 StripeCount = SizesTriangulated[FaceSetIndex++];
			for (uint32 Index = 0; Index < StripeCount; ++Index)
			{
				ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
				AddFaceTriangleStripe(Tessellation, SizesTriangulated[FaceSetIndex++]);
			}
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleOneNormal)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			AddFaceTriangleWithOneNormal(Tessellation, SizesTriangulated[FaceSetIndex++]);
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleFanOneNormal)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 FanCount = SizesTriangulated[FaceSetIndex++];
			for (uint32 Index = 0; Index < FanCount; ++Index)
			{
				ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
				AddFaceTriangleFanWithOneNormal(Tessellation, SizesTriangulated[FaceSetIndex++]);
			}
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeOneNormal)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 StripeCount = SizesTriangulated[FaceSetIndex++];
			for (uint32 Index = 0; Index < StripeCount; ++Index)
			{
				ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
				AddFaceTriangleStripeWithOneNormal(Tessellation, SizesTriangulated[FaceSetIndex++]);
			}
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleTextured)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			AddFaceTriangleWithTexture(Tessellation, SizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize);
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleFanTextured)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 FanCount = SizesTriangulated[FaceSetIndex++];
			for (uint32 Index = 0; Index < FanCount; ++Index)
			{
				ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
				AddFaceTriangleFanWithTexture(Tessellation, SizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize);
			}
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeTextured)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 StripeCount = SizesTriangulated[FaceSetIndex++];
			for (uint32 Index = 0; Index < StripeCount; ++Index)
			{
				ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
				AddFaceTriangleStripeWithTexture(Tessellation, SizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize);
			}
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleOneNormalTextured)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			AddFaceTriangleWithOneNormalAndTexture(Tessellation, SizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize);
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleFanOneNormalTextured)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 FanCount = SizesTriangulated[FaceSetIndex++];
			for (uint32 Index = 0; Index < FanCount; ++Index)
			{
				ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
				AddFaceTriangleFanWithOneNormalAndTexture(Tessellation, SizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize);
			}
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleStripeOneNormalTextured)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 StripeCount = SizesTriangulated[FaceSetIndex++];
			for (uint32 Index = 0; Index < StripeCount; ++Index)
			{
				ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
				AddFaceTriangleStripeWithOneNormalAndTexture(Tessellation, SizesTriangulated[FaceSetIndex++], FaceTessData.m_uiTextureCoordIndexesSize);
			}
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangle(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		// Get Triangles
		for (uint32 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++];
			FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;
			NormalIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++];
			FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;
			NormalIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++];
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleFan(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;
		NormalIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

		for (unsigned long TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}

			NormalIndex[1] = NormalIndex[2];
			FaceIndex[1] = FaceIndex[2];
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleStripe(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;
		NormalIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

		for (unsigned long TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}

			TriangleIndex++;
			if (TriangleIndex == InTriangleCount)
			{
				break;
			}

			Swap(FaceIndex[1], FaceIndex[2]);
			Swap(NormalIndex[1], NormalIndex[2]);

			NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
			FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}

			Swap(FaceIndex[0], FaceIndex[1]);
			Swap(NormalIndex[0], NormalIndex[1]);
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleWithOneNormal(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		// Get Triangles
		for (uint32 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
			NormalIndex[1] = NormalIndex[0];
			NormalIndex[2] = NormalIndex[0];

			FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;
			FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

			if (!AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				continue;
			}

			AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleFanWithOneNormal(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		NormalIndex[1] = NormalIndex[0];
		NormalIndex[2] = NormalIndex[0];

		FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

		// Get Triangles
		for (uint32 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleStripeWithOneNormal(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		NormalIndex[1] = NormalIndex[0];
		NormalIndex[2] = NormalIndex[0];

		FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;
		FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

		for (unsigned long TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}

			TriangleIndex++;
			if (TriangleIndex == InTriangleCount)
			{
				break;
			}

			Swap(FaceIndex[1], FaceIndex[2]);

			NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
			FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}

			Swap(FaceIndex[0], FaceIndex[1]);
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleWithTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 InTextureCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		// Get Triangles
		for (uint64 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++];
			TextureIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex];
			LastVertexIndex += InTextureCount;
			FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;
			NormalIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++];
			TextureIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex];
			LastVertexIndex += InTextureCount;
			FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;
			NormalIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++];
			TextureIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex];
			LastVertexIndex += InTextureCount;
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(Tessellation3DData->m_pdTextureCoords, TextureIndex, Tessellation.TexCoordArray);
			}
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleFanWithTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 TextureCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		TextureIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
		LastTriangleIndex += TextureCount;
		FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

		NormalIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		TextureIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
		LastTriangleIndex += TextureCount;
		FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

		for (uint32 TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
			TextureIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
			LastTriangleIndex += TextureCount;
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(Tessellation3DData->m_pdTextureCoords, TextureIndex, Tessellation.TexCoordArray);
			}
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleStripeWithTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 TextureCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		TextureIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
		LastTriangleIndex += TextureCount;
		FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		NormalIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		TextureIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
		LastTriangleIndex += TextureCount;
		FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

		for (unsigned long TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
			TextureIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
			LastTriangleIndex += TextureCount;
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(Tessellation3DData->m_pdTextureCoords, TextureIndex, Tessellation.TexCoordArray);
			}

			TriangleIndex++;
			if (TriangleIndex == InTriangleCount)
			{
				break;
			}

			Swap(FaceIndex[1], FaceIndex[2]);
			Swap(NormalIndex[1], NormalIndex[2]);
			Swap(TextureIndex[1], TextureIndex[2]);

			NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
			FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}

			Swap(FaceIndex[0], FaceIndex[1]);
			Swap(NormalIndex[0], NormalIndex[1]);
			Swap(TextureIndex[0], TextureIndex[1]);
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleWithOneNormalAndTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 TextureCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		// Get Triangles
		for (uint32 TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++];
			NormalIndex[1] = NormalIndex[0];
			NormalIndex[2] = NormalIndex[0];

			TextureIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex];
			LastVertexIndex += TextureCount;
			FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;
			TextureIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex];
			LastVertexIndex += TextureCount;
			FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;
			TextureIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex];
			LastVertexIndex += TextureCount;
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastVertexIndex++] / 3;

			if (!AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				continue;
			}

			AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			AddTextureCoordinates(Tessellation3DData->m_pdTextureCoords, TextureIndex, Tessellation.TexCoordArray);
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleFanWithOneNormalAndTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 TextureCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		NormalIndex[1] = NormalIndex[0];
		NormalIndex[2] = NormalIndex[0];

		TextureIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
		LastTriangleIndex += TextureCount;
		FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

		TextureIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
		LastTriangleIndex += TextureCount;
		FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

		for (uint32 TriangleIndex = 2; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			TextureIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
			LastTriangleIndex += TextureCount;
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(Tessellation3DData->m_pdTextureCoords, TextureIndex, Tessellation.TexCoordArray);
			}
		}
	}

	void FTechSoftMeshConverter::AddFaceTriangleStripeWithOneNormalAndTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 TextureCount)
	{
		int32 FaceIndex[3] = { 0, 0, 0 };
		int32 NormalIndex[3] = { 0, 0, 0 };
		int32 TextureIndex[3] = { 0, 0, 0 };

		NormalIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];
		NormalIndex[1] = NormalIndex[0];
		NormalIndex[2] = NormalIndex[0];

		TextureIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
		LastTriangleIndex += TextureCount;
		FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

		TextureIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
		LastTriangleIndex += TextureCount;
		FaceIndex[1] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

		for (unsigned long TriangleIndex = 0; TriangleIndex < InTriangleCount; TriangleIndex++)
		{
			TextureIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex];
			LastTriangleIndex += TextureCount;
			FaceIndex[2] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++];

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
				AddTextureCoordinates(Tessellation3DData->m_pdTextureCoords, TextureIndex, Tessellation.TexCoordArray);
			}

			TriangleIndex++;
			if (TriangleIndex == InTriangleCount)
			{
				break;
			}

			Swap(FaceIndex[1], FaceIndex[2]);
			Swap(TextureIndex[1], TextureIndex[2]);

			FaceIndex[0] = Tessellation3DData->m_puiTriangulatedIndexes[LastTriangleIndex++] / 3;

			if (AddFace(FaceIndex, Tessellation, LastVertexIndex))
			{
				AddNormals(Tessellation3DData->m_pdNormals, NormalIndex, Tessellation.NormalArray);
			}

			Swap(FaceIndex[0], FaceIndex[1]);
			Swap(TextureIndex[0], TextureIndex[1]);
		}
	}

	uint32 CountTriangles(const A3DTessFaceData& FaceTessData)
	{
		// To revisit
		//const int32 TessellationFaceDataWithTriangle = 0x2222;
		//const int32 TessellationFaceDataWithFan = 0x4444;
		//const int32 TessellationFaceDataWithStrip = 0x8888;
		//const int32 TessellationFaceDataWithOneNormal = 0xE0E0;

		uint32 UsedEntitiesFlags = FaceTessData.m_usUsedEntitiesFlags;
		const A3DUns32* SizesTriangulated = FaceTessData.m_puiSizesTriangulated;

		uint32 TriangleCount = 0;
		uint32 FaceSetIndex = 0;
		if (UsedEntitiesFlags & kA3DTessFaceDataTriangle)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			TriangleCount += SizesTriangulated[FaceSetIndex++];
		}

		if (UsedEntitiesFlags & kA3DTessFaceDataTriangleFan)
		{
			ensure(FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex);
			uint32 LastFanIndex = 1 + FaceSetIndex;
			LastFanIndex += SizesTriangulated[FaceSetIndex++];
			for (; FaceSetIndex < LastFanIndex; FaceSetIndex++)
			{
				uint32 FanSize = (FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalMask);
				TriangleCount += (FanSize - 2);
			}
		}

		if (FaceTessData.m_uiSizesTriangulatedSize > FaceSetIndex)
		{
			FaceSetIndex++;
			for (; FaceSetIndex < FaceTessData.m_uiSizesTriangulatedSize; FaceSetIndex++)
			{
				uint32 StripSize = (FaceTessData.m_puiSizesTriangulated[FaceSetIndex] & kA3DTessFaceDataNormalMask);
				TriangleCount += (StripSize - 2);
			}
		}

		return TriangleCount;
	}

} // ns TechSoftParserUtils

#endif // USE_TECHSOFT_SDK