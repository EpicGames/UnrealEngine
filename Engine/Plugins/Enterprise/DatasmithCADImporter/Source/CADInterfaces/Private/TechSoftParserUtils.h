// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "TechSoftInterface.h"
#include "TUniqueTechSoftObj.h"

#include "CADData.h"
#include "CADFileData.h"
#include "CADFileParser.h"
#include "CADOptions.h"
#include "CADSceneGraph.h"

typedef void A3DAsmModelFile;
typedef void A3DAsmPartDefinition;
typedef void A3DAsmProductOccurrence;
typedef void A3DEntity;
typedef void A3DGraphics;
typedef void A3DMiscAttribute;
typedef void A3DMiscCartesianTransformation;
typedef void A3DMiscCartesianTransformation;
typedef void A3DMiscGeneralTransformation;
typedef void A3DRiBrepModel;
typedef void A3DRiCoordinateSystem;
typedef void A3DRiPolyBrepModel;
typedef void A3DRiRepresentationItem;
typedef void A3DRiSet;
typedef void A3DTess3D;
typedef void A3DTessBase;
typedef void A3DTopoBrepData;

#ifdef USE_TECHSOFT_SDK

namespace TechSoftParserUtils
{
	typedef double A3DDouble;

	inline bool AddFace(int32 FaceIndex[3], CADLibrary::FTessellationData& Tessellation, int32& InOutVertexIndex)
	{
		if (FaceIndex[0] == FaceIndex[1] || FaceIndex[0] == FaceIndex[2] || FaceIndex[1] == FaceIndex[2])
		{
			return false;
		}

		for (int32 Index = 0; Index < 3; ++Index)
		{
			Tessellation.VertexIndices.Add(InOutVertexIndex++);
		}
		Tessellation.PositionIndices.Append(FaceIndex, 3);
		return true;
	};

	inline void AddNormals(const A3DDouble* Normals, const int32 Indices[3], TArray<FVector3f>& TessellationNormals)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			int32 NormalIndex = Indices[Index];
			TessellationNormals.Emplace(Normals[NormalIndex], Normals[NormalIndex + 1], Normals[NormalIndex + 2]);
		}
	};

	inline void AddTextureCoordinates(const A3DDouble* TextureCoords, const int32 Indices[3], TArray<FVector2f>& TessellationTextures)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			int32 TextureIndex = Indices[Index];
			TessellationTextures.Emplace(TextureCoords[TextureIndex], TextureCoords[TextureIndex + 1]);
		}
	};

	inline void Reserve(CADLibrary::FTessellationData& Tessellation, int32 InTrinangleCount, bool bWithTexture)
	{
		Tessellation.PositionIndices.Reserve(3 * InTrinangleCount);
		Tessellation.VertexIndices.Reserve(3 * InTrinangleCount);
		Tessellation.NormalArray.Reserve(3 * InTrinangleCount);
		if (bWithTexture)
		{
			Tessellation.TexCoordArray.Reserve(3 * InTrinangleCount);
		}
	};

	uint32 CountTriangles(const A3DTessFaceData& FaceTessData);

	class FTechSoftMeshConverter
	{
	public:
		FTechSoftMeshConverter(const A3DTess3D* TessellationPtr, double ScaleFactor, CADLibrary::FBodyMesh& InBodyMesh)
			: BodyMesh(InBodyMesh)
			, TessellationBaseData(TessellationPtr)
			, Tessellation3DData(TessellationPtr)
		{
		}

		bool UpdateBodyMesh();

	private:
		void ConvertFace(const A3DTessFaceData& FaceTessData, CADLibrary::FTessellationData& Tessellation);

		void AddFaceTriangle(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount);

		void AddFaceTriangleFan(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount);

		void AddFaceTriangleStripe(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount);

		void AddFaceTriangleWithOneNormal(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount);

		void AddFaceTriangleFanWithOneNormal(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount);

		void AddFaceTriangleStripeWithOneNormal(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount);

		void AddFaceTriangleWithTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 InTextureCount);

		void AddFaceTriangleFanWithTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 InTextureCount);

		void AddFaceTriangleStripeWithTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 InTextureCount);

		void AddFaceTriangleWithOneNormalAndTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 InTextureCount);

		void AddFaceTriangleFanWithOneNormalAndTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 InTextureCount);

		void AddFaceTriangleStripeWithOneNormalAndTexture(CADLibrary::FTessellationData& Tessellation, A3DUns32 InTriangleCount, A3DUns32 InTextureCount);

	private:
		CADLibrary::FBodyMesh& BodyMesh;

		CADLibrary::TUniqueTSObj<A3DTessBaseData> TessellationBaseData;

		CADLibrary::TUniqueTSObj<A3DTess3DData> Tessellation3DData;

		double ScaleFactor;

		uint32 LastTriangleIndex;

		int32 LastVertexIndex;
	};
} // ns TechSoftParserUtils

#endif
