// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechSoftUtils.h"


#include "TUniqueTechSoftObj.h"

#include "CADOptions.h"

#include "GenericPlatform/GenericPlatformMisc.h"
#include "Math/Color.h"

#ifndef CADKERNEL_DEV
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "TechSoftInterfaceUtils.inl"
#endif

#include <string>

namespace CADLibrary
{

namespace TechSoftUtils
{

#ifdef USE_TECHSOFT_SDK
TSharedPtr<FJsonObject> GetJsonObject(A3DAsmProductOccurrence* ProductOcccurence);
void RestoreMaterials(const TSharedPtr<FJsonObject>& DefaultValues, CADLibrary::FBodyMesh& BodyMesh);
void SaveModelFileToPrcFile(void* ModelFile, const FString& Filename);
A3DUns32 CreateRGBColor(FColor& Color);
void SetRootOccurenceAttributes(A3DEntity* Entity);
#endif

bool GetBodyFromPcrFile(const FString& Filename, const FImportParameters& ImportParameters, FBodyMesh& BodyMesh)
{
#if defined USE_TECHSOFT_SDK && !defined CADKERNEL_DEV
	A3DRWParamsPrcReadHelper* ReadHelper = nullptr;

	FUniqueTechSoftModelFile ModelFile = TechSoftInterface::LoadModelFileFromPrcFile(TCHAR_TO_UTF8(*Filename), &ReadHelper);

	if (!ModelFile.IsValid())
	{
		return false;
	}

	if(ImportParameters.GetStitchingTechnique() != CADLibrary::EStitchingTechnique::StitchingNone)
	{
		CADLibrary::TUniqueTSObj<A3DSewOptionsData> SewData;
		SewData->m_bComputePreferredOpenShellOrientation = false;

		TechSoftInterface::SewModel(ModelFile.Get(), CADLibrary::FImportParameters::GStitchingTolerance, SewData.GetPtr());
	}

	TUniqueTSObj<A3DAsmModelFileData> ModelFileData(ModelFile.Get());
	if (!ModelFileData.IsValid() || ModelFileData->m_uiPOccurrencesSize == 0)
	{
		return false;
	}

	TUniqueTSObj<A3DAsmProductOccurrenceData> OccurenceData(ModelFileData->m_ppPOccurrences[0]);
	if (!OccurenceData.IsValid() || !OccurenceData->m_pPart)
	{
		return false;
	}

	TUniqueTSObj<A3DAsmPartDefinitionData> PartDefinitionData(OccurenceData->m_pPart);
	if (!PartDefinitionData.IsValid() || PartDefinitionData->m_uiRepItemsSize == 0)
	{
		return false;
	}

	TSharedPtr<FJsonObject> JsonObject = GetJsonObject(ModelFileData->m_ppPOccurrences[0]);
	if (JsonObject.IsValid())
	{
		double BodyUnit = 1.0;
		JsonObject->TryGetNumberField(JSON_ENTRY_BODY_UNIT, BodyUnit);

		for (A3DUns32 Index = 0; Index < PartDefinitionData->m_uiRepItemsSize; ++Index)
		{
			if (!FillBodyMesh(PartDefinitionData->m_ppRepItems[Index], ImportParameters, BodyUnit, BodyMesh))
			{
				return false;
			}
		}
		RestoreMaterials(JsonObject, BodyMesh);
		return true;
	}
#endif
	return false;
}

FUniqueTechSoftModelFile SaveBodiesToPrcFile(void** Bodies, uint32 BodyCount, const FString& Filename, const FString& JsonString)
{
#if defined USE_TECHSOFT_SDK && !defined CADKERNEL_DEV
	if (!Bodies)
	{
		return FUniqueTechSoftModelFile();
	}

	// Create PartDefinition
	A3DRiRepresentationItem** RepresentationItems = (A3DRiRepresentationItem**)Bodies;

	TUniqueTSObj<A3DAsmPartDefinitionData> PartDefinitionData;
	PartDefinitionData->m_uiRepItemsSize = BodyCount;
	PartDefinitionData->m_ppRepItems = RepresentationItems;
	A3DAsmPartDefinition* PartDefinition = TechSoftInterface::CreateAsmPartDefinition(*PartDefinitionData);

	TUniqueTSObj<A3DAsmProductOccurrenceData> ProductOccurrenceData;
	ProductOccurrenceData->m_pPart = PartDefinition;
	A3DAsmProductOccurrence* ProductOccurrence = TechSoftInterface::CreateAsmProductOccurrence(*ProductOccurrenceData);

	// Add MaterialTable as attribute to ProductOccurrence
	std::string StringAnsi(TCHAR_TO_UTF8(*JsonString));

	TUniqueTSObj<A3DMiscSingleAttributeData> SingleAttributeData;
	SingleAttributeData->m_eType = kA3DModellerAttributeTypeString;
	SingleAttributeData->m_pcTitle = (char*)"MaterialTable";
	SingleAttributeData->m_pcData = (char*)StringAnsi.c_str();

	TUniqueTSObj<A3DMiscAttributeData> AttributesData;
	AttributesData->m_pcTitle = SingleAttributeData->m_pcTitle;
	AttributesData->m_asSingleAttributesData = SingleAttributeData.GetPtr();
	AttributesData->m_uiSize = 1;
	A3DMiscAttribute* Attributes = TechSoftInterface::CreateMiscAttribute(*AttributesData);

	TUniqueTSObj<A3DRootBaseData> RootBaseData;
	RootBaseData->m_pcName = SingleAttributeData->m_pcTitle;
	RootBaseData->m_ppAttributes = &Attributes;
	RootBaseData->m_uiSize = 1;
	TechSoftInterface::SetRootBase(ProductOccurrence, *RootBaseData);

	// Create ModelFile
	TUniqueTSObj<A3DAsmModelFileData> ModelFileData;
	ModelFileData->m_uiPOccurrencesSize = 1;
	ModelFileData->m_dUnit = 1.0;
	ModelFileData->m_ppPOccurrences = &ProductOccurrence;
	FUniqueTechSoftModelFile ModelFile = TechSoftInterface::CreateAsmModelFile(*ModelFileData);

	// Save ModelFile to Pcr file
	SaveModelFileToPrcFile(ModelFile.Get(), Filename);

	// #ueent_techsoft: Deleting the model seems to delete the entire content. To be double-checked
	TechSoftInterface::DeleteEntity(Attributes);

	return ModelFile;
#else
	return FUniqueTechSoftModelFile();
#endif
}

bool FillBodyMesh(void* BodyPtr, const FImportParameters& ImportParameters, double BodyUnit, FBodyMesh& BodyMesh)
{
#if defined USE_TECHSOFT_SDK && !defined CADKERNEL_DEV
	A3DRiRepresentationItem* RepresentationItemPtr = (A3DRiRepresentationItem*)BodyPtr;

	A3DEEntityType Type;
	A3DEntityGetType(RepresentationItemPtr, &Type);
	if (Type == kA3DTypeRiPolyBrepModel)
	{
		TechSoftInterfaceUtils::FTechSoftTessellationExtractor Extractor(RepresentationItemPtr, BodyUnit);
		return Extractor.FillBodyMesh(BodyMesh);
	}

	// TUniqueTechSoftObj does not work in this case
	TUniqueTSObj<A3DRWParamsTessellationData> TessellationParameters;

	TessellationParameters->m_eTessellationLevelOfDetail = kA3DTessLODUserDefined; // Enum to specify predefined values for some following members.
	TessellationParameters->m_bUseHeightInsteadOfRatio = A3D_TRUE;
	TessellationParameters->m_dMaxChordHeight = ImportParameters.GetChordTolerance();
	if (!FMath::IsNearlyZero(BodyUnit))
	{
		TessellationParameters->m_dMaxChordHeight /= BodyUnit;
	}

	TessellationParameters->m_dAngleToleranceDeg = ImportParameters.GetMaxNormalAngle();
	TessellationParameters->m_dMaximalTriangleEdgeLength = 0; //ImportParameters.MaxEdgeLength;

	TessellationParameters->m_bAccurateTessellation = A3D_FALSE;  // A3D_FALSE' indicates the tessellation is set for visualization
	TessellationParameters->m_bAccurateTessellationWithGrid = A3D_FALSE; // Enable accurate tessellation with faces inner points on a grid.
	TessellationParameters->m_dAccurateTessellationWithGridMaximumStitchLength = 0; 	// Maximal grid stitch length. Disabled if value is 0. Be careful, a too small value can generate a huge tessellation.

	TessellationParameters->m_bKeepUVPoints = A3D_TRUE; // Keep parametric points as texture points.

	// Get the tessellation
	A3DStatus Status = A3DRiRepresentationItemComputeTessellation(RepresentationItemPtr, TessellationParameters.GetPtr());
	TUniqueTSObj<A3DRiRepresentationItemData> RepresentationItemData(RepresentationItemPtr);
	if (!RepresentationItemData.IsValid())
	{
		return false;
	}

	{
		A3DEntityGetType(RepresentationItemData->m_pTessBase, &Type);
		if (Type != kA3DTypeTess3D)
		{
			return false;
		}
	}

	TechSoftInterfaceUtils::FTechSoftTessellationExtractor Extractor(RepresentationItemPtr, BodyUnit);
	return Extractor.FillBodyMesh(BodyMesh);

#else
	return false;
#endif
}

#if defined USE_TECHSOFT_SDK && !defined CADKERNEL_DEV
TSharedPtr<FJsonObject> GetJsonObject(A3DAsmProductOccurrence* ProductOcccurence)
{
	TUniqueTSObj<A3DRootBaseData> RootBaseData(ProductOcccurence);

	if (RootBaseData.IsValid() && RootBaseData->m_uiSize > 0)
	{
		TUniqueTSObj<A3DMiscAttributeData> AttributeData(RootBaseData->m_ppAttributes[0]);
		if (AttributeData->m_uiSize > 0 && AttributeData->m_asSingleAttributesData[0].m_eType == kA3DModellerAttributeTypeString)
		{
			FString JsonString = UTF8_TO_TCHAR(AttributeData->m_asSingleAttributesData[0].m_pcData);

			TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);
			TSharedPtr<FJsonObject> JsonObject;

			if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
			{
				return JsonObject;
			}
		}
	}

	return {};
}

// Replicate logic in FTechSoftFileParser::FindOrAddMaterial and the methods it calls
void BuildCADMaterial(uint32 MaterialIndex, const A3DGraphStyleData& GraphStyleData, FCADMaterial& Material)
{
	if (TechSoftInterface::IsMaterialTexture(MaterialIndex))
	{
		TUniqueTSObjFromIndex<A3DGraphTextureApplicationData> TextureData(MaterialIndex);
		if (TextureData.IsValid())
		{
			MaterialIndex = TextureData->m_uiMaterialIndex;
		}
	}

	TUniqueTSObjFromIndex<A3DGraphMaterialData> MaterialData(MaterialIndex);
	if (MaterialData.IsValid())
	{
		Material.Diffuse = GetColorAt(MaterialData->m_uiDiffuse);
		Material.Ambient = GetColorAt(MaterialData->m_uiAmbient);
		Material.Specular = GetColorAt(MaterialData->m_uiSpecular);
		Material.Shininess = MaterialData->m_dShininess;
		if (GraphStyleData.m_bIsTransparencyDefined)
		{
			Material.Transparency = 1. - GraphStyleData.m_ucTransparency / 255.;
		}
		// todo: find how to convert Emissive color into ? reflexion coef...
		// Material.Emissive = GetColor(MaterialData->m_uiEmissive);
		// Material.Reflexion;
	}
}
#endif

// Replicates the logic in FTechSoftFileParser::ExtractGraphStyleProperties
void GetMaterialValues(uint32 StyleIndex, FMaterialUId& OutColorName, FMaterialUId& OutMaterialName)
{
#if defined USE_TECHSOFT_SDK && !defined CADKERNEL_DEV
	TUniqueTSObjFromIndex<A3DGraphStyleData> GraphStyleData(StyleIndex);

	if (GraphStyleData.IsValid())
	{
		if (GraphStyleData->m_bMaterial)
		{
			FCADMaterial Material;

			BuildCADMaterial(GraphStyleData->m_uiRgbColorIndex, *GraphStyleData, Material);

			OutMaterialName = BuildMaterialUId(Material);
		}
		else
		{
			TUniqueTSObjFromIndex<A3DGraphRgbColorData> ColorData(GraphStyleData->m_uiRgbColorIndex);
			if (ColorData.IsValid())
			{
				// Alpha == Opacity == (255 - Transparency)
				const uint8 Alpha = GraphStyleData->m_bIsTransparencyDefined ? (255 - GraphStyleData->m_ucTransparency) : 255;
				const FColor ColorValue((uint8)(ColorData->m_dRed * 255), (uint8)(ColorData->m_dGreen * 255), (uint8)(ColorData->m_dBlue * 255), Alpha);

				OutColorName = BuildColorUId(ColorValue);
			}
		}
	}
#endif
}

void RestoreMaterials(const TSharedPtr<FJsonObject>& DefaultValues, FBodyMesh& BodyMesh)
{
	FMaterialUId DefaultColorName = 0;
	FMaterialUId DefaultMaterialName = 0;

#ifndef CADKERNEL_DEV
	DefaultValues->TryGetNumberField(JSON_ENTRY_COLOR_NAME, DefaultColorName);
	DefaultValues->TryGetNumberField(JSON_ENTRY_MATERIAL_NAME, DefaultMaterialName);
#endif

	BodyMesh.MaterialSet.Empty();
	BodyMesh.ColorSet.Empty();

	for (FTessellationData& Tessellation : BodyMesh.Faces)
	{
		// Extract proper color or material based on style index
		uint32 CachedStyleIndex = Tessellation.MaterialUId;
		Tessellation.MaterialUId = 0;

		FMaterialUId ColorUId = DefaultColorName;
		FMaterialUId MaterialUId = DefaultMaterialName;

		GetMaterialValues(CachedStyleIndex, ColorUId, MaterialUId);

		if (ColorUId)
		{
			Tessellation.ColorUId = ColorUId;
			BodyMesh.ColorSet.Add(ColorUId);
		}

		if (MaterialUId)
		{
			Tessellation.MaterialUId = MaterialUId;
			BodyMesh.MaterialSet.Add(MaterialUId);
		}
	}
}

FColor GetColorAt(uint32 ColorIndex)
{
#if defined USE_TECHSOFT_SDK && !defined CADKERNEL_DEV
	TUniqueTSObjFromIndex<A3DGraphRgbColorData> ColorData(ColorIndex);
	if (ColorData.IsValid())
	{
		return FColor((uint8)(ColorData->m_dRed * 255)
			, (uint8)(ColorData->m_dGreen * 255)
			, (uint8)(ColorData->m_dBlue * 255));
	}
#endif
	return FColor(200, 200, 200);
}

void SaveModelFileToPrcFile(void* ModelFile, const FString& Filename)
{
#if defined USE_TECHSOFT_SDK && !defined CADKERNEL_DEV
	TUniqueTSObj<A3DRWParamsExportPrcData> ParamsExportData;
	ParamsExportData->m_bCompressBrep = false;
	ParamsExportData->m_bCompressTessellation = false;

#if PLATFORM_WINDOWS
	A3DUTF8Char HsfFileName[MAX_PATH];
	FCStringAnsi::Strncpy(HsfFileName, TCHAR_TO_UTF8(*Filename), MAX_PATH);
#elif PLATFORM_LINUX
	A3DUTF8Char HsfFileName[PATH_MAX];
	FCStringAnsi::Strncpy(HsfFileName, TCHAR_TO_UTF8(*Filename), PATH_MAX);
#else
#error Platform not supported
#endif // PLATFORM_WINDOWS

	TechSoftInterface::ExportModelFileToPrcFile(ModelFile, ParamsExportData.GetPtr(), HsfFileName, nullptr);
#endif
}

#if defined USE_TECHSOFT_SDK && !defined CADKERNEL_DEV
A3DUns32 CreateRGBColor(FColor& Color)
{
	TUniqueTSObjFromIndex<A3DGraphRgbColorData> RgbColor;

	RgbColor->m_dRed = Color.R / 255.;
	RgbColor->m_dGreen = Color.G / 255.;
	RgbColor->m_dBlue = Color.B / 255.;
	return TechSoftInterface::InsertGraphRgbColor(*RgbColor);
}

int32 SetEntityGraphicsColor(A3DEntity* InEntity, FColor Color)
{
	TUniqueTSObj<A3DRootBaseWithGraphicsData> BaseWithGraphicsData(InEntity);

	//Create a style color
	A3DUns32 ColorIndex = CreateRGBColor(Color);

	A3DUns32 StyleIndex = 0;
	TUniqueTSObjFromIndex<A3DGraphStyleData> StyleData;
	StyleData->m_bMaterial = false;
	StyleData->m_bVPicture = false;
	StyleData->m_dWidth = 0.1; // default
	A3DUns8 Alpha = Color.A;
	if (Alpha < 255)
	{
		StyleData->m_bIsTransparencyDefined = true;
		StyleData->m_ucTransparency = 255 - Alpha;
	}
	else
	{
		StyleData->m_bIsTransparencyDefined = false;
		StyleData->m_ucTransparency = 0;
	}

	StyleData->m_bSpecialCulling = false;
	StyleData->m_bBackCulling = false;
	StyleData->m_uiRgbColorIndex = ColorIndex;
	StyleIndex = TechSoftInterface::InsertGraphStyle(*StyleData);

	TUniqueTSObj<A3DGraphicsData> GraphicsData;

	GraphicsData->m_uiStyleIndex = StyleIndex;
	GraphicsData->m_usBehaviour = kA3DGraphicsShow;
	GraphicsData->m_usBehaviour |= kA3DGraphicsSonHeritColor;

	BaseWithGraphicsData->m_pGraphics = TechSoftInterface::CreateGraphics(*GraphicsData);
	if (BaseWithGraphicsData->m_pGraphics == nullptr)
	{
		return A3DStatus::A3D_ERROR;
	}

	return TechSoftInterface::SetRootBaseWithGraphics(*BaseWithGraphicsData, InEntity);
}

void SetRootOccurenceAttributes(A3DEntity* Entity)
{
	A3DMiscAttribute* Attributes[3];

	TUniqueTSObj<A3DMiscSingleAttributeData> Single;
	Single->m_eType = kA3DModellerAttributeTypeString;

	TUniqueTSObj<A3DMiscAttributeData> AttributeData;
	AttributeData->m_asSingleAttributesData = Single.GetPtr();
	AttributeData->m_uiSize = 1;

	Single->m_pcTitle = (char*)"Title";
	Single->m_pcData = (char*)"Body model";
	AttributeData->m_pcTitle = Single->m_pcTitle;
	Attributes[0] = TechSoftInterface::CreateMiscAttribute(*AttributeData);

	Single->m_pcTitle = (char*)"Author";
	Single->m_pcData = (char*)"Unreal Engine";
	AttributeData->m_pcTitle = Single->m_pcTitle;
	Attributes[1] = TechSoftInterface::CreateMiscAttribute(*AttributeData);

	Single->m_pcTitle = (char*)"Company";
	Single->m_pcData = (char*)"Epic Games";
	AttributeData->m_pcTitle = Single->m_pcTitle;
	Attributes[2] = TechSoftInterface::CreateMiscAttribute(*AttributeData);

	TUniqueTSObj<A3DRootBaseData> RootData;
	RootData->m_pcName = (char*)"Body model";
	RootData->m_ppAttributes = Attributes;
	RootData->m_uiSize = 3;
	TechSoftInterface::SetRootBase(Entity, *RootData);

	for (A3DUns32 i = 0; i < RootData->m_uiSize; ++i)
	{
		TechSoftInterface::DeleteEntity(Attributes[i]);
	}
}

A3DAsmModelFile* CreateModelFile(A3DAsmProductOccurrence* Occurrence)
{
	if (Occurrence != NULL)
	{
		return nullptr;
	}

	CADLibrary::TUniqueTSObj<A3DAsmModelFileData> ModelFileData;

	// In this model there will be only one product occurrence
	ModelFileData->m_uiPOccurrencesSize = 1;
	ModelFileData->m_dUnit = 1.0;
	ModelFileData->m_ppPOccurrences = &Occurrence;

	return TechSoftInterface::CreateModelFile(*ModelFileData);
}

A3DAsmProductOccurrence* CreateOccurrence(A3DAsmPartDefinition* Part)
{
	if (Part == nullptr)
	{
		return nullptr;
	}
	TUniqueTSObj<A3DAsmProductOccurrenceData> ProductOccurrenceData;

	ProductOccurrenceData->m_pPart = Part;
	A3DAsmProductOccurrence* ProductOccurrencePtr = TechSoftInterface::CreateAsmProductOccurrence(*ProductOccurrenceData);
	if (ProductOccurrencePtr != nullptr)
	{
		SetRootOccurenceAttributes(ProductOccurrencePtr);
	}
	return ProductOccurrencePtr;
}

A3DAsmPartDefinition* CreatePart(TArray<A3DRiRepresentationItem*>& RepresentationItems)
{
	if (RepresentationItems.IsEmpty())
	{
		return nullptr;
	}

	TUniqueTSObj<A3DAsmPartDefinitionData> PartDefinitionData;

	PartDefinitionData->m_uiRepItemsSize = RepresentationItems.Num();
	PartDefinitionData->m_ppRepItems = RepresentationItems.GetData();

	return TechSoftInterface::CreateAsmPartDefinition(*PartDefinitionData);
}

A3DRiRepresentationItem* CreateRIBRep(A3DTopoShell* TopoShellPtr)
{
	if (TopoShellPtr == nullptr)
	{
		return nullptr;
	}

	A3DTopoConnex* TopoConnexPtr = nullptr;
	{
		CADLibrary::TUniqueTSObj<A3DTopoConnexData> TopoConnexData;
		TopoConnexData->m_ppShells = &TopoShellPtr;
		TopoConnexData->m_uiShellSize = 1;
		TopoConnexPtr = TechSoftInterface::CreateTopoConnex(*TopoConnexData);
		if (TopoConnexPtr == nullptr)
		{
			return nullptr;
		}
	}

	A3DTopoBrepData* TopoBRepPtr = nullptr;
	{
		CADLibrary::TUniqueTSObj<A3DTopoBrepDataData> TopoBRepData;
		TopoBRepData->m_uiConnexSize = 1;
		TopoBRepData->m_ppConnexes = &TopoConnexPtr;
		TopoBRepPtr = TechSoftInterface::CreateTopoBRep(*TopoBRepData);
		if (TopoBRepPtr == nullptr)
		{
			return nullptr;
		}
	}

	CADLibrary::TUniqueTSObj<A3DRiBrepModelData> RiBRepModelData;
	RiBRepModelData->m_pBrepData = TopoBRepPtr;
	RiBRepModelData->m_bSolid = false;
	return TechSoftInterface::CreateRiBRepModel(*RiBRepModelData);
}

A3DTopoEdge* CreateTopoEdge()
{
	CADLibrary::TUniqueTSObj<A3DTopoEdgeData> EdgeData;
	return TechSoftInterface::CreateTopoEdge(*EdgeData);
}

A3DTopoFace* CreateTopoFaceWithNaturalLoop(A3DSurfBase* CarrierSurface)
{
	CADLibrary::TUniqueTSObj<A3DTopoFaceData> Face;
	Face->m_pSurface = CarrierSurface;
	Face->m_bHasTrimDomain = false;
	Face->m_ppLoops = nullptr;
	Face->m_uiLoopSize = 0;
	Face->m_uiOuterLoopIndex = 0;
	Face->m_dTolerance = 0.01; //mm

	return CADLibrary::TechSoftInterface::CreateTopoFace(*Face);
}

A3DCrvNurbs* CreateTrimNurbsCurve(A3DCrvNurbs* CurveNurbsPtr, double UMin, double UMax, bool bIs2D)
{
	if (CurveNurbsPtr == nullptr)
	{
		return nullptr;
	}

	TUniqueTSObj<A3DCrvTransformData> TransformCurveData;

	TransformCurveData->m_bIs2D = bIs2D;
	TransformCurveData->m_sParam.m_sInterval.m_dMin = UMin;
	TransformCurveData->m_sParam.m_sInterval.m_dMax = UMax;
	TransformCurveData->m_sParam.m_dCoeffA = 1.;
	TransformCurveData->m_sParam.m_dCoeffB = 0.;
	TransformCurveData->m_pBasisCrv = CurveNurbsPtr;
	TransformCurveData->m_pTransfo = nullptr;

	TransformCurveData->m_sTrsf.m_sXVector.m_dX = 1.;
	TransformCurveData->m_sTrsf.m_sYVector.m_dY = 1.;
	TransformCurveData->m_sTrsf.m_sScale.m_dX = 1.;
	TransformCurveData->m_sTrsf.m_sScale.m_dY = 1.;
	TransformCurveData->m_sTrsf.m_sScale.m_dZ = 1.;

	A3DCrvTransform* CurveTransformPtr = CADLibrary::TechSoftInterface::CreateCurveTransform(*TransformCurveData);
	if (CurveTransformPtr == nullptr)
	{
		return nullptr;
	}

	TUniqueTSObj<A3DCrvNurbsData> TransformedNurbsData;
	if (CADLibrary::TechSoftInterface::GetCurveAsNurbs(CurveTransformPtr, &*TransformedNurbsData, 0.01 /*mm*/, /*bUseSameParameterization*/ true) != A3DStatus::A3D_SUCCESS)
	{
		return nullptr;
	}

	return CADLibrary::TechSoftInterface::CreateCurveNurbs(*TransformedNurbsData);
}
#endif

#ifdef USE_TECHSOFT_SDK
void ExtractAttribute(const A3DMiscAttributeData& AttributeData, TMap<FString, FString>& OutMetaData)
{
	FString AttributeName;
	if (AttributeData.m_bTitleIsInt)
	{
		A3DUns32 UnsignedValue = 0;
		memcpy(&UnsignedValue, AttributeData.m_pcTitle, sizeof(A3DUns32));
		AttributeName = FString::Printf(TEXT("%u"), UnsignedValue);
	}
	else if (AttributeData.m_pcTitle && AttributeData.m_pcTitle[0] != '\0')
	{
		AttributeName = UTF8_TO_TCHAR(AttributeData.m_pcTitle);
	}

	for (A3DUns32 Index = 0; Index < AttributeData.m_uiSize; ++Index)
	{
		FString AttributeValue;
		switch (AttributeData.m_asSingleAttributesData[Index].m_eType)
		{
		case kA3DModellerAttributeTypeTime:
		case kA3DModellerAttributeTypeInt:
		{
			A3DInt32 Value = *reinterpret_cast<A3DInt32*>(AttributeData.m_asSingleAttributesData[Index].m_pcData);
			AttributeValue = FString::Printf(TEXT("%d"), Value);
			break;
		}
		case kA3DModellerAttributeTypeReal:
		{
			A3DDouble Value = *reinterpret_cast<A3DDouble*>(AttributeData.m_asSingleAttributesData[Index].m_pcData);
			AttributeValue = FString::Printf(TEXT("%f"), Value);
			break;
		}

		case kA3DModellerAttributeTypeString:
		{
			if (AttributeData.m_asSingleAttributesData[Index].m_pcData && AttributeData.m_asSingleAttributesData[Index].m_pcData[0] != '\0')
			{
				AttributeValue = UTF8_TO_TCHAR(AttributeData.m_asSingleAttributesData[Index].m_pcData);
			}
			break;
		}

		default:
			break;
		}

		if (AttributeName.Len())
		{
			if (Index)
			{
				OutMetaData.Emplace(FString::Printf(TEXT("%s_%u"), *AttributeName, (int32)Index), AttributeValue);
			}
			else
			{
				OutMetaData.Emplace(AttributeName, AttributeValue);
			}
		}
	}
}
#endif

FString CleanLabel(const FString& Name)
{
	int32 Index;
	if (Name.FindLastChar(TEXT('['), Index))
	{
		return Name.Left(Index);
	}
	return Name;
}

FString CleanCatiaInstanceLabel(const FString& Name)
{
	int32 Index;
	if (Name.FindChar(TEXT('('), Index))
	{
		FString NewName = Name.RightChop(Index + 1);
		if (NewName.FindLastChar(TEXT(')'), Index))
		{
			NewName = NewName.Left(Index);
		}
		return NewName;
	}
	return Name;
}

FString Clean3dxmlInstanceLabel(const FString& Name)
{
	FString NewName = CleanCatiaInstanceLabel(Name);

	int32 Index = NewName.Find(TEXT("_InstanceRep"));
	if (Index != INDEX_NONE)
	{
		NewName = NewName.Left(Index);
	}

	return NewName;
}

FString Clean3dxmlReferenceLabel(const FString& Name)
{
	int32 Index;
	if (Name.FindChar(TEXT('('), Index))
	{
		FString NewName = Name.Left(Index);
		return NewName;
	}

	Index = Name.Find(TEXT("_InstanceRep"));
	if (Index != INDEX_NONE)
	{
		FString NewName = Name.Left(Index);
		return NewName;
	}

	return Name;
}

FString CleanSwInstanceLabel(const FString& Name)
{
	int32 Position;
	if (Name.FindLastChar(TEXT('-'), Position))
	{
		FString NewName = Name.Left(Position) + TEXT("<") + Name.RightChop(Position + 1) + TEXT(">");
		return NewName;
	}
	return Name;
}

FString CleanSwReferenceLabel(const FString& Name)
{
	int32 Position;
	if (Name.FindLastChar(TEXT('-'), Position))
	{
		FString NewName = Name.Left(Position);
		return NewName;
	}
	return Name;
}

FString CleanCatiaReferenceLabel(const FString& Name)
{
	int32 Position;
	if (Name.FindLastChar(TEXT('.'), Position))
	{
		FString Indice = Name.RightChop(Position + 1);
		if (Indice.IsNumeric())
		{
			FString NewName = Name.Left(Position);
			return NewName;
		}
	}
	return Name;
}
FString CleanCreoLabel(const FString& Name)
{
	int32 Position;
	if (Name.FindLastChar(TEXT('.'), Position))
	{
		FString Extension = Name.RightChop(Position);
		if (Extension.Equals(TEXT("prt"), ESearchCase::IgnoreCase))
		{
			FString NewName = Name.Left(Position);
			return NewName;
		}
	}
	return Name;
}

} // NS TechSoftUtils

} // NS CADLibrary
