// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "TechSoftInterface.h"

namespace CADLibrary
{

namespace TechSoftUtils
{

#ifdef USE_TECHSOFT_SDK
void ExtractAttribute(const A3DMiscAttributeData& AttributeData, TMap<FString, FString>& OutMetaData);
#endif

FString CleanLabel(const FString& Name);
FString CleanCatiaInstanceLabel(const FString& Name);
FString Clean3dxmlInstanceLabel(const FString& Name);
FString CleanCatiaReferenceLabel(const FString& Name);
FString Clean3dxmlReferenceLabel(const FString& Name);
FString CleanSwInstanceLabel(const FString& Name);
FString CleanSwReferenceLabel(const FString& Name);
FString CleanCreoLabel(const FString& Name);
bool CheckIfNameExists(TMap<FString, FString>& MetaData);
bool ReplaceOrAddNameValue(TMap<FString, FString>& MetaData, const TCHAR* Key);

} // NS TechSoftUtils

} // CADLibrary

