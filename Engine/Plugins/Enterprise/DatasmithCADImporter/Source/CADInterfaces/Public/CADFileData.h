// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADOptions.h"
#include "CADSceneGraph.h"

namespace CADLibrary
{

#define CSV_HEADER TEXT("Input File,FileHash,FileSize(KB),Time.LoadStep(ms),Time.SavePrcTime(ms),Time.AdaptBRepTime(ms),Time.SewStep(ms),Time.MeshStep(ms),Time.Total(ms),MemoryUsage(MB),SceneGrapheData.InstanceNum,SceneGrapheData.Bodies,SceneGrapheData.References,SceneGrapheData.UnloadedReferences,SceneGrapheData.ExternalReferenceFiles,SceneGrapheData.ColorHIdToColor,SceneGrapheData.MaterialHIdToMaterial,Meshes.FaceCount,Meshes.VertexCount, 0\n")

struct FImportRecord
{
	double ImportTime = 0;
	double SavePrcTime = 0;
	double SewTime = 0;
	double AdaptBRepTime = 0;
	double MeshTime = 0;
	double LoadProcessTime = 0; // Full process

	uint64 StartMemoryUsed = 0;
	uint64 MaxMemoryUsed = 0;
};

class FCADFileData
{
public:
	FCADFileData(const FImportParameters& InImportParameters, const FFileDescriptor& InFileDescription, const FString& InCachePath)
		: ImportParameters(InImportParameters)
		, CachePath(InCachePath)
		, bIsCacheDefined(!InCachePath.IsEmpty())
		, FileDescription(InFileDescription)
	{
		SceneGraphArchive.FullPath = FileDescription.GetSourcePath();
		SceneGraphArchive.CADFileName = FileDescription.GetFileName();
	}

	uint32 GetSceneFileHash() const
	{
		if (!SceneFileHash)
		{
			SceneFileHash = HashCombine(FileDescription.GetDescriptorHash(), ::GetTypeHash(ImportParameters.GetStitchingTechnique()));
		}
		return SceneFileHash;
	}

	uint32 GetGeomFileHash() const
	{
		if (!GeomFileHash)
		{
			if (FileDescription.GetFileFormat() == ECADFormat::JT && FImportParameters::bGPreferJtFileEmbeddedTessellation)
			{
				GeomFileHash = GetSceneFileHash();
			}
			else
			{
				GeomFileHash = GetSceneFileHash();
				GeomFileHash = HashCombine(GeomFileHash, GetTypeHash(ImportParameters));
				GeomFileHash = HashCombine(GeomFileHash, ::GetTypeHash(FImportParameters::bGSewMeshIfNeeded));
				GeomFileHash = HashCombine(GeomFileHash, ::GetTypeHash(FImportParameters::bGRemoveDuplicatedTriangle));
				GeomFileHash = HashCombine(GeomFileHash, ::GetTypeHash(FImportParameters::GStitchingTolerance));
				GeomFileHash = HashCombine(GeomFileHash, ::GetTypeHash(FImportParameters::bGStitchingForceSew));
				GeomFileHash = HashCombine(GeomFileHash, ::GetTypeHash(FImportParameters::bGStitchingRemoveThinFaces));
				GeomFileHash = HashCombine(GeomFileHash, ::GetTypeHash(FImportParameters::bGStitchingRemoveDuplicatedFaces));
				GeomFileHash = HashCombine(GeomFileHash, ::GetTypeHash(FImportParameters::GStitchingForceFactor));
			}
		}
		return GeomFileHash;
	}

	void SetArchiveNames()
	{
		SceneGraphArchive.SceneGraphId = FileDescription.GetDescriptorHash();
		SceneGraphArchive.ArchiveFileName = FString::Printf(TEXT("UEx%08x"), GetSceneFileHash());
		MeshArchiveFile = FString::Printf(TEXT("UEx%08x"), GetGeomFileHash());
	}

	const FString GetSceneGraphFilePath() const
	{
		if (IsCacheDefined())
		{
			return FPaths::Combine(CachePath, TEXT("scene"), SceneGraphArchive.ArchiveFileName + TEXT(".sg"));
		}
		return FString();
	}

	const FString GetValidationFilePath() const
	{
		if (IsCacheDefined())
		{
			const FString& RootPath = FileDescription.GetRootFolder();
			const FString& FilePath = FileDescription.GetSourcePath();
			FString FileRootFolder = FilePath.Right(FilePath.Len() - RootPath.Len() - 1);
			int32 Index;
			if (FileRootFolder.FindChar(TEXT('/'), Index))
			{
				FileRootFolder = FileRootFolder.Left(Index);
			}
			return FPaths::Combine(CachePath, TEXT("validation"), FileRootFolder, SceneGraphArchive.ArchiveFileName + TEXT(".csv"));
		}
		return FString();
	}

	const FString GetMeshArchiveFilePath() const
	{
		if (IsCacheDefined())
		{
			return FPaths::Combine(CachePath, TEXT("mesh"), MeshArchiveFile + TEXT(".gm"));
		}
		return FString();
	}

	const FString GetBodyCachePath(uint32 BodyHash) const
	{
		return CADLibrary::BuildCacheFilePath(*CachePath, TEXT("body"), BodyHash, ImportParameters.GetMesher());
	}

	/**
	 * @return the path of CAD cache file
	 */
	const FString GetCADCachePath() const
	{
		if (IsCacheDefined())
		{
			uint32 CADFileHash = FileDescription.GetDescriptorHash();
			if (FileDescription.GetFileFormat() == ECADFormat::JT && FImportParameters::bGPreferJtFileEmbeddedTessellation)
			{
				CADFileHash = HashCombine(CADFileHash, 1);
			}
			return CADLibrary::BuildCadCachePath(*CachePath, CADFileHash);
		}
		return FString();
	}

	const FString GetCachePath() const
	{
		if (IsCacheDefined())
		{
			return CachePath;
		}
		return FString();
	}

	bool IsCacheDefined() const
	{
		return bIsCacheDefined;
	}

	void LogError(const FString& Message)
	{
		Messages.Emplace(ELogVerbosity::Error, Message);
	}

	void LogWarning(const FString& Message)
	{
		Messages.Emplace(ELogVerbosity::Warning, Message);
	}

	/** Logs a message to console (and log file) */
	void LogDisplayMessage(const FString& Message)
	{
		Messages.Emplace(ELogVerbosity::Display, Message);
	}

	/** logs a message to a log file (does not print to console) */
	void LogMessage(const FString& Message)
	{
		Messages.Emplace(ELogVerbosity::Log, Message);
	}

	void LogVerboseMessage(const FString& Message)
	{
		Messages.Emplace(ELogVerbosity::Verbose, Message);
	}

	void LoadSceneGraphArchive()
	{
		FString SceneGraphFilePath = GetSceneGraphFilePath();
		SceneGraphArchive.DeserializeMockUpFile(*SceneGraphFilePath);
	}

	void ExportSceneGraphFile()
	{
		FString SceneGraphFilePath = GetSceneGraphFilePath();
		SceneGraphArchive.SerializeMockUp(*SceneGraphFilePath);
	}

	FFileDescriptor& GetExternalReferences(int32 Index)
	{
		return SceneGraphArchive.ExternalReferenceFiles[Index];
	}

	FFileDescriptor& AddExternalRef(const TCHAR* InFilePath, const TCHAR* InConfiguration, const TCHAR* InRootFilePath)
	{
		return SceneGraphArchive.ExternalReferenceFiles.Emplace_GetRef(InFilePath, InConfiguration, InRootFilePath);
	}

	FFileDescriptor& AddExternalRef(const FFileDescriptor& InFileDescription)
	{
		return SceneGraphArchive.ExternalReferenceFiles.Emplace_GetRef(InFileDescription);
	}

	/** return a unique value that will be used to define the static mesh name */
	uint32 GetStaticMeshHash(const int32 BodyId)
	{
		return HashCombine(GetSceneFileHash(), ::GetTypeHash(BodyId));
	}

	FBodyMesh& AddBodyMesh(FCadId BodyId, FArchiveBody& Body)
	{
		FBodyMesh& BodyMesh = BodyMeshes.Emplace_GetRef(BodyId);
		BodyMesh.MeshActorUId = GetStaticMeshHash(BodyId);
		BodyMesh.bIsFromCad = Body.bIsFromCad;
		Body.MeshActorUId = BodyMesh.MeshActorUId;
		return BodyMesh;
	}

	void ExportMeshArchiveFile()
	{
		FString MeshArchiveFilePath = GetMeshArchiveFilePath();
		SerializeBodyMeshSet(*MeshArchiveFilePath, BodyMeshes);
	}

	const TArray<FFileDescriptor>& GetExternalRefSet() const
	{
		return SceneGraphArchive.ExternalReferenceFiles;
	}

	const FString& GetSceneGraphFileName() const
	{
		return SceneGraphArchive.ArchiveFileName;
	}

	const FString& GetMeshFileName() const
	{
		return MeshArchiveFile;
	}

	const TArray<TPair<uint8, FString>>& GetMessages() const
	{
		return Messages;
	}

	const FArchiveSceneGraph& GetSceneGraphArchive() const
	{
		return SceneGraphArchive;
	}

	FArchiveSceneGraph& GetSceneGraphArchive()
	{
		return SceneGraphArchive;
	}

	FArchiveMaterial* FindMaterial(FMaterialUId MaterialId)
	{
		return SceneGraphArchive.MaterialHIdToMaterial.Find(MaterialId);
	}

	FArchiveMaterial& AddMaterial(FMaterialUId MaterialId)
	{
		return SceneGraphArchive.MaterialHIdToMaterial.Emplace(MaterialId, MaterialId);
	}

	FArchiveColor* FindColor(FMaterialUId ColorId)
	{
		return SceneGraphArchive.ColorHIdToColor.Find(ColorId);
	}

	FArchiveColor& AddColor(FMaterialUId ColorId)
	{
		return SceneGraphArchive.ColorHIdToColor.Emplace(ColorId, ColorId);
	}

	const TArray<FBodyMesh>& GetBodyMeshes() const
	{
		return BodyMeshes;
	}

	TArray<FBodyMesh>& GetBodyMeshes()
	{
		return BodyMeshes;
	}

	const FFileDescriptor& GetCADFileDescription() const
	{
		return FileDescription;
	}

	FFileDescriptor& GetCADFileDescription()
	{
		return FileDescription;
	}

	void ReserveBodyMeshes(int32 MaxBodyCount)
	{
		BodyMeshes.Reserve(MaxBodyCount);
	}

	const FImportParameters& GetImportParameters() const
	{
		return ImportParameters;
	}

	void UpdateExternalRefPath()
	{
		if(!FImportParameters::bValidationProcess)
		{
			const FString Root = FileDescription.GetRootFolder();
			const FString ArchiveRoot = FPaths::GetPath(SceneGraphArchive.FullPath);
			for (FFileDescriptor& File : SceneGraphArchive.ExternalReferenceFiles)
			{
				File.ChangePath(ArchiveRoot, Root);
			}
		}
	}

	void InitCsvFile();
	void ExportValidationData();
	void GetMeshStatsToCsv(int32& FaceCount, int32& VertexCount);

	FImportRecord& GetRecord()
	{
		return Record;
	}

private:
	const FImportParameters ImportParameters;
	const FString CachePath;
	const bool bIsCacheDefined;

	FFileDescriptor FileDescription;

	FString MeshArchiveFile;

	FArchiveSceneGraph SceneGraphArchive;
	TArray<FBodyMesh> BodyMeshes;

	TArray<TPair<uint8, FString>> Messages;

	FImportRecord Record;

	mutable uint32 SceneFileHash = 0;
	mutable uint32 GeomFileHash = 0;
};
}