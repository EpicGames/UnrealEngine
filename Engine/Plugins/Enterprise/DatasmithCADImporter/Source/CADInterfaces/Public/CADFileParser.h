// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace CADLibrary
{
enum class ECADParsingResult : uint8;

class ICADFileParser
{
public:
	virtual ~ICADFileParser() = default;
	virtual ECADParsingResult Process() = 0;
};

}