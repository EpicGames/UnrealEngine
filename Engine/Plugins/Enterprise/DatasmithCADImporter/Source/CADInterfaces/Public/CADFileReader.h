// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADFileData.h"
#include "CADFileParser.h"
#include "CADOptions.h"
#include "CADSceneGraph.h"

namespace CADLibrary
{

	class CADINTERFACES_API FCADFileReader
	{
	public:
		/**
		 * @param ImportParams Parameters that setting import data like mesh SAG...
		 * @param EnginePluginsPath Full Path of EnginePlugins. Mandatory to set KernelIO to import DWG, or DGN files
		 * @param InCachePath Full path of the cache in which the data will be saved
		 */
		FCADFileReader(const FImportParameters& ImportParams, FFileDescriptor& InCTFileDescription, const FString& EnginePluginsPath = TEXT(""), const FString& InCachePath = TEXT(""));
		ECADParsingResult ProcessFile();

		const FCADFileData& GetCADFileData() const
		{
			return CADFileData;
		}

		FCADFileData& GetCADFileData()
		{
			return CADFileData;
		}

	private:
		bool FindFile(FFileDescriptor& File);

		uint32 GetSceneFileHash(const uint32 InSGHash, const FImportParameters& ImportParam);
		uint32 GetGeomFileHash(const uint32 InSGHash, const FImportParameters& ImportParam);

		void LoadSceneGraphArchive(const FString& SceneGraphFilePath);

		void ExportSceneGraphFile();
		void ExportMeshArchiveFile();

		FCADFileData CADFileData;
		TUniquePtr<ICADFileParser> CADParser;
	};
} // ns CADLibrary
