// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define CADINTERFACES_MODULE_NAME TEXT("CADInterfaces")

DECLARE_LOG_CATEGORY_EXTERN(LogCADInterfaces, Log, All);

enum class ECADInterfaceAvailability
{
	Unknown,
	Available,
	Unavailable,
};

class CADINTERFACES_API ICADInterfacesModule : public IModuleInterface
{
public:
	static ICADInterfacesModule& Get();
	static ECADInterfaceAvailability GetAvailability();
	static const TCHAR* GetLibraryVersion();
};
