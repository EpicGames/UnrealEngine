// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADOptions.h"

class FArchive;

namespace CADLibrary
{

class FArchiveUnloadedReference;

class CADINTERFACES_API FArchiveCADObject : public FArchiveGraphicProperties
{
public:
	FArchiveCADObject()
		: Id(0)
		, Unit(1)
	{
	}

	FArchiveCADObject(FCadId Id, const FArchiveCADObject& Parent)
		: FArchiveGraphicProperties(Parent)
		, Id(Id)
		, Unit(Parent.Unit)
	{
	}

	virtual ~FArchiveCADObject() = default;

	friend FArchive& operator<<(FArchive& Ar, FArchiveCADObject& C);

public:
	uint32 Id;
	FString Label;
	TMap<FString, FString> MetaData;
	FMatrix TransformMatrix = FMatrix::Identity;
	double Unit = 1;

	bool IsNameDefined() const
	{
		return !Label.IsEmpty();
	}

	bool SetNameWithAttributeValue(const TCHAR* Key);

};

class CADINTERFACES_API FArchiveWithOverridenChildren : public FArchiveCADObject
{
public:
	FArchiveWithOverridenChildren() = default;
	FArchiveWithOverridenChildren(FCadId Id, const FArchiveCADObject& Parent)
		: FArchiveCADObject(Id, Parent)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FArchiveWithOverridenChildren& C);

	void AddOverridenChild(const FCadId ChildId);

public:
	TArray<FCadId> OverridenChildren;
};

/**
 * This class save the override data of an occurrence. 
 * Indeed, some tools/formats allows to override some occurrences of the instances of a reference (e.g. only the 4th occurrence of C is hidden in the graph below)
 * 
 *     A
 *     | - B
 *     |   | - C (show)
 *     |   | - C (show)
 *     | 
 *     | - B
 *     |   | - C (show)
 *     |   | - C (hide)
 */
class CADINTERFACES_API FArchiveOverrideOccurrence : public FArchiveWithOverridenChildren
{
public:
	FArchiveOverrideOccurrence() = default;
	FArchiveOverrideOccurrence(FCadId Id, const FArchiveCADObject& Parent)
		: FArchiveWithOverridenChildren(Id, Parent)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FArchiveOverrideOccurrence& C);
};

/**
 * An instance is link to its reference but It also save its tree of override occurrence children
 */
class CADINTERFACES_API FArchiveInstance : public FArchiveWithOverridenChildren
{
public:
	FArchiveInstance() = default;
	FArchiveInstance(FCadId Id, const FArchiveCADObject& Parent)
		: FArchiveWithOverridenChildren(Id, Parent)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FArchiveInstance& C);

public:
	FCadId ReferenceNodeId = 0;
	bool bIsExternalReference = false;
};

class CADINTERFACES_API FArchiveReference : public FArchiveCADObject
{
public:
	FArchiveReference() = default;
	FArchiveReference(FCadId Id, const FArchiveCADObject& Reference)
		: FArchiveCADObject(Id, Reference)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FArchiveReference& C);

	void AddChild(const FCadId ChildId);
	void RemoveLastChild();

	int32 ChildrenCount();
	void MoveTemp(FArchiveUnloadedReference& Reference);
	void CopyMetaData(FArchiveUnloadedReference& Reference);

public:
	TArray<FCadId> Children;
};

class CADINTERFACES_API FArchiveUnloadedReference : public FArchiveCADObject
{
public:
	FArchiveUnloadedReference() = default;

	FArchiveUnloadedReference(FCadId Id, const FArchiveCADObject& Parent)
		: FArchiveCADObject(Id, Parent)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FArchiveUnloadedReference& C);


public:
	FFileDescriptor ExternalFile;

// not serialized
	bool bIsUnloaded = true;
};

class CADINTERFACES_API FArchiveBody : public FArchiveCADObject
{
public:
	FArchiveBody() = default;
	FArchiveBody(FCadId Id, const FArchiveCADObject& Parent, EMesher InMesher)
		: FArchiveCADObject(Id, Parent)
		, ParentId(Parent.Id)
		, Mesher(InMesher)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FArchiveBody& C);

public:
	FCadId ParentId = 0;
	FCadUuid MeshActorUId = 0;

	TSet<FMaterialUId> MaterialFaceSet;
	TSet<FMaterialUId> ColorFaceSet;

	EMesher Mesher = EMesher::TechSoft;

	void Delete();

	// non serialized fields
	bool bIsFromCad = true;
	bool bIsASolid = true;
};

class CADINTERFACES_API FArchiveColor
{
public:
	FArchiveColor(FMaterialUId InId = 0)
		: Id(InId)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FArchiveColor& Color);

public:
	FMaterialUId Id;
	FColor Color;
	FMaterialUId UEMaterialUId;
};

class CADINTERFACES_API FArchiveMaterial
{
public:
	FArchiveMaterial(FMaterialUId InId = 0)
		: Id(InId)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FArchiveMaterial& Material);

public:
	FMaterialUId Id;
	FCADMaterial Material;
	FMaterialUId UEMaterialUId;
};

class CADINTERFACES_API FArchiveSceneGraph
{
public:
	friend FArchive& operator<<(FArchive& Ar, FArchiveSceneGraph& C);

	void SerializeMockUp(const TCHAR* Filename);
	void DeserializeMockUpFile(const TCHAR* Filename);

public:
	uint32 SceneGraphId = 0;
	FString CADFileName;
	FString ArchiveFileName;
	FString FullPath;

	TMap<FMaterialUId, FArchiveColor> ColorHIdToColor;
	TMap<FMaterialUId, FArchiveMaterial> MaterialHIdToMaterial;

	TArray<FArchiveBody> Bodies;
	TArray<FArchiveReference> References;
	TArray<FArchiveUnloadedReference> UnloadedReferences;
	TArray<FFileDescriptor> ExternalReferenceFiles;
	TArray<FArchiveInstance> Instances;
	TArray<FArchiveOverrideOccurrence> OverrideOccurrences;

	TArray<int32> CADIdToIndex;
	FCadId LastEntityId = 1;

	void Reserve(uint32* ComponentCount)
	{
		// Must be call once
		ensure(LastEntityId == 1);

		Instances.Reserve(ComponentCount[EComponentType::Instance]);
		References.Reserve(ComponentCount[EComponentType::Reference]);
		UnloadedReferences.Reserve(ComponentCount[EComponentType::Reference]);
		ExternalReferenceFiles.Reserve(ComponentCount[EComponentType::Reference]);
		Bodies.Reserve(ComponentCount[EComponentType::Body]);

		OverrideOccurrences.Reserve(ComponentCount[EComponentType::OverriddeOccurence]);

		CADIdToIndex.Reserve(ComponentCount[EComponentType::OverriddeOccurence] + ComponentCount[EComponentType::Instance] + ComponentCount[EComponentType::Reference] + ComponentCount[EComponentType::Body] + 1);
		CADIdToIndex.Add(0);
	}

	FArchiveInstance& AddInstance(const FArchiveCADObject& Parent);
	FArchiveInstance& GetInstance(const FCadId CadId);
	void RemoveLastInstance();
	bool IsAInstance(const FCadId CadId) const;

	FArchiveReference& AddReference(FArchiveUnloadedReference&);
	FArchiveReference& AddReference(FArchiveInstance& Parent);
	FArchiveReference& GetReference(const FCadId CadId);
	FArchiveReference* GetRootReference()
	{
		constexpr FCadId RootId = 1;
		return CADIdToIndex.IsValidIndex(RootId) ? &GetReference(RootId) : nullptr;
	}

	void RemoveLastReference();
	bool IsAReference(FCadId CadId) const;

	FArchiveReference& AddOccurrence(FArchiveReference& Parent);
	void RemoveLastOccurrence();

	FArchiveUnloadedReference& AddUnloadedReference(FArchiveInstance& Parent);
	FArchiveUnloadedReference& GetUnloadedReference(const FCadId CadId);
	void RemoveLastUnloadedReference();
	bool IsAUnloadedReference(const FCadId CadId) const;

	FArchiveOverrideOccurrence& AddOverrideOccurrence(FArchiveWithOverridenChildren& Parent);
	FArchiveOverrideOccurrence& GetOverrideOccurrence(const FCadId CadId);

	FArchiveBody& AddBody(FArchiveReference& Parent, EMesher InMesher);
	FArchiveBody& GetBody(const FCadId CadId);
	void RemoveLastBody();
	bool IsABody(const FCadId CadId) const;

	void AddExternalReferenceFile(const FArchiveUnloadedReference& Reference);

	int32 ReferencesCount() const
	{
		return References.Num();
	}

private:

};


}


