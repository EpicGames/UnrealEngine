// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifdef USE_TECHSOFT_SDK

#include "TechSoftInterface.h"

namespace CADLibrary
{

// Single-ownership smart TeshSoft object
// Use this when you need to manage TechSoft object's lifetime.
//
// TechSoft give access to void pointers
// According to the context, the class name of the void pointer is known but the class is unknown
// i.e. A3DSDKTypes.h defines all type like :
// 	   typedef void A3DEntity;		
// 	   typedef void A3DAsmModelFile; ...
// 
// From a pointer, TechSoft give access to a copy of the associated structure :
//
// const A3DXXXXX* pPointer;
// A3DXXXXXData sData; // the structure
// A3D_INITIALIZE_DATA(A3DXXXXXData, sData); // initialization of the structure
// A3DXXXXXXGet(pPointer, &sData); // Copy of the data of the pointer in the structure
// ...
// A3DXXXXXXGet(NULL, &sData); // Free the structure
//
// A3D_INITIALIZE_DATA, and all A3DXXXXXXGet methods are TechSoft macro
//

template<class ObjectType, class IndexerType>
class TUniqueTSObjBase
{
public:

	/**
	 * Constructor of an initialized ObjectType object
	 */
	TUniqueTSObjBase()
		: bDataFromTechSoft(false)
		, Status(A3DStatus::A3D_SUCCESS)
	{
		InitializeData();
	}

	/**
	 * Constructor of an filled ObjectType object with the data of DataPtr
	 * @param DataPtr: the pointer of the data to copy
	 */
	explicit TUniqueTSObjBase(IndexerType DataPtr)
		: bDataFromTechSoft(false)
	{
		InitializeData();
		FillFrom(DataPtr);
	}

	~TUniqueTSObjBase()
	{
		ResetData();
	}

	/**
	 * Fill the structure with the data of a new DataPtr
	 */
	A3DStatus FillFrom(IndexerType EntityPtr)
	{
		ResetData();

		if (EntityPtr == GetDefaultIndexerValue())
		{
			Status = A3DStatus::A3D_ERROR;
		}
		else
		{
			Status = GetData(EntityPtr);
			if (Status == A3DStatus::A3D_SUCCESS)
			{
				bDataFromTechSoft = true;
			}
		}
		return Status;
	}

	template<typename... InArgTypes>
	A3DStatus FillWith(A3DStatus(*Getter)(const A3DEntity*, ObjectType*, InArgTypes&&...), const A3DEntity* EntityPtr, InArgTypes&&... Args)
	{
		ResetData();

		if (EntityPtr == GetDefaultIndexerValue())
		{
			Status = A3DStatus::A3D_ERROR;
		}
		else
		{
			Status = Getter(EntityPtr, &Data, Forward<InArgTypes>(Args)...);
			if (Status == A3DStatus::A3D_SUCCESS)
			{
				bDataFromTechSoft = true;
			}
		}

		return Status;
	}

	/**
	 * Empty the structure
	 */
	void Reset()
	{
		ResetData();
	}

	/**
	 * Return
	 *  - A3DStatus::A3D_SUCCESS if the data is filled
	 *  - A3DStatus::A3D_ERROR if the data is empty
	 */
	A3DStatus GetStatus()
	{
		return Status;
	}

	/**
	 * Return true if the data is filled
	 */
	const bool IsValid() const
	{
		return Status == A3DStatus::A3D_SUCCESS;
	}

	// Non-copyable
	TUniqueTSObjBase(const TUniqueTSObjBase&) = delete;
	TUniqueTSObjBase& operator=(const TUniqueTSObjBase&) = delete;

	// Conversion methods

	const ObjectType& operator*() const
	{
		return Data;
	}

	ObjectType& operator*()
	{
		check(IsValid());
		return Data;
	}

	const ObjectType* operator->() const
	{
		check(IsValid());
		return &Data;
	}

	ObjectType* operator->()
	{
		check(IsValid());
		return &Data;
	}

	/**
	 * Return the structure pointer
	 */
	ObjectType* GetPtr()
	{
		if (Status != A3DStatus::A3D_SUCCESS)
		{
			return nullptr;
		}
		return &Data;
	}

private:
	ObjectType Data;
	bool bDataFromTechSoft = false;
	A3DStatus Status = A3DStatus::A3D_ERROR;

	/**
	 * DefaultValue is used to initialize "Data" with GetData method
	 * According to IndexerType, the value is either nullptr for const A3DEntity* either something like "A3D_DEFAULT_MATERIAL_INDEX" ((A3DUns16)-1) for uint32
	 * @see ResetData
	 */
	CADINTERFACES_API IndexerType GetDefaultIndexerValue() const;

	CADINTERFACES_API void InitializeData()
#ifdef USE_TECHSOFT_SDK
		;
#else
	{
		return A3DStatus::A3D_ERROR;
	}
#endif

	CADINTERFACES_API A3DStatus GetData(IndexerType AsmModelFilePtr);
#ifdef USE_TECHSOFT_SDK
	;
#else
	{
		return A3DStatus::A3D_ERROR;
	}
#endif

	void ResetData()
	{
		if (bDataFromTechSoft)
		{
			GetData(GetDefaultIndexerValue());
		}
		else
		{
			InitializeData();
		}
		Status = A3DStatus::A3D_SUCCESS;
		bDataFromTechSoft = false;
	}
};


template<class ObjectType>
using TUniqueTSObj = TUniqueTSObjBase<ObjectType, const A3DEntity*>;

template<class ObjectType>
using TUniqueTSObjFromIndex = TUniqueTSObjBase<ObjectType, uint32>;


// TUniqueTSObj -----------------------------------

template<>
CADINTERFACES_API void TUniqueTSObj<A3DAsmModelFileData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DAsmPartDefinitionData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DAsmProductOccurrenceData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DAsmProductOccurrenceDataCV5>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DAsmProductOccurrenceDataSLW>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DAsmProductOccurrenceDataNX>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DBoundingBoxData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCopyAndAdaptBrepModelData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvCircleData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvCompositeData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvEllipseData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvHelixData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvHyperbolaData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvLineData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvNurbsData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvParabolaData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvPolyLineData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DCrvTransformData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DDomainData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DGlobalData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DGraphicsData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DIntervalData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscAttributeData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscCartesianTransformationData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscEntityReferenceData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscGeneralTransformationData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscMaterialPropertiesData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscReferenceOnCsysItemData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscReferenceOnTessData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscReferenceOnTopologyData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DMiscSingleAttributeData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRWParamsExportPrcData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRiBrepModelData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRiCoordinateSystemData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRiDirectionData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRiPolyBrepModelData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRiRepresentationItemData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRiSetData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRootBaseData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRootBaseWithGraphicsData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DRWParamsTessellationData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSewOptionsData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfBlend01Data>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfBlend02Data>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfBlend03Data>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfConeData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfCylinderData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfCylindricalData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfExtrusionData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfFromCurvesData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfNurbsData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfPipeData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfPlaneData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfRevolutionData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfRuledData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfSphereData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DSurfTorusData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTess3DData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTessBaseData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoBodyData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoBrepDataData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoCoEdgeData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoConnexData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoContextData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoEdgeData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoFaceData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoLoopData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoShellData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoUniqueVertexData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DTopoWireEdgeData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DVector2dData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObj<A3DVector3dData>::InitializeData();

template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DAsmModelFileData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DAsmPartDefinitionData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DAsmProductOccurrenceData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DAsmProductOccurrenceDataCV5>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DAsmProductOccurrenceDataSLW>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DAsmProductOccurrenceDataNX>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DBoundingBoxData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCopyAndAdaptBrepModelData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvCircleData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvCompositeData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvEllipseData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvHelixData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvHyperbolaData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvLineData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvNurbsData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvParabolaData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvPolyLineData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DCrvTransformData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DDomainData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DGlobalData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DGraphicsData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DIntervalData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscAttributeData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscCartesianTransformationData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscEntityReferenceData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscGeneralTransformationData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscMaterialPropertiesData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscReferenceOnCsysItemData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscReferenceOnTessData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscReferenceOnTopologyData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DMiscSingleAttributeData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRWParamsExportPrcData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRiBrepModelData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRiCoordinateSystemData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRiDirectionData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRiPolyBrepModelData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRiRepresentationItemData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRiSetData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRootBaseData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRootBaseWithGraphicsData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DRWParamsTessellationData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSewOptionsData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfBlend01Data>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfBlend02Data>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfBlend03Data>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfConeData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfCylinderData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfCylindricalData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfExtrusionData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfFromCurvesData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfNurbsData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfPipeData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfPlaneData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfRevolutionData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfRuledData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfSphereData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DSurfTorusData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTess3DData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTessBaseData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoBodyData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoBrepDataData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoCoEdgeData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoConnexData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoContextData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoEdgeData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoFaceData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoLoopData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoShellData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoUniqueVertexData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DTopoWireEdgeData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DVector2dData>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DVector3dData>::GetData(const A3DEntity* InEntityPtr);

template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DAsmModelFileData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DAsmPartDefinitionData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DAsmProductOccurrenceData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DAsmProductOccurrenceDataCV5>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DAsmProductOccurrenceDataSLW>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DAsmProductOccurrenceDataNX>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DBoundingBoxData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCopyAndAdaptBrepModelData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvCircleData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvCompositeData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvEllipseData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvHelixData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvHyperbolaData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvLineData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvNurbsData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvParabolaData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvPolyLineData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DCrvTransformData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DDomainData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DGlobalData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DGraphicsData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DIntervalData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscAttributeData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscCartesianTransformationData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscEntityReferenceData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscGeneralTransformationData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscMaterialPropertiesData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscReferenceOnCsysItemData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscReferenceOnTessData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscReferenceOnTopologyData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DMiscSingleAttributeData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRWParamsExportPrcData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRiBrepModelData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRiCoordinateSystemData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRiDirectionData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRiPolyBrepModelData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRiRepresentationItemData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRiSetData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRootBaseData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRootBaseWithGraphicsData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DRWParamsTessellationData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSewOptionsData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfBlend01Data>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfBlend02Data>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfBlend03Data>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfConeData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfCylinderData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfCylindricalData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfExtrusionData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfFromCurvesData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfNurbsData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfPipeData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfPlaneData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfRevolutionData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfRuledData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfSphereData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DSurfTorusData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTess3DData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTessBaseData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoBodyData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoBrepDataData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoCoEdgeData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoConnexData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoContextData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoEdgeData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoFaceData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoLoopData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoShellData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoUniqueVertexData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DTopoWireEdgeData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DVector2dData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DVector3dData>::GetDefaultIndexerValue() const;

// TUniqueTSObjFromIndex -----------------------------------

template<>
CADINTERFACES_API void TUniqueTSObjFromIndex<A3DGraphMaterialData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObjFromIndex<A3DGraphPictureData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObjFromIndex<A3DGraphRgbColorData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObjFromIndex<A3DGraphStyleData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObjFromIndex<A3DGraphTextureApplicationData>::InitializeData();
template<>
CADINTERFACES_API void TUniqueTSObjFromIndex<A3DGraphTextureDefinitionData>::InitializeData();

template<>
CADINTERFACES_API A3DStatus TUniqueTSObjFromIndex<A3DGraphMaterialData>::GetData(const uint32 InEntityIndex);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObjFromIndex<A3DGraphPictureData>::GetData(const uint32 InEntityIndex);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObjFromIndex<A3DGraphRgbColorData>::GetData(const uint32 InEntityIndex);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObjFromIndex<A3DGraphStyleData>::GetData(const uint32 InEntityIndex);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObjFromIndex<A3DGraphTextureApplicationData>::GetData(const uint32 InEntityIndex);
template<>
CADINTERFACES_API A3DStatus TUniqueTSObjFromIndex<A3DGraphTextureDefinitionData>::GetData(const uint32 InEntityIndex);

template<>
CADINTERFACES_API uint32 TUniqueTSObjFromIndex<A3DGraphMaterialData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API uint32 TUniqueTSObjFromIndex<A3DGraphPictureData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API uint32 TUniqueTSObjFromIndex<A3DGraphRgbColorData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API uint32 TUniqueTSObjFromIndex<A3DGraphStyleData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API uint32 TUniqueTSObjFromIndex<A3DGraphTextureApplicationData>::GetDefaultIndexerValue() const;
template<>
CADINTERFACES_API uint32 TUniqueTSObjFromIndex<A3DGraphTextureDefinitionData>::GetDefaultIndexerValue() const;

// A3DUTF8Char* -----------------------------------

template<>
CADINTERFACES_API void TUniqueTSObj<A3DUTF8Char*>::InitializeData();
template<>
CADINTERFACES_API A3DStatus TUniqueTSObj<A3DUTF8Char*>::GetData(const A3DEntity* InEntityPtr);
template<>
CADINTERFACES_API const A3DEntity* TUniqueTSObj<A3DUTF8Char*>::GetDefaultIndexerValue() const;

class CADINTERFACES_API FTechSoftDefaultValue
{
public:
	static const uint32 Material;
	static const uint32 Picture;
	static const uint32 RgbColor;
	static const uint32 Style;
	static const uint32 TextureApplication;
	static const uint32 TextureDefinition;
};

}
#endif
