// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifdef CADKERNEL_DEV
#include "Core/Types.h"
#else
#include "CoreMinimal.h"
#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#ifdef USE_TECHSOFT_SDK
#pragma push_macro("TEXT")
#include "A3DSDKIncludes.h"
#pragma pop_macro("TEXT")
#endif
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define JSON_ENTRY_BODY_UNIT		TEXT("BodyUnit")
#define JSON_ENTRY_COLOR_NAME		TEXT("ColorName")
#define JSON_ENTRY_MATERIAL_NAME	TEXT("MaterialName")

typedef void A3DAsmModelFile;

namespace CADLibrary
{

class FBodyMesh;
class FImportParameters;
class FUniqueTechSoftModelFile;

class CADINTERFACES_API FTechSoftInterface
{
public:

	static FTechSoftInterface& Get();

	/*
	* Returns true if the object has been created outside of the memory pool of the running process
	* This is the case when the object has been created by the DatasmithRuntime plugin
	*/
	bool IsExternal()
	{
		return bIsExternal;
	}

	void SetExternal(bool Value)
	{
		bIsExternal = Value;
	}

	bool InitializeKernel(const TCHAR* = TEXT(""));
	const TCHAR* GetVersion();

private:

	bool bIsExternal = false;
	bool bIsInitialize = false;
};

namespace TechSoftInterface
{
CADINTERFACES_API bool TECHSOFT_InitializeKernel(const TCHAR* = TEXT(""));

CADINTERFACES_API FString GetTechSoftVersion();

#ifdef USE_TECHSOFT_SDK

CADINTERFACES_API FUniqueTechSoftModelFile LoadModelFileFromFile(const A3DImport& Import, A3DStatus& Status);
CADINTERFACES_API FUniqueTechSoftModelFile LoadModelFileFromPrcFile(const A3DUTF8Char* CADFileName, A3DRWParamsPrcReadHelper** ReadHelper);

CADINTERFACES_API A3DStatus AdaptBRepInModelFile(A3DAsmModelFile* ModelFile, const A3DCopyAndAdaptBrepModelData& Setting, int32& ErrorCount, A3DCopyAndAdaptBrepModelErrorData** Errors);

CADINTERFACES_API A3DStatus DeleteModelFile(A3DAsmModelFile* ModelFile);
CADINTERFACES_API A3DStatus DeleteEntity(A3DEntity* pEntity);

CADINTERFACES_API A3DGlobal* GetGlobalPointer();
CADINTERFACES_API A3DEntity* GetPointerFromIndex(const uint32 Index, const A3DEEntityType Type);

CADINTERFACES_API A3DStatus GetSurfaceAsNurbs(const A3DSurfBase* SurfacePtr, A3DSurfNurbsData* DataPtr, A3DDouble Tolerance, A3DBool bUseSameParameterization);
CADINTERFACES_API A3DStatus GetSurfaceDomain(const A3DSurfBase* SurfacePtr, A3DDomainData& OutDomain);

/**
 * @param OutPointAndDerivatives size must be equal to (Derivatives+1)*(Derivatives+1)
 * @param Derivatives The number of derivatives must be less or equal to 2
 * @param OutPointAndDerivatives = Point, Derivatives=1 OutPointAndDerivatives = Point + dU + dV, ...
 * @see A3DSDKAdvancedTools.h for more details
 */
CADINTERFACES_API A3DStatus Evaluate(const A3DSurfBase* SurfacePtr, const A3DVector2dData& UVParameter, A3DUns32 Derivatives, A3DVector3dData* OutPointAndDerivatives);

CADINTERFACES_API A3DStatus GetCurveAsNurbs(const A3DCrvBase* A3DCurve, A3DCrvNurbsData* DataPtr, A3DDouble Tolerance, A3DBool bUseSameParameterization);

CADINTERFACES_API A3DStatus GetOriginalFilePathName(const A3DAsmProductOccurrence* A3DOccurrencePtr, A3DUTF8Char** FilePathUTF8Ptr);
CADINTERFACES_API A3DStatus GetFilePathName(const A3DAsmProductOccurrence* A3DOccurrencePtr, A3DUTF8Char** FilePathUTF8Ptr);

CADINTERFACES_API A3DStatus GetEntityType(const A3DEntity* pEntity, A3DEEntityType* peEntityType);
CADINTERFACES_API bool IsEntityBaseWithGraphicsType(const A3DEntity* pEntity);
CADINTERFACES_API bool IsEntityBaseType(const A3DEntity* EntityPtr);
CADINTERFACES_API bool IsMaterialTexture(const uint32 MaterialIndex);


CADINTERFACES_API FUniqueTechSoftModelFile CreateAsmModelFile(const A3DAsmModelFileData& ModelFileData);

CADINTERFACES_API A3DRiBrepModel* CreateRiBRepModel(const A3DRiBrepModelData& RiBRepModelData);
CADINTERFACES_API A3DAsmPartDefinition* CreateAsmPartDefinition(const A3DAsmPartDefinitionData& PartDefinitionData);
CADINTERFACES_API A3DAsmProductOccurrence* CreateAsmProductOccurrence(const A3DAsmProductOccurrenceData& ProductOccurrenceData);
CADINTERFACES_API A3DMiscAttribute* CreateMiscAttribute(const A3DMiscAttributeData& AttributeData);
CADINTERFACES_API A3DAsmModelFile* CreateModelFile(const A3DAsmModelFileData& ModelFileData);
CADINTERFACES_API A3DTopoBrepData* CreateTopoBRep(const A3DTopoBrepDataData& TopoBRepDataData);
CADINTERFACES_API A3DTopoCoEdge* CreateTopoCoEdge(const A3DTopoCoEdgeData& TopoCoEdgeData);
CADINTERFACES_API A3DTopoConnex* CreateTopoConnex(const A3DTopoConnexData& TopoConnexData);
CADINTERFACES_API A3DTopoEdge* CreateTopoEdge(const A3DTopoEdgeData& TopoEdgeData);
CADINTERFACES_API A3DTopoFace* CreateTopoFace(const A3DTopoFaceData& TopoFaceData);
CADINTERFACES_API A3DTopoLoop* CreateTopoLoop(const A3DTopoLoopData& TopoLoopData);
CADINTERFACES_API A3DTopoShell* CreateTopoShell(const A3DTopoShellData& TopoShellData);

CADINTERFACES_API A3DCrvNurbs* CreateCurveTransform(const A3DCrvTransformData& CurveTransformData);
CADINTERFACES_API A3DCrvNurbs* CreateCurveNurbs(const A3DCrvNurbsData& CurveNurbsData);

CADINTERFACES_API A3DSurfNurbs* CreateSurfaceCylinder(const A3DSurfCylinderData& SurfaceCylinderData);
CADINTERFACES_API A3DSurfNurbs* CreateSurfaceNurbs(const A3DSurfNurbsData& SurfaceNurbsData);

CADINTERFACES_API A3DGraphics* CreateGraphics(const A3DGraphicsData& GraphicsData);

CADINTERFACES_API A3DStatus LinkCoEdges(A3DTopoCoEdge* CoEdgePtr, A3DTopoCoEdge* NeighbourCoEdgePtr);

CADINTERFACES_API A3DStatus SetRootBase(A3DEntity* EntityPtr, const A3DRootBaseData& RootBaseData);
CADINTERFACES_API A3DStatus SetRootBaseWithGraphics(const A3DRootBaseWithGraphicsData& RootBaseWithGraphicsData, A3DRootBaseWithGraphics* RootPtr);

CADINTERFACES_API A3DStatus ExportModelFileToPrcFile(const A3DAsmModelFile* ModelFile, const A3DRWParamsExportPrcData* ParamsExportData, const A3DUTF8Char* CADFileName, A3DRWParamsPrcWriteHelper** PrcWriteHelper);

CADINTERFACES_API A3DUns32 InsertGraphRgbColor(const A3DGraphRgbColorData& InRgbColorData);
CADINTERFACES_API A3DUns32 InsertGraphMaterial(const A3DGraphMaterialData& InMaterialData);
CADINTERFACES_API A3DUns32 InsertGraphStyle(const A3DGraphStyleData& InStyleData);

CADINTERFACES_API double GetModelFileUnit(const A3DAsmModelFile* pModelFile);

CADINTERFACES_API A3DStatus AddAttribute(A3DEntity* EntityPtr, const TCHAR*, const TCHAR*);

/**
 * @param ToleranceInCM : The maximum tolerance for the sewing (in cm).
 */
CADINTERFACES_API A3DStatus SewModel(A3DAsmModelFile* ModelPtr, double ToleranceInCM, A3DSewOptionsData const* SewOptions);

/**
 * @param ToleranceInCM : The maximum tolerance for the sewing (in cm).
 */
CADINTERFACES_API A3DStatus SewBReps(A3DRiBrepModel** BRepsToSew, uint32 const BRepCount, double ToleranceInCM, double FileUnit, A3DSewOptionsData const* SewOptions, A3DRiBrepModel*** OutNewBReps, uint32& OutNewBRepCount);

#endif

} // NS TechSoftUtils

/**
 * Class to manage model file lifetime
 */
class CADINTERFACES_API FUniqueTechSoftModelFile
{
public:

	FUniqueTechSoftModelFile()
		: ModelFile(nullptr)
	{
	}

	explicit FUniqueTechSoftModelFile(A3DAsmModelFile* InModelFile)
		: ModelFile(InModelFile)
	{
	}

	~FUniqueTechSoftModelFile()
	{
		Reset();
	}

	/**
	 * Return true if the model is loaded
	 */
	const bool IsValid() const
	{
		return ModelFile != nullptr;
	}

	// Non-copyable
	FUniqueTechSoftModelFile(const FUniqueTechSoftModelFile&) = delete;
	FUniqueTechSoftModelFile& operator=(const FUniqueTechSoftModelFile&) = delete;

	FUniqueTechSoftModelFile(FUniqueTechSoftModelFile&& Old)
		: ModelFile(Old.ModelFile)
	{
		Old.ModelFile = nullptr;
	}

	FUniqueTechSoftModelFile& operator=(FUniqueTechSoftModelFile&& Old)
	{
		Reset();
		ModelFile = Old.ModelFile;
		Old.ModelFile = nullptr;
		return *this;
	}

	A3DAsmModelFile* Get()
	{
		return ModelFile;
	}

	void Reset()
	{
		if (ModelFile != nullptr)
		{
#ifdef USE_TECHSOFT_SDK
			TechSoftInterface::DeleteModelFile(ModelFile);
#endif
			ModelFile = nullptr;
		}
	}

private:
	A3DAsmModelFile* ModelFile;
};

enum EModelerType : uint32
{
	ModelerUnknown = 0,              /*!< User Modeler. */
	ModelerCatia = 2,                /*!< CATIA Modeler. */
	ModelerCatiaV5 = 3,              /*!< CATIA V5 Modeler. */
	ModelerCadds = 4,                /*!< CADDS Modeler. */
	ModelerUnigraphics = 5,          /*!< Unigraphics Modeler. */
	ModelerParasolid = 6,            /*!< Parasolid Modeler. */
	ModelerEuclid = 7,               /*!< Euclid Modeler. */
	ModelerIges = 9,                 /*!< IGES Modeler. */
	ModelerUnisurf = 10,             /*!< Unisurf Modeler. */
	ModelerVda = 11,                 /*!< VDA Modeler. */
	ModelerStl = 12,                 /*!< STL Modeler. */
	ModelerWrl = 13,                 /*!< WRL Modeler. */
	ModelerDxf = 14,                 /*!< DXF Modeler. */
	ModelerAcis = 15,                /*!< ACIS Modeler. */
	ModelerProE = 16,                /*!< Pro/E Modeler. */
	ModelerStep = 18,                /*!< STEP Modeler. */
	ModelerIdeas = 19,               /*!< I-DEAS Modeler. */
	ModelerJt = 20,                  /*!< JT Modeler. */
	ModelerSlw = 22,                 /*!< SolidWorks Modeler. */
	ModelerCgr = 23,                 /*!< CGR Modeler. */
	ModelerPrc = 24,                 /*!< PRC Modeler. */
	ModelerXvl = 25,                 /*!< XVL Modeler. */
	ModelerHpgl = 26,                /*!< HPGL Modeler. */
	ModelerTopSolid = 27,            /*!< TopSolid Modeler. */
	ModelerOneSpaceDesigner = 28,    /*!< OneSpace designer Modeler. */
	Modeler3dxml = 29,               /*!< 3DXML Modeler. */
	ModelerInventor = 30,            /*!< Inventor Modeler. */
	ModelerPostScript = 31,          /*!< Postscript Modeler. */
	ModelerPDF = 32,                 /*!< PDF Modeler. */
	ModelerU3D = 33,                 /*!< U3D Modeler. */
	ModelerIFC = 34,                 /*!< IFC Modeler. */
	ModelerDWG = 35,                 /*!< DWG Modeler. */
	ModelerDWF = 36,                 /*!< DWF Modeler. */
	ModelerSE = 37,                  /*!< SolidEdge Modeler. */
	ModelerOBJ = 38,                 /*!< OBJ Modeler. */
	ModelerKMZ = 39,                 /*!< KMZ Modeler. */
	ModelerDAE = 40,                 /*!< COLLADA Modeler. */
	Modeler3DS = 41,                 /*!< 3DS Modeler. */
	ModelerRhino = 43,               /*!< Rhino Modeler. */
	ModelerXML = 44,                 /*!< XML Modeler. */
	Modeler3mf = 45,                 /*!< 3MF Modeler. */
	ModelerScs = 46,                 /*!< SCS Modeler. */
	Modeler3dHtml = 47,              /*!< 3DHTML Modeler. */
	ModelerHsf = 48,                 /*!< Hsf Modeler. */
	ModelerGltf = 49,                /*!< GL Modeler. */
	ModelerRevit = 50,               /*!< Revit Modeler. */
	ModelerFBX = 51,                 /*!< FBX Modeler. */
	ModelerStepXML = 52,             /*!< StepXML Modeler. */
	ModelerPLMXML = 53,              /*!< PLMXML Modeler. */
	ModelerNavisworks = 54,			/*!< For Future Use: Navisworks Modeler. */
	ModelerLast
};

} // CADLibrary

