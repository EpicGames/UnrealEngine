// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "TechSoftInterface.h"

struct FColor;

typedef void A3DAsmPartDefinition;
typedef void A3DCrvNurbs;
typedef void A3DEntity;
typedef void A3DRiRepresentationItem;
typedef void A3DSurfBase;
typedef void A3DTopoEdge;
typedef void A3DTopoFace;
typedef void A3DTopoShell;

class FJsonObject;

namespace CADLibrary
{

class FImportParameters;
class FBodyMesh;

namespace TechSoftInterface
{
class FTechSoftInterface;
}

namespace TechSoftUtils
{

CADINTERFACES_API bool GetBodyFromPcrFile(const FString& Filename, const FImportParameters& ImportParameters, FBodyMesh& BodyMesh);
CADINTERFACES_API bool FillBodyMesh(void* BodyPtr, const FImportParameters& ImportParameters, double FileUnit, FBodyMesh& BodyMesh);

CADINTERFACES_API FUniqueTechSoftModelFile SaveBodiesToPrcFile(void** Bodies, uint32 BodyCount, const FString& Filename, const FString& JsonString);
CADINTERFACES_API void SaveModelFileToPrcFile(void* ModelFile, const FString& Filename);

CADINTERFACES_API int32 SetEntityGraphicsColor(A3DEntity* InEntity, FColor Color);

CADINTERFACES_API A3DAsmPartDefinition* CreatePart(TArray<A3DRiRepresentationItem*>& RepresentationItems);
CADINTERFACES_API A3DRiRepresentationItem* CreateRIBRep(A3DTopoShell* TopoShellPtr);
CADINTERFACES_API A3DTopoEdge* CreateTopoEdge();
CADINTERFACES_API A3DTopoFace* CreateTopoFaceWithNaturalLoop(A3DSurfBase* CarrierSurface);
CADINTERFACES_API A3DCrvNurbs* CreateTrimNurbsCurve(A3DCrvNurbs* CurveNurbsPtr, double UMin, double UMax, bool bIs2D);

CADINTERFACES_API FColor GetColorAt(uint32 ColorIndex);
CADINTERFACES_API void RestoreMaterials(const TSharedPtr<FJsonObject>& DefaultValues, CADLibrary::FBodyMesh& BodyMesh);

} // NS TechSoftUtils

} // CADLibrary

