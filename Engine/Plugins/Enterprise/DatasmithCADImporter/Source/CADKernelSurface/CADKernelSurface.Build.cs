// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class CADKernelSurface : ModuleRules
	{
		public CADKernelSurface(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CADKernel",
					"CADLibrary",
					"CADTools",
					"CoreUObject",
					"DatasmithContent",
					"DatasmithCore",
					"DatasmithTranslator",
					"Engine",
					"MeshDescription",
					"ParametricSurface",
					"StaticMeshDescription",
				}
			);
		}
	}
}
