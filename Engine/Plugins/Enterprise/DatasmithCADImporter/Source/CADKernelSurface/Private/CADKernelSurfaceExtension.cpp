// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelSurfaceExtension.h"

#include "CADKernelTools.h"
#include "CADData.h"
#include "CADOptions.h"
#include "DatasmithPayload.h"
#include "Engine/StaticMesh.h"
#include "MeshDescription.h"
#include "CADMeshDescriptionHelper.h"
#include "Misc/FileHelper.h"
#include "StaticMeshAttributes.h"
#include "UObject/EnterpriseObjectVersion.h"

#include "Core/CADKernelArchive.h"
#include "Core/CADEntity.h"
#include "Core/Session.h"
#include "Core/Types.h"
#include "Mesh/Structure/ModelMesh.h"
#include "Topo/Model.h"
#include "Topo/Body.h"

void UCADKernelParametricSurfaceData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEnterpriseObjectVersion::GUID);
	Super::Serialize(Ar);
	Ar << RawData;
}

bool UCADKernelParametricSurfaceData::Tessellate(UStaticMesh& StaticMesh, const FDatasmithRetessellationOptions& RetessellateOptions)
{
	using namespace UE::CADKernel;

	bool bSuccessfulTessellation = false;

#if WITH_EDITOR
	CADLibrary::FImportParameters ImportParameters;
	ImportParameters.SetModelCoordinateSystem((FDatasmithUtils::EModelCoordSystem) SceneParameters.ModelCoordSys);
	ImportParameters.SetTesselationParameters(RetessellateOptions.ChordTolerance, RetessellateOptions.MaxEdgeLength, RetessellateOptions.NormalTolerance, (CADLibrary::EStitchingTechnique) RetessellateOptions.StitchingTechnique);

	CADLibrary::FMeshParameters CadMeshParameters;
	CadMeshParameters.bNeedSwapOrientation = MeshParameters.bNeedSwapOrientation;
	CadMeshParameters.bIsSymmetric = MeshParameters.bIsSymmetric;
	CadMeshParameters.SymmetricNormal = (FVector3f) MeshParameters.SymmetricNormal;
	CadMeshParameters.SymmetricOrigin = (FVector3f) MeshParameters.SymmetricOrigin;


	// Previous MeshDescription is get to be able to create a new one with the same order of PolygonGroup (the matching of color and partition is currently based on their order)
	if (FMeshDescription* DestinationMeshDescription = StaticMesh.GetMeshDescription(0))
	{
		FMeshDescription MeshDescription;
		FStaticMeshAttributes MeshDescriptionAttributes(MeshDescription);
		MeshDescriptionAttributes.Register();

		TSharedRef<FSession> CADKernelSession = MakeShared<FSession>(RetessellateOptions.GetGeometricTolerance(true));
		CADKernelSession->AddDatabase(RawData);

		FModel& CADKernelModel = CADKernelSession->GetModel();
		if (CADKernelModel.GetBodies().IsEmpty())
		{
			return bSuccessfulTessellation;
		}

		CADLibrary::FMeshConversionContext MeshConversionContext(ImportParameters, CadMeshParameters, CADKernelSession->GetGeometricTolerance());

		if (RetessellateOptions.RetessellationRule == EDatasmithCADRetessellationRule::SkipDeletedSurfaces)
		{
			CADLibrary::GetExistingPatches(*DestinationMeshDescription, MeshConversionContext.PatchesToMesh);
		}

		if(CADLibrary::FCADKernelTools::Tessellate(CADKernelModel, MeshConversionContext, MeshDescription))
		{
			// To update the SectionInfoMap 
			TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();
			FMeshSectionInfoMap& SectionInfoMap = StaticMesh.GetSectionInfoMap();

			for (FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
			{
				FMeshSectionInfo Section = SectionInfoMap.Get(0, PolygonGroupID.GetValue());
				int32 MaterialIndex = StaticMesh.GetMaterialIndex(MaterialSlotNames[PolygonGroupID]);
				if (MaterialIndex < 0)
				{
					MaterialIndex = 0;
				}
				Section.MaterialIndex = MaterialIndex;
				SectionInfoMap.Set(0, PolygonGroupID.GetValue(), Section);
			}
			*DestinationMeshDescription = MoveTemp(MeshDescription);
			bSuccessfulTessellation = true;
		}
	}
#endif 

	return bSuccessfulTessellation;
}

namespace CADKernelSurface
{
	void AddSurfaceDataForMesh(const TCHAR* CADKernelArchive, const CADLibrary::FImportParameters& InSceneParameters, const CADLibrary::FMeshParameters& InMeshParameters, const FDatasmithTessellationOptions& InTessellationOptions, FDatasmithMeshElementPayload& OutMeshPayload)
	{
		// Store CADKernel archive if provided
		UCADKernelParametricSurfaceData* CADKernelData = Datasmith::MakeAdditionalData<UCADKernelParametricSurfaceData>();
		if (CADKernelData->SetFile(CADKernelArchive))
		{
			CADKernelData->SetImportParameters(InSceneParameters);
			CADKernelData->SetMeshParameters(InMeshParameters);
			CADKernelData->SetLastTessellationOptions(InTessellationOptions);

			OutMeshPayload.AdditionalData.Add(CADKernelData);
		}
	}

}
