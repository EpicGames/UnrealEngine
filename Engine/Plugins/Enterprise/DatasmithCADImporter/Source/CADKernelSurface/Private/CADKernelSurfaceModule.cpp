// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelSurfaceModule.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/CoreRedirects.h"

#define LOCTEXT_NAMESPACE "CADKernelSurfaceModule"

void FCADKernelSurfaceModule::StartupModule()
{
}

FCADKernelSurfaceModule& FCADKernelSurfaceModule::Get()
{
	return FModuleManager::LoadModuleChecked< FCADKernelSurfaceModule >(CADKERNELSURFACE_MODULE_NAME);
}

bool FCADKernelSurfaceModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(CADKERNELSURFACE_MODULE_NAME);
}

IMPLEMENT_MODULE(FCADKernelSurfaceModule, CADKernelSurface);

#undef LOCTEXT_NAMESPACE

