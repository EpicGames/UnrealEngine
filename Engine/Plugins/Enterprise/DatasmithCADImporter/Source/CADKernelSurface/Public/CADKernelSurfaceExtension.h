// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithParametricSurfaceData.h"
#include "DatasmithAdditionalData.h"
#include "DatasmithCustomAction.h"
#include "DatasmithImportOptions.h"
#include "DatasmithUtils.h"

#include "CADKernelSurfaceExtension.generated.h"

UCLASS(meta = (DisplayName = "CADKernel Parametric Surface Data"))
class CADKERNELSURFACE_API UCADKernelParametricSurfaceData : public UDatasmithParametricSurfaceData
{
	GENERATED_BODY()

public:
	virtual bool Tessellate(UStaticMesh& StaticMesh, const FDatasmithRetessellationOptions& RetessellateOptions) override;

private:

	virtual void Serialize(FArchive& Ar) override;
};

struct FDatasmithMeshElementPayload;

namespace CADLibrary
{
	class FImportParameters;
	struct FMeshParameters;
}

namespace CADKernelSurface
{
	void CADKERNELSURFACE_API AddSurfaceDataForMesh(const TCHAR* CADKernelArchive, const CADLibrary::FImportParameters& InSceneParameters, const CADLibrary::FMeshParameters&, const FDatasmithTessellationOptions& InTessellationOptions, FDatasmithMeshElementPayload& OutMeshPayload);
}
