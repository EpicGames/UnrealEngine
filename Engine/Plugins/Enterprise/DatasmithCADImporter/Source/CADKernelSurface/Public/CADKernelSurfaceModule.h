// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#define CADKERNELSURFACE_MODULE_NAME TEXT("CADKernelSurface")

/**
 * This module exposes additional features for assets containing CADKernel data.
 */
class FCADKernelSurfaceModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;

	static FCADKernelSurfaceModule& Get();
	static bool IsAvailable();
};
