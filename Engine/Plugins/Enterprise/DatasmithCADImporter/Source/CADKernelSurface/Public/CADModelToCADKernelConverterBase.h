// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithImportOptions.h"
#include "CADKernelSurfaceExtension.h"
#include "CADKernelTools.h"
#include "CADModelConverter.h"
#include "CADOptions.h"
#include "IDatasmithSceneElements.h"
#include "CADMeshDescriptionHelper.h"

#include "Core/Session.h"
#include "Topo/Model.h"
#include "Topo/Topomaker.h"

struct FDatasmithMeshElementPayload;
struct FDatasmithTessellationOptions;

class FCADModelToCADKernelConverterBase : public CADLibrary::ICADModelConverter
{
public:

	FCADModelToCADKernelConverterBase(const CADLibrary::FImportParameters& InImportParameters)
		: CADKernelSession(0.01)
		, ImportParameters(InImportParameters)
	{
	}

	virtual void InitializeProcess() override
	{
		CADKernelSession.Clear();
	}

	virtual bool RepairTopology() override
	{
		using namespace CADLibrary;
		// Apply stitching if applicable
		if(ImportParameters.GetStitchingTechnique() != StitchingNone)
		{
			UE::CADKernel::FTopomakerOptions TopomakerOptions((UE::CADKernel::ESewOption)SewOption::GetFromImportParameters(), StitchingTolerance, FImportParameters::GStitchingForceFactor);

			UE::CADKernel::FTopomaker Topomaker(CADKernelSession, TopomakerOptions);
			Topomaker.Sew();
			Topomaker.SplitIntoConnectedShells();
			Topomaker.OrientShells();
		}

		return true;
	}

	virtual bool SaveModel(const TCHAR* InFolderPath, TSharedPtr<IDatasmithMeshElement> MeshElement) override
	{
		if (MeshElement.IsValid())
		{
			FString FilePath = FPaths::Combine(InFolderPath, MeshElement->GetName()) + TEXT(".ugeom");
			CADKernelSession.SaveDatabase(*FilePath);
			MeshElement->SetFile(*FilePath);
			return true;
		}

		return false;
	}

	virtual bool Tessellate(const CADLibrary::FMeshParameters& InMeshParameters, FMeshDescription& OutMeshDescription) override
	{
		UE::CADKernel::FModel& Model = CADKernelSession.GetModel();
		
		CADLibrary::FMeshConversionContext Context(ImportParameters, InMeshParameters, CADKernelSession.GetGeometricTolerance());

		return CADLibrary::FCADKernelTools::Tessellate(Model, Context, OutMeshDescription);
	}

	virtual void SetImportParameters(double ChordTolerance, double MaxEdgeLength, double NormalTolerance, CADLibrary::EStitchingTechnique StitchingTechnique) override
	{
		ImportParameters.SetTesselationParameters(ChordTolerance, MaxEdgeLength, NormalTolerance, StitchingTechnique);
	}

	virtual bool IsSessionValid() override
	{
		return true;
	}

	virtual bool AddGeometry(const CADLibrary::FCADModelGeometry& Geometry) override { return false; }

	virtual void AddSurfaceDataForMesh(const TCHAR* InFilePath, const CADLibrary::FMeshParameters& InMeshParameters, const FDatasmithTessellationOptions& InTessellationOptions, FDatasmithMeshElementPayload& OutMeshPayload) const override
	{
		CADKernelSurface::AddSurfaceDataForMesh(InFilePath, ImportParameters, InMeshParameters, InTessellationOptions, OutMeshPayload);
	}

protected:
	void SetTolerances(double InGeometryTolerance = 0.01, double InStitchingTolerance = 0.01)
	{
		GeometricTolerance = InGeometryTolerance;
		SquareTolerance = GeometricTolerance * GeometricTolerance;
		StitchingTolerance = InStitchingTolerance;
		EdgeLengthTolerance = 2. * GeometricTolerance;
		CADKernelSession.SetGeometricTolerance(InGeometryTolerance);
	}

protected:

	UE::CADKernel::FSession CADKernelSession;

	CADLibrary::FImportParameters ImportParameters;
	double GeometricTolerance = 0.01;
	double SquareTolerance = 0.0001;
	double EdgeLengthTolerance = 0.02;
	double StitchingTolerance = 0.01;
};

