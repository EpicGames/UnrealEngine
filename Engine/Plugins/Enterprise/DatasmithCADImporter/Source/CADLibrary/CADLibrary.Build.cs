// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;

public class CADLibrary : ModuleRules
{
	public CADLibrary(ReadOnlyTargetRules Target) : base(Target)
	{
		bLegalToDistributeObjectCode = true;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CADTools",
				"CADKernel",
				"DatasmithCore",
				"DynamicMesh",
				"MeshConversion",
				"MeshDescription",
				"GeometryCore",
				"StaticMeshDescription"
			}
		);
	}
}
