// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADKernelTools.h"

#include "CADData.h"
#include "CADOptions.h"
#include "MeshDescription.h"
#include "CADMeshDescriptionHelper.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#include "Core/CADEntity.h"
#include "Core/MetadataDictionary.h"
#include "Core/Session.h"
#include "Core/Types.h"

#include "Mesh/Criteria/Criterion.h"
#include "Mesh/Meshers/Mesher.h"
#include "Mesh/Structure/FaceMesh.h"
#include "Mesh/Structure/ModelMesh.h"

#include "Topo/Model.h"
#include "Topo/Body.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalShapeEntity.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalVertex.h"

typedef uint32 TriangleIndex[3];

namespace CADLibrary
{

static void FillVertexPosition(FMeshConversionContext& Context, UE::CADKernel::FModelMesh& ModelMesh, FMeshDescription& MeshDescription)
{
	TArray<FVector3f> VertexArray;
	ModelMesh.GetNodeCoordinates(VertexArray);

	for (FVector3f& Vertex : VertexArray)
	{
		Vertex *= 0.1f; // mm (CADKernel unit) to cm (UE unit)
	}

	int32 VertexCount = VertexArray.Num();

	TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();
	MeshDescription.ReserveNewVertices(Context.MeshParameters.bIsSymmetric ? VertexCount * 2 : VertexCount);

	Context.VertexIds.SetNumZeroed(VertexCount);

	// Make MeshDescription.VertexPositions and VertexID
	int32 VertexIndex = -1;
	for (const FVector3f& Vertex : VertexArray)
	{
		VertexIndex++;

		FVertexID VertexID = MeshDescription.CreateVertex();
		VertexPositions[VertexID] = FDatasmithUtils::ConvertVector((FDatasmithUtils::EModelCoordSystem)Context.ImportParameters.GetModelCoordSys(), Vertex);
		Context.VertexIds[VertexIndex] = VertexID;
	}

	// if Symmetric mesh, the symmetric side of the mesh have to be generated
	if (Context.MeshParameters.bIsSymmetric)
	{
		FMatrix44f SymmetricMatrix = FDatasmithUtils::GetSymmetricMatrix(Context.MeshParameters.SymmetricOrigin, Context.MeshParameters.SymmetricNormal);

		Context.SymmetricVertexIds.SetNum(VertexArray.Num());

		VertexIndex = 0;
		for (const FVector3f& Vertex : VertexArray)
		{
			FVertexID VertexID = MeshDescription.CreateVertex();
			VertexPositions[VertexID] = FDatasmithUtils::ConvertVector((FDatasmithUtils::EModelCoordSystem)Context.ImportParameters.GetModelCoordSys(), Vertex);
			VertexPositions[VertexID] = SymmetricMatrix.TransformPosition(VertexPositions[VertexID]);
			Context.SymmetricVertexIds[VertexIndex++] = VertexID;
		}
	}
}

bool FillMesh(FMeshConversionContext& Context, UE::CADKernel::FModelMesh& ModelMesh, FMeshDescription& MeshDescription)
{
	using namespace UE::CADKernel;

	const int32 UVChannel = 0;
	const int32 TriangleCount = 3;
	const TriangleIndex Clockwise = { 0, 1, 2 };
	const TriangleIndex CounterClockwise = { 0, 2, 1 };

	TArray<FVertexInstanceID> TriangleVertexInstanceIDs;
	TriangleVertexInstanceIDs.SetNum(TriangleCount);

	TArray<FVertexInstanceID> MeshVertexInstanceIDs;

	// Gather all array data
	FStaticMeshAttributes Attributes(MeshDescription);
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

	if (!VertexInstanceNormals.IsValid() || !VertexInstanceTangents.IsValid() || !VertexInstanceBinormalSigns.IsValid() || !VertexInstanceColors.IsValid() || !VertexInstanceUVs.IsValid() || !PolygonGroupImportedMaterialSlotNames.IsValid())
	{
		return false;
	}

	// Find all the materials used
	TMap<uint32, FPolygonGroupID> MaterialToPolygonGroupMapping;
	for (const FFaceMesh* FaceMesh : ModelMesh.GetFaceMeshes())
	{
		const FTopologicalFace& Face = (const FTopologicalFace&) FaceMesh->GetGeometricEntity();
		uint32 MaterialId = Face.GetMaterialId() ? Face.GetMaterialId() : Face.GetColorId();
		MaterialToPolygonGroupMapping.Add(MaterialId, INDEX_NONE);
	}

	// Add to the mesh, a polygon groups per material
	for (auto& Material : MaterialToPolygonGroupMapping)
	{
		uint32 MaterialHash = Material.Key;
		FName ImportedSlotName = *LexToString(MaterialHash);

		FPolygonGroupID PolyGroupID = MeshDescription.CreatePolygonGroup();
		PolygonGroupImportedMaterialSlotNames[PolyGroupID] = ImportedSlotName;
		Material.Value = PolyGroupID;
	}

	VertexInstanceUVs.SetNumChannels(1);

	int32 NbStep = 1;
	if (Context.MeshParameters.bIsSymmetric)
	{
		NbStep = 2;
	}

	int32 FaceIndex = 0;

	const TSet<int32>& PatchIdSet = Context.PatchesToMesh;
	bool bImportOnlyAlreadyPresent = (bool)PatchIdSet.Num();

	TPolygonAttributesRef<int32> PatchGroups = EnableCADPatchGroups(MeshDescription);

	int32 PatchIndex = 0;
	for (int32 Step = 0; Step < NbStep; ++Step)
	{
		// Swap mesh if needed
		const TriangleIndex& Orientation = (!Context.MeshParameters.bNeedSwapOrientation == (bool)Step) ? CounterClockwise : Clockwise;
		TArray<int32>& VertexIdSet = (Step == 0) ? Context.VertexIds : Context.SymmetricVertexIds;

		// Loop through the FaceMeshes and collect all tessellation data
		for (FFaceMesh* FaceMesh : ModelMesh.GetFaceMeshes())
		{
			const FTopologicalFace& Face = (const FTopologicalFace&)FaceMesh->GetGeometricEntity();
			if (bImportOnlyAlreadyPresent && !PatchIdSet.Contains(Face.GetPatchId()))
			{
				continue;
			}

			// Get the polygonGroup
			const FPolygonGroupID* PolygonGroupID = MaterialToPolygonGroupMapping.Find(Face.GetColorId());
			if (PolygonGroupID == nullptr)
			{
				continue;
			}

			int32 VertexIDs[3];
			//FVector Temp3D = { 0, 0, 0 };
			//FVector2D TexCoord2D = { 0, 0 };

			const TArray<int32>& TriangleVertexIndices = FaceMesh->TrianglesVerticesIndex;
			const TArray<int32>& VerticesGlobalIndex = FaceMesh->VerticesGlobalIndex;
			MeshVertexInstanceIDs.Empty(TriangleVertexIndices.Num());

			PatchIndex++;

			// build each valid face i.e. 3 different indexes
			for (int32 Index = 0; Index < TriangleVertexIndices.Num(); Index += 3)
			{
				VertexIDs[0] = VertexIdSet[VerticesGlobalIndex[TriangleVertexIndices[Index + Orientation[0]]]];
				VertexIDs[1] = VertexIdSet[VerticesGlobalIndex[TriangleVertexIndices[Index + Orientation[1]]]];
				VertexIDs[2] = VertexIdSet[VerticesGlobalIndex[TriangleVertexIndices[Index + Orientation[2]]]];

				MeshVertexInstanceIDs.Add(TriangleVertexInstanceIDs[0] = MeshDescription.CreateVertexInstance((FVertexID)VertexIDs[0]));
				MeshVertexInstanceIDs.Add(TriangleVertexInstanceIDs[1] = MeshDescription.CreateVertexInstance((FVertexID)VertexIDs[1]));
				MeshVertexInstanceIDs.Add(TriangleVertexInstanceIDs[2] = MeshDescription.CreateVertexInstance((FVertexID)VertexIDs[2]));

				// Add the triangle as a polygon to the mesh description
				const FPolygonID PolygonID = MeshDescription.CreatePolygon(*PolygonGroupID, TriangleVertexInstanceIDs);

				// Set patch id attribute
				PatchGroups[PolygonID] = Face.GetPatchId();
			}

			for (int32 IndexFace = 0; IndexFace < MeshVertexInstanceIDs.Num(); IndexFace += 3)
			{
				for (int32 Index = 0; Index < TriangleCount; Index++)
				{
					const FVertexInstanceID VertexInstanceID = MeshVertexInstanceIDs[IndexFace + Index];

					// TODO
					// The UV scaling should be done in CADKernel
					const double ScaleUV = 0.001; // mm to m
					VertexInstanceUVs.Set(VertexInstanceID, UVChannel, FVector2f(FaceMesh->UVMap[TriangleVertexIndices[IndexFace + Orientation[Index]]] * static_cast<float>(ScaleUV)));

					VertexInstanceColors[VertexInstanceID] = FLinearColor::White;
					VertexInstanceTangents[VertexInstanceID] = FVector3f(ForceInitToZero);
					VertexInstanceBinormalSigns[VertexInstanceID] = 0.0f;
				}
			}

			if (!Step)
			{
				FDatasmithUtils::ConvertVectorArray(Context.ImportParameters.GetModelCoordSys(), FaceMesh->Normals);
				for (FVector3f& Normal : FaceMesh->Normals)
				{
					Normal = Normal.GetSafeNormal();
				}
			}

			for (int32 IndexFace = 0; IndexFace < MeshVertexInstanceIDs.Num(); IndexFace += 3)
			{
				for (int32 Index = 0; Index < 3; Index++)
				{
					const FVertexInstanceID VertexInstanceID = MeshVertexInstanceIDs[IndexFace + Index];
					VertexInstanceNormals[VertexInstanceID] = (FVector3f)FaceMesh->Normals[TriangleVertexIndices[IndexFace + Orientation[Index]]];
				}
			}

			// compute normals
			if (Step)
			{
				// compute normals of Symmetric vertex
				FMatrix44f SymmetricMatrix = FDatasmithUtils::GetSymmetricMatrix(Context.MeshParameters.SymmetricOrigin, Context.MeshParameters.SymmetricNormal);
				for (const FVertexInstanceID& VertexInstanceID : MeshVertexInstanceIDs)
				{
					VertexInstanceNormals[VertexInstanceID] = SymmetricMatrix.TransformVector(VertexInstanceNormals[VertexInstanceID]);
				}
			}
		}
	}
	return true;
}

static bool ConvertModelMeshToMeshDescription(FMeshConversionContext& Context, UE::CADKernel::FModelMesh& InModelMesh, FMeshDescription& MeshDescription)
{
	int32 VertexCount = InModelMesh.GetVertexCount();
	int32 TriangleCount = InModelMesh.GetTriangleCount();

	MeshDescription.ReserveNewVertexInstances(VertexCount);
	MeshDescription.ReserveNewPolygons(TriangleCount);
	MeshDescription.ReserveNewEdges(TriangleCount);

	FillVertexPosition(Context, InModelMesh, MeshDescription);
	if (!FillMesh(Context, InModelMesh, MeshDescription))
	{
		return false;
	}

	// Build edge meta data
	FStaticMeshOperations::DetermineEdgeHardnessesFromVertexInstanceNormals(MeshDescription);

	return MeshDescription.Polygons().Num() > 0;
}

bool FCADKernelTools::Tessellate(UE::CADKernel::FTopologicalShapeEntity& CADTopologicalEntity, FMeshConversionContext& TessellationContext, FMeshDescription& OutMeshDescription)
{
	using namespace UE::CADKernel;

	// Tessellate the model
	TSharedRef<FModelMesh> CADKernelModelMesh = FEntity::MakeShared<FModelMesh>();

	const bool bActivateThinZoneMeshing = FImportParameters::bGActivateThinZoneMeshing;
	FMesher Mesher(*CADKernelModelMesh, TessellationContext.GeometricTolerance, bActivateThinZoneMeshing);

	DefineMeshCriteria(*CADKernelModelMesh, TessellationContext.ImportParameters, TessellationContext.GeometricTolerance);
	Mesher.MeshEntity(CADTopologicalEntity);

	return ConvertModelMeshToMeshDescription(TessellationContext, *CADKernelModelMesh, OutMeshDescription);
}

uint32 FCADKernelTools::GetFaceTessellation(UE::CADKernel::FFaceMesh& FaceMesh, FBodyMesh& OutBodyMesh, FObjectDisplayDataId FaceMaterial)
{
	// Something wrong happened, either an error or no data to collect
	if (FaceMesh.TrianglesVerticesIndex.Num() == 0)
	{
		return 0;
	}

	FTessellationData& Tessellation = OutBodyMesh.Faces.Emplace_GetRef();

	const UE::CADKernel::FTopologicalFace& TopologicalFace = (const UE::CADKernel::FTopologicalFace&)FaceMesh.GetGeometricEntity();
	Tessellation.PatchId = TopologicalFace.GetPatchId();

	Tessellation.PositionIndices = MoveTemp(FaceMesh.VerticesGlobalIndex);
	Tessellation.VertexIndices = MoveTemp(FaceMesh.TrianglesVerticesIndex);

	Tessellation.NormalArray = MoveTemp(FaceMesh.Normals);
	Tessellation.TexCoordArray = MoveTemp(FaceMesh.UVMap);

	if (FaceMaterial.ColorUId)
	{
		Tessellation.ColorUId = FaceMaterial.ColorUId;
		OutBodyMesh.ColorSet.Add(FaceMaterial.ColorUId);
	}

	if (FaceMaterial.MaterialUId)
	{
		Tessellation.MaterialUId = FaceMaterial.MaterialUId;
		OutBodyMesh.MaterialSet.Add(FaceMaterial.MaterialUId);
	}

	return Tessellation.VertexIndices.Num() / 3;
}

void GetDisplayDataIds(const UE::CADKernel::FTopologicalShapeEntity& ShapeEntity, FObjectDisplayDataId& DisplayDataId)
{
	if (ShapeEntity.GetColorId() != 0)
	{
		DisplayDataId.ColorUId = ShapeEntity.GetColorId();
	}
	if (ShapeEntity.GetMaterialId() != 0)
	{
		DisplayDataId.MaterialUId = ShapeEntity.GetMaterialId();
	}
}

void FCADKernelTools::GetBodyTessellation(const UE::CADKernel::FModelMesh& ModelMesh, const UE::CADKernel::FBody& Body, FBodyMesh& OutBodyMesh)
{
	ModelMesh.GetNodeCoordinates(OutBodyMesh.VertexArray);

	for (FVector3f& Vertex : OutBodyMesh.VertexArray)
	{
		Vertex *= 0.1f;
	}

	uint32 FaceSize = Body.FaceCount();

	// Allocate memory space for tessellation data
	OutBodyMesh.Faces.Reserve(FaceSize);
	OutBodyMesh.ColorSet.Reserve(FaceSize);
	OutBodyMesh.MaterialSet.Reserve(FaceSize);

	FObjectDisplayDataId BodyMaterial;

	GetDisplayDataIds(Body, BodyMaterial);
	BodyMaterial.DefaultMaterialUId = BodyMaterial.MaterialUId != 0 ? BodyMaterial.MaterialUId : BodyMaterial.ColorUId;

	// Loop through the face of bodies and collect all tessellation data
	for (const TSharedPtr<UE::CADKernel::FShell>& Shell : Body.GetShells())
	{
		if (!Shell.IsValid())
		{
			continue;
		}

		FObjectDisplayDataId ShellMaterial = BodyMaterial;
		GetDisplayDataIds(*Shell, ShellMaterial);

		for (const UE::CADKernel::FOrientedFace& Face : Shell->GetFaces())
		{
			if (!Face.Entity.IsValid())
			{
				continue;
			}

			if (!Face.Entity->HasTesselation())
			{
				continue;
			}

			FObjectDisplayDataId FaceMaterial = ShellMaterial;
			GetDisplayDataIds(*Face.Entity, FaceMaterial);

			UE::CADKernel::FFaceMesh* FaceMesh = Face.Entity->GetMesh();
			uint32 TriangleNum = GetFaceTessellation(*FaceMesh, OutBodyMesh, FaceMaterial);
			OutBodyMesh.TriangleCount += TriangleNum;
		}
	}
}

void FCADKernelTools::DefineMeshCriteria(UE::CADKernel::FModelMesh& MeshModel, const FImportParameters& ImportParameters, double GeometricTolerance)
{
	{
		TSharedPtr<UE::CADKernel::FCriterion> CurvatureCriterion = UE::CADKernel::FCriterion::CreateCriterion(UE::CADKernel::ECriterion::CADCurvature);
		MeshModel.AddCriterion(CurvatureCriterion);

		TSharedPtr<UE::CADKernel::FCriterion> MinSizeCriterion = UE::CADKernel::FCriterion::CreateCriterion(UE::CADKernel::ECriterion::MinSize, 2. * GeometricTolerance);
		MeshModel.AddCriterion(MinSizeCriterion);
	}

	if (ImportParameters.GetMaxEdgeLength() > SMALL_NUMBER)
	{
		const double MaxSize = ImportParameters.GetMaxEdgeLength() * 10.; // cm to mm
		TSharedPtr<UE::CADKernel::FCriterion> MaxSizeCriterion = UE::CADKernel::FCriterion::CreateCriterion(UE::CADKernel::ECriterion::MaxSize, MaxSize);
		MeshModel.AddCriterion(MaxSizeCriterion);
	}

	if (ImportParameters.GetChordTolerance() > SMALL_NUMBER)
	{
		double MaxChord = ImportParameters.GetChordTolerance() * 10.; // cm to mm
		TSharedPtr<UE::CADKernel::FCriterion> ChordCriterion = UE::CADKernel::FCriterion::CreateCriterion(UE::CADKernel::ECriterion::Sag, MaxChord);
		MeshModel.AddCriterion(ChordCriterion);
	}

	if (ImportParameters.GetMaxNormalAngle() > SMALL_NUMBER)
	{
		TSharedPtr<UE::CADKernel::FCriterion> MaxNormalAngleCriterion = UE::CADKernel::FCriterion::CreateCriterion(UE::CADKernel::ECriterion::Angle, ImportParameters.GetMaxNormalAngle());
		MeshModel.AddCriterion(MaxNormalAngleCriterion);
	}
}

}

