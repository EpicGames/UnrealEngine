// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FCADLibraryModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// #ueent_todo: startup external module (Coretech)
		// (currently in CADTranslator)
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FCADLibraryModule, CADLibrary);
