// Copyright Epic Games, Inc. All Rights Reserved.
#include "CADMeshDescriptionHelper.h"

#include "Algo/AnyOf.h"
#include "CADData.h"
#include "CADOptions.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "Math/UnrealMathUtility.h"
#include "MeshDescription.h"
#include "MeshOperator.h"
#include "Misc/FileHelper.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

typedef uint32 TriangleIndex[3];

DEFINE_LOG_CATEGORY_STATIC(LogCADLibrary, Log, All);

namespace CADLibrary
{

template<typename VectorType>
struct TGeometricData
{
	float W;
	int32 Index;
	VectorType Coordinates;
	bool bIsMerged;

	/** Default constructor. */
	TGeometricData() {}

	/** Initialization constructor. */
	TGeometricData(int32 InIndex, const VectorType& V, const VectorType& OneVector)
	{
		W = V | OneVector;  // V.X + V.Y or V.X + V.Y + V.Z according to the vector dimension
		Index = InIndex;
		Coordinates = V;
		bIsMerged = false;
	}
};

// Verify the 3 input indices are not defining a degenerated triangle and fill up the corresponding FVertexIDs
bool IsTriangleDegenerated(const int32_t* Indices, const TArray<FVertexID>& RemapVertexPosition, FVertexID VertexIDs[3])
{
	if (Indices[0] == Indices[1] || Indices[0] == Indices[2] || Indices[1] == Indices[2])
	{
		return true;
	}

	for (int32 Corner = 0; Corner < 3; ++Corner)
	{
		VertexIDs[Corner] = RemapVertexPosition[Indices[Corner]];
	}

	return (VertexIDs[0] == VertexIDs[1] || VertexIDs[0] == VertexIDs[2] || VertexIDs[1] == VertexIDs[2]);
}

template<typename VectorType>
void MergeCoincidents(const TArray<VectorType>& DataArray, const VectorType& OneVector, const double CoincidenceTolerance, TArray<int32>& InOutIndices)
{
	// Create a list of Data W/index pairs
	TArray<TGeometricData<VectorType>> GeometicDataArray;
	GeometicDataArray.Reserve(DataArray.Num());

	const VectorType* Data = DataArray.GetData();
	const int32* DataId = InOutIndices.GetData();
	for (int32 Index = 0; Index < DataArray.Num(); ++Index, ++Data, ++DataId)
	{
		GeometicDataArray.Emplace(*DataId, *Data, OneVector);
	}

	// Sort the vertices by z value
	GeometicDataArray.Sort([](TGeometricData<VectorType> const& A, TGeometricData<VectorType> const& B) { return A.W < B.W; });

	// Search for duplicates
	for (int32 Index = 0; Index < GeometicDataArray.Num(); Index++)
	{
		if (GeometicDataArray[Index].bIsMerged)
		{
			continue;
		}

		GeometicDataArray[Index].bIsMerged = true;
		int32 NewIndex = GeometicDataArray[Index].Index;

		const VectorType& PositionA = GeometicDataArray[Index].Coordinates;

		// only need to search forward, since we add pairs both ways
		for (int32 Bndex = Index + 1; Bndex < GeometicDataArray.Num(); Bndex++)
		{
			if ((GeometicDataArray[Bndex].W - GeometicDataArray[Index].W) > CoincidenceTolerance)
			{
				break; // can't be any more duplicated
			}

			const VectorType& PositionB = GeometicDataArray[Bndex].Coordinates;
			if (PositionA.Equals(PositionB, CoincidenceTolerance))
			{
				GeometicDataArray[Bndex].bIsMerged = true;
				InOutIndices[GeometicDataArray[Bndex].Index] = NewIndex;
			}
		}
	}
}

void FillVertexPosition(const FMeshConversionContext& MeshConversionContext, FBodyMesh& Body, FMeshDescription& MeshDescription)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CADLibrary::FillVertexPosition);

	int32 TriangleCount = Body.TriangleCount;
	TArray<FTessellationData>& FaceTessellationSet = Body.Faces;

	TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();

	TArray<FVector3f>& VertexArray = Body.VertexArray;
	TArray<int32>& VertexIdSet = Body.VertexIds;
	int32 VertexCount = VertexArray.Num();
	VertexIdSet.SetNumZeroed(VertexCount);

	// Make MeshDescription.VertexPositions and VertexID
	MeshDescription.ReserveNewVertices(MeshConversionContext.MeshParameters.bIsSymmetric ? VertexCount * 2 : VertexCount);

	int32 VertexIndex = -1;
	for (const FVector3f& Vertex : VertexArray)
	{
		VertexIndex++;

		// Vertex is outside bbox
		if (VertexIdSet[VertexIndex] == INDEX_NONE)
		{
			continue;
		}

		FVertexID VertexID = MeshDescription.CreateVertex();
		VertexPositions[VertexID] = (FVector3f)FDatasmithUtils::ConvertVector((FDatasmithUtils::EModelCoordSystem)MeshConversionContext.ImportParameters.GetModelCoordSys(), Vertex);
		VertexIdSet[VertexIndex] = VertexID;
	}

	MergeCoincidents(VertexArray, FVector3f::OneVector, (float)MeshConversionContext.GeometricTolerance, VertexIdSet);

	// if Symmetric mesh, the symmetric side of the mesh have to be generated
	if (MeshConversionContext.MeshParameters.bIsSymmetric)
	{
		FMatrix44f SymmetricMatrix = (FMatrix44f) FDatasmithUtils::GetSymmetricMatrix(MeshConversionContext.MeshParameters.SymmetricOrigin, MeshConversionContext.MeshParameters.SymmetricNormal);

		TArray<int32>& SymmetricVertexIds = Body.SymmetricVertexIds;
		SymmetricVertexIds.SetNum(VertexArray.Num());

		VertexIndex = 0;
		for (const FVector3f& Vertex : VertexArray)
		{
			if (VertexIdSet[VertexIndex] == INDEX_NONE)
			{
				SymmetricVertexIds[VertexIndex++] = INDEX_NONE;
				continue;
			}

			FVertexID VertexID = MeshDescription.CreateVertex();
			const FVector3f VertexPosition = FDatasmithUtils::ConvertVector((FDatasmithUtils::EModelCoordSystem)MeshConversionContext.ImportParameters.GetModelCoordSys(), Vertex);
			const FVector4f SymmetricPosition = FVector4f(SymmetricMatrix.TransformPosition(VertexPosition));
			VertexPositions[VertexID] = FVector3f(SymmetricPosition);
			SymmetricVertexIds[VertexIndex++] = VertexID;
		}
	}
}

// PolygonAttributes name used into modeling tools (ExtendedMeshAttribute::PolyTriGroups)
const FName PolyTriGroups("PolyTriGroups");

// Copy of FMeshDescriptionBuilder::EnablePolyGroups()
TPolygonAttributesRef<int32> EnableCADPatchGroups(FMeshDescription& OutMeshDescription)
{
	TPolygonAttributesRef<int32> PatchGroups = OutMeshDescription.PolygonAttributes().GetAttributesRef<int32>(PolyTriGroups);
	if (PatchGroups.IsValid() == false)
	{
		OutMeshDescription.PolygonAttributes().RegisterAttribute<int32>(PolyTriGroups, 1, 0, EMeshAttributeFlags::AutoGenerated);
		PatchGroups = OutMeshDescription.PolygonAttributes().GetAttributesRef<int32>(PolyTriGroups);
		check(PatchGroups.IsValid());
	}
	return PatchGroups;
}

void GetExistingPatches(FMeshDescription& MeshSource, TSet<int32>& OutExistingPatchIds)
{
	TPolygonAttributesRef<int32> ElementToGroups = MeshSource.PolygonAttributes().GetAttributesRef<int32>(PolyTriGroups);
	int32 LastPatchId = -1;
	for (const FPolygonID TriangleID : MeshSource.Polygons().GetElementIDs())
	{
		int32 PatchId = ElementToGroups[TriangleID];
		if (PatchId != LastPatchId)
		{
			OutExistingPatchIds.Add(PatchId);
			LastPatchId = PatchId;
		}
	}
}

FMeshDescriptionDataCache::FMeshDescriptionDataCache(FMeshDescription& MeshSource)
{
	FStaticMeshAttributes MeshSourceAttributes(MeshSource);

	TPolygonAttributesRef<FPolygonGroupID> PolygoneToPolygoneGroupId = MeshSourceAttributes.GetPolygonPolygonGroupIndices();
	TPolygonAttributesRef<int32> PatchGroup = EnableCADPatchGroups(MeshSource);

	int32 LastPatchGroupId = -1;
	for (int32 PolygoneID = 0; PolygoneID < PolygoneToPolygoneGroupId.GetNumElements(); ++PolygoneID)
	{
		const FPolygonGroupID GroupID = PolygoneToPolygoneGroupId[PolygoneID];
		int32 PatchGroupID = PatchGroup[PolygoneID];

		if (PatchGroupID != LastPatchGroupId)
		{
			PatchGroupToPolygonGroup.Add(PatchGroupID, GroupID);
			LastPatchGroupId = PatchGroupID;
		}
	}

	TPolygonGroupAttributesRef<FName> MeshSourcePolygonGroupImportedMaterialSlotNames = MeshSourceAttributes.GetPolygonGroupMaterialSlotNames();
	MaterialSlotNames = MeshSourcePolygonGroupImportedMaterialSlotNames.GetRawArray(0);
}

void FMeshDescriptionDataCache::RestoreMaterialSlotNames(FMeshDescription& Mesh) const
{
	TPolygonAttributesRef<int32> PatchGroup = EnableCADPatchGroups(Mesh);
	FStaticMeshAttributes MeshAttributes(Mesh);
	TPolygonAttributesRef<FPolygonGroupID> PolygoneToPolygoneGroupId = MeshAttributes.GetPolygonPolygonGroupIndices();
	TPolygonGroupAttributesRef<FName> PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();

	TSet<FPolygonGroupID> GroupIdDone;
	int32 LastPatchGroupId = -1;
	for (int32 PolygoneID = 0; PolygoneID < PolygoneToPolygoneGroupId.GetNumElements(); ++PolygoneID)
	{
		const int32 PatchGroupID = PatchGroup[PolygoneID];
		if (PatchGroupID != LastPatchGroupId)
		{
			const FPolygonGroupID GroupID = PolygoneToPolygoneGroupId[PolygoneID];
			if (!GroupIdDone.Contains(GroupID))
			{
				GroupIdDone.Add(GroupID);

				const FPolygonGroupID* GroupIDMeshSource = Find(PatchGroupID);
				if (GroupIDMeshSource)
				{
					const FName& SlotName = GetSlotName(*GroupIDMeshSource);
					ensure(GroupID < PolygonGroupMaterialSlotNames.GetNumElements());
					PolygonGroupMaterialSlotNames[GroupID] = SlotName;
				}
				// else PatchGroupeID doesn't exist in MeshSource
			}

			LastPatchGroupId = PatchGroupID;
		}
	}
}

bool FillMesh(const FMeshConversionContext& MeshConversionContext, FBodyMesh& BodyTessellation, FMeshDescription& MeshDescription)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CADLibrary::FillMesh);

	const int32 UVChannel = 0;
	const int32 VertexCountPerFace = 3;
	const TriangleIndex Clockwise = { 0, 1, 2 };
	const TriangleIndex CounterClockwise = { 0, 2, 1 };

	TArray<FVertexInstanceID> TriangleVertexInstanceIDs;
	TriangleVertexInstanceIDs.SetNum(VertexCountPerFace);

	TArray<FVertexInstanceID> MeshVertexInstanceIDs;

	// Gather all array data
	FStaticMeshAttributes Attributes(MeshDescription);
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

	if (!VertexInstanceNormals.IsValid() || !VertexInstanceTangents.IsValid() || !VertexInstanceBinormalSigns.IsValid() || !VertexInstanceColors.IsValid() || !VertexInstanceUVs.IsValid() || !PolygonGroupImportedMaterialSlotNames.IsValid())
	{
		return false;
	}

	// To avoid duplicated triangle
	TSet<TTuple<FVertexInstanceID, FVertexInstanceID, FVertexInstanceID>> VertexIdsToTriangle;

	const TSet<int32>& ExistingPatches = MeshConversionContext.PatchesToMesh;
	bool bImportOnlyAlreadyPresent = (bool)ExistingPatches.Num();

	// Find all the materials used
	TMap<uint32, FPolygonGroupID> MaterialToPolygonGroupMapping;
	for (const FTessellationData& FaceTessellation : BodyTessellation.Faces)
	{
		// don't add material of empty face
		if (FaceTessellation.VertexIndices.Num() == 0)
		{
			continue;
		}

		if (bImportOnlyAlreadyPresent && !ExistingPatches.Contains(FaceTessellation.PatchId))
		{
			continue;
		}

		// material is preferred over color
		MaterialToPolygonGroupMapping.Add(FaceTessellation.MaterialUId ? FaceTessellation.MaterialUId : FaceTessellation.ColorUId, INDEX_NONE);
	}

	// Add to the mesh, a polygon groups per material
	int32 PolyGroupIndex = 0;
	FPolygonGroupID PolyGroupID = 0;
	for (auto& Material : MaterialToPolygonGroupMapping)
	{
		if (PolyGroupIndex < FImportParameters::GMaxMaterialCountPerMesh)
		{
			uint32 MaterialHash = Material.Key;
			FName ImportedSlotName = *LexToString(MaterialHash);
			PolyGroupID = MeshDescription.CreatePolygonGroup();
			PolygonGroupImportedMaterialSlotNames[PolyGroupID] = ImportedSlotName;
		}
		else if (PolyGroupIndex == FImportParameters::GMaxMaterialCountPerMesh)
		{
			UE_LOGF(LogCADLibrary, Warning, "The main UE5 rendering systems do not support more than 256 materials per mesh and the limit has been defined to %d materials. Only the first %d materials are kept. The others are replaced by the last one", FImportParameters::GMaxMaterialCountPerMesh, FImportParameters::GMaxMaterialCountPerMesh);
		}
		Material.Value = PolyGroupID;
		PolyGroupIndex++;
	}

	if (Algo::AnyOf(BodyTessellation.Faces, [](const FTessellationData& FaceTessellation) { return !FaceTessellation.TexCoordArray.IsEmpty(); }))
	{
		VertexInstanceUVs.SetNumChannels(1);
	}

	int32 NbStep = 1;
	if (MeshConversionContext.MeshParameters.bIsSymmetric)
	{
		NbStep = 2;
	}

	TPolygonAttributesRef<int32> PatchGroups = EnableCADPatchGroups(MeshDescription);
	int32 PatchIndex = 0;
	for (int32 Step = 0; Step < NbStep; ++Step)
	{
		// Swap mesh if needed
		const TriangleIndex& Orientation = (!MeshConversionContext.MeshParameters.bNeedSwapOrientation == (bool)Step) ? CounterClockwise : Clockwise;
		TArray<int32>& VertexIdSet = (Step == 0) ? BodyTessellation.VertexIds : BodyTessellation.SymmetricVertexIds;

		for (FTessellationData& Tessellation : BodyTessellation.Faces)
		{
			if (bImportOnlyAlreadyPresent && !ExistingPatches.Contains(Tessellation.PatchId))
			{
				continue;
			}

			// Get the polygonGroup (material is preferred over color)
			FMaterialUId GraphicUId = Tessellation.MaterialUId ? Tessellation.MaterialUId : Tessellation.ColorUId;
			const FPolygonGroupID* PolygonGroupID = MaterialToPolygonGroupMapping.Find(GraphicUId);
			if (PolygonGroupID == nullptr)
			{
				continue;
			}

			if (!Step)
			{
				FDatasmithUtils::ConvertVectorArray(MeshConversionContext.ImportParameters.GetModelCoordSys(), Tessellation.NormalArray);
				for (FVector3f& Normal : Tessellation.NormalArray)
				{
					Normal = Normal.GetSafeNormal();
				}
			}

			TArray<int32> UVIndices;
			TArray<int32> NormalIndices;
			TMap<TTuple<FVertexID, int32, int32>, FVertexInstanceID> VertexIDToInstanceIDForMesh;
			TFunction<FVertexInstanceID(FVertexID, int32)> FindOrAddVertexInstanceIDForMesh = [&](FVertexID VertexID, int32 VertexIndex) ->FVertexInstanceID
			{
				int32 NormalIndex = NormalIndices[VertexIndex];
				int32 UVIndex = UVIndices[VertexIndex];

				FVertexInstanceID& VertexInstanceID = VertexIDToInstanceIDForMesh.FindOrAdd(TTuple<FVertexID, int32, int32>(VertexID, NormalIndices[VertexIndex], UVIndices[VertexIndex]));
				if (VertexInstanceID == -1)
				{
					VertexInstanceID = MeshDescription.CreateVertexInstance(VertexID);
					VertexInstanceColors[VertexInstanceID] = FLinearColor::White;
					VertexInstanceTangents[VertexInstanceID] = FVector3f(ForceInitToZero);
					VertexInstanceBinormalSigns[VertexInstanceID] = 0.0f;

					MeshVertexInstanceIDs.Add(VertexInstanceID);
					VertexInstanceNormals[VertexInstanceID] = (FVector3f)Tessellation.NormalArray[NormalIndex];
					if (!Tessellation.TexCoordArray.IsEmpty())
			{
						VertexInstanceUVs.Set(VertexInstanceID, UVChannel, FVector2f(Tessellation.TexCoordArray[UVIndex]));
					}
				}
				return VertexInstanceID;
			};

			TMap<FVertexID, FVertexInstanceID> VertexIDToInstanceIDForCad;
			TFunction<FVertexInstanceID(FVertexID, int32)> FindOrAddVertexInstanceIDForCad = [&](FVertexID VertexID, int32 VertexIndex) ->FVertexInstanceID
				{
				FVertexInstanceID& VertexInstanceID = VertexIDToInstanceIDForCad.FindOrAdd(VertexID);
				if (VertexInstanceID == -1)
				{
					VertexInstanceID = MeshDescription.CreateVertexInstance(VertexID);
					VertexInstanceColors[VertexInstanceID] = FLinearColor::White;
					VertexInstanceTangents[VertexInstanceID] = FVector3f(ForceInitToZero);
					VertexInstanceBinormalSigns[VertexInstanceID] = 0.0f;

					MeshVertexInstanceIDs.Add(VertexInstanceID);
					VertexInstanceNormals[VertexInstanceID] = (FVector3f)Tessellation.NormalArray[VertexIndex];
					if (!Tessellation.TexCoordArray.IsEmpty())
				{
						VertexInstanceUVs.Set(VertexInstanceID, UVChannel, FVector2f(Tessellation.TexCoordArray[VertexIndex]));
					}
				}
				return VertexInstanceID;
			};

			TFunction<FVertexInstanceID(FVertexID, int32)> FindOrAddVertexInstanceID = BodyTessellation.bIsFromCad ? FindOrAddVertexInstanceIDForCad : FindOrAddVertexInstanceIDForMesh;


			if (BodyTessellation.bIsFromCad)
			{
				VertexIDToInstanceIDForMesh.Reserve(Tessellation.VertexIndices.Num());
			}
			else
			{
				VertexIDToInstanceIDForCad.Reserve(Tessellation.VertexIndices.Num());

				if (Tessellation.TexCoordArray.IsEmpty())
			{
					UVIndices.Init(0, Tessellation.VertexIndices.Num());
				}
				else
				{
					UVIndices.SetNum(Tessellation.VertexIndices.Num());
					for (int32 Index = 0; Index < Tessellation.VertexIndices.Num(); ++Index)
					{
						UVIndices[Index] = Index;
				}
					const FVector2f OneVector(1.f, 1.f);
					MergeCoincidents(Tessellation.TexCoordArray, OneVector, KINDA_SMALL_NUMBER, UVIndices);
			}

				if (Tessellation.NormalArray.Num() == 1)
			{
					NormalIndices.Init(0, Tessellation.VertexIndices.Num());
			}
				else
			{
					NormalIndices.SetNum(Tessellation.VertexIndices.Num());
					for (int32 Index = 0; Index < Tessellation.VertexIndices.Num(); ++Index)
				{
						NormalIndices[Index] = Index;
					}
					MergeCoincidents(Tessellation.NormalArray, FVector3f::OneVector, KINDA_SMALL_NUMBER, NormalIndices);
				}
			}

			int32 FaceVertexIDs[3];
			int32 FaceVertexIndices[3];
			int32 FaceVertexPositionIndices[3];
			FVector Temp3D = { 0, 0, 0 };
			FVector2D TexCoord2D = { 0, 0 };

			MeshVertexInstanceIDs.Empty(Tessellation.VertexIndices.Num());

			PatchIndex++;

			// build each valid face i.e. 3 different indexes
			for (int32 FaceIndex = 0; FaceIndex < Tessellation.VertexIndices.Num(); FaceIndex += VertexCountPerFace)
			{
				FaceVertexIndices[0] = Tessellation.VertexIndices[FaceIndex + Orientation[0]];
				FaceVertexIndices[1] = Tessellation.VertexIndices[FaceIndex + Orientation[1]];
				FaceVertexIndices[2] = Tessellation.VertexIndices[FaceIndex + Orientation[2]];

				FaceVertexPositionIndices[0] = Tessellation.PositionIndices[FaceVertexIndices[0]];
				FaceVertexPositionIndices[1] = Tessellation.PositionIndices[FaceVertexIndices[1]];
				FaceVertexPositionIndices[2] = Tessellation.PositionIndices[FaceVertexIndices[2]];

				if (FaceVertexPositionIndices[0] == INDEX_NONE || FaceVertexPositionIndices[1] == INDEX_NONE || FaceVertexPositionIndices[2] == INDEX_NONE)
				{
					continue;
				}

				FaceVertexIDs[0] = (FVertexID) VertexIdSet[FaceVertexPositionIndices[0]];
				FaceVertexIDs[1] = (FVertexID) VertexIdSet[FaceVertexPositionIndices[1]];
				FaceVertexIDs[2] = (FVertexID) VertexIdSet[FaceVertexPositionIndices[2]];

				// Verify the 3 input indices are not defining a degenerated triangle
				if (FaceVertexIDs[0] == FaceVertexIDs[1] || FaceVertexIDs[0] == FaceVertexIDs[2] || FaceVertexIDs[1] == FaceVertexIDs[2])
				{
					continue;
				}

				TriangleVertexInstanceIDs[0] = FindOrAddVertexInstanceID(FaceVertexIDs[0], FaceVertexIndices[0]);
				TriangleVertexInstanceIDs[1] = FindOrAddVertexInstanceID(FaceVertexIDs[1], FaceVertexIndices[1]);
				TriangleVertexInstanceIDs[2] = FindOrAddVertexInstanceID(FaceVertexIDs[2], FaceVertexIndices[2]);

				if(FImportParameters::bGRemoveDuplicatedTriangle)
				{
					Algo::Sort(FaceVertexIDs);
					bool bIsAlreadyInSet;
					VertexIdsToTriangle.Emplace(TTuple<FVertexInstanceID, FVertexInstanceID, FVertexInstanceID>(FaceVertexIDs[0], FaceVertexIDs[1], FaceVertexIDs[2]), &bIsAlreadyInSet);
					if (bIsAlreadyInSet)
					{
						continue;
					}
				}

				// Add the triangle as a polygon to the mesh description
				const FPolygonID PolygonID = MeshDescription.CreatePolygon(*PolygonGroupID, TriangleVertexInstanceIDs);

				// Set patch id attribute
				PatchGroups[PolygonID] = Tessellation.PatchId;
			}

			if (Step)
			{
				// compute normals of Symmetric vertex
				FMatrix44f SymmetricMatrix = FDatasmithUtils::GetSymmetricMatrix(MeshConversionContext.MeshParameters.SymmetricOrigin, MeshConversionContext.MeshParameters.SymmetricNormal);
				for (const FVertexInstanceID& VertexInstanceID : MeshVertexInstanceIDs) //-V1078
				{
					VertexInstanceNormals[VertexInstanceID] = SymmetricMatrix.TransformVector(VertexInstanceNormals[VertexInstanceID]);
				}
			}

			if (MeshConversionContext.MeshParameters.bNeedSwapOrientation)
			{
				for (int32 Index = 0; Index < MeshVertexInstanceIDs.Num(); Index++)
				{
					VertexInstanceNormals[MeshVertexInstanceIDs[Index]] = VertexInstanceNormals[MeshVertexInstanceIDs[Index]] * -1.f;
				}
			}
		}
	}
	return true;
}

bool ConvertBodyMeshToMeshDescription(const FMeshConversionContext& MeshConversionContext, FBodyMesh& Body, FMeshDescription& MeshDescription)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CADLibrary::ConvertBodyMeshToMeshDescription);

	// in a closed big mesh VertexCount ~ TriangleCount / 2, EdgeCount ~ 1.5* TriangleCount
	MeshDescription.ReserveNewVertexInstances(Body.VertexArray.Num());
	MeshDescription.ReserveNewPolygons(Body.TriangleCount);
	MeshDescription.ReserveNewEdges(Body.TriangleCount * 3);

	FillVertexPosition(MeshConversionContext, Body, MeshDescription);

	if (!FillMesh(MeshConversionContext, Body, MeshDescription))
	{
		return false;
	}
	if (!Body.bIsFromCad)
	{
		MeshOperator::FixNonManifoldMesh(MeshDescription);
	}

	// Workaround SDHE-19725: Compute any null normals.
	MeshOperator::RecomputeNullNormal(MeshDescription);

	// Orient mesh
	MeshOperator::OrientMesh(MeshDescription);

	// Sew mesh
	if(FImportParameters::bGSewMeshIfNeeded)
	{
		double Tolerance = FImportParameters::GStitchingTolerance;
		MeshOperator::ResolveTJunctions(MeshDescription, Tolerance);
	}

	// Build edge meta data
	FStaticMeshOperations::DetermineEdgeHardnessesFromVertexInstanceNormals(MeshDescription);

	return MeshDescription.Polygons().Num() > 0;
}

TSharedPtr<IDatasmithUEPbrMaterialElement> CreateDefaultUEPbrMaterial()
{
	// Take the Material diffuse color and connect it to the BaseColor of a UEPbrMaterial
	TSharedRef<IDatasmithUEPbrMaterialElement> MaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(TEXT("0"));
	MaterialElement->SetLabel(TEXT("DefaultCADImportMaterial"));

	FLinearColor LinearColor = FLinearColor::FromPow22Color(FColor(200, 200, 200, 255));
	IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
	ColorExpression->SetName(TEXT("Base Color"));
	ColorExpression->GetColor() = LinearColor;
	MaterialElement->GetBaseColor().SetExpression(ColorExpression);
	MaterialElement->SetParentLabel(TEXT("M_DatasmithCAD"));

	return MaterialElement;
}

TSharedPtr<IDatasmithUEPbrMaterialElement> CreateUEPbrMaterialFromColor(const FColor& InColor)
{
	FString Name = FString::FromInt(BuildColorUId(InColor));
	FString Label = FString::Printf(TEXT("color_%02x%02x%02x%02x"), InColor.R, InColor.G, InColor.B, InColor.A);

	// Take the Material diffuse color and connect it to the BaseColor of a UEPbrMaterial
	TSharedRef<IDatasmithUEPbrMaterialElement> MaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(*Name);
	MaterialElement->SetLabel(*Label);

	FLinearColor LinearColor = FLinearColor::FromSRGBColor(InColor);

	IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
	ColorExpression->SetName(TEXT("Base Color"));
	ColorExpression->GetColor() = LinearColor;

	MaterialElement->GetBaseColor().SetExpression(ColorExpression);

	if (LinearColor.A < 1.0f)
	{
		MaterialElement->SetBlendMode(/*EBlendMode::BLEND_Translucent*/2);

		IDatasmithMaterialExpressionScalar* Scalar = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		Scalar->GetScalar() = LinearColor.A;
		Scalar->SetName(TEXT("Opacity Level"));

		MaterialElement->GetOpacity().SetExpression(Scalar);
		MaterialElement->SetParentLabel(TEXT("M_DatasmithCADTransparent"));
	}
	else
	{
		MaterialElement->SetParentLabel(TEXT("M_DatasmithCAD"));
	}

	return MaterialElement;
}

TSharedPtr<IDatasmithUEPbrMaterialElement> CreateUEPbrMaterialFromMaterial(FCADMaterial& InMaterial, TSharedRef<IDatasmithScene> Scene)
{
	FString Name = FString::FromInt(BuildMaterialUId(InMaterial));

	// Take the Material diffuse color and connect it to the BaseColor of a UEPbrMaterial
	TSharedRef<IDatasmithUEPbrMaterialElement> MaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(*Name);
	FString MaterialLabel(InMaterial.MaterialName);
	if (MaterialLabel.IsEmpty())
	{
		MaterialLabel = TEXT("Material");
	}
	MaterialElement->SetLabel(*MaterialLabel);

	// Set a diffuse color if there's nothing in the BaseColor
	if (MaterialElement->GetBaseColor().GetExpression() == nullptr)
	{
		FLinearColor LinearColor = FLinearColor::FromSRGBColor(InMaterial.Diffuse);

		IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		ColorExpression->SetName(TEXT("Base Color"));
		ColorExpression->GetColor() = LinearColor;

		MaterialElement->GetBaseColor().SetExpression(ColorExpression);
	}

	if (InMaterial.Transparency > 0.0f)
	{
		MaterialElement->SetBlendMode(/*EBlendMode::BLEND_Translucent*/2);
		IDatasmithMaterialExpressionScalar* Scalar = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		Scalar->GetScalar() = InMaterial.Transparency;
		Scalar->SetName(TEXT("Opacity Level"));
		MaterialElement->GetOpacity().SetExpression(Scalar);
		MaterialElement->SetParentLabel(TEXT("M_DatasmithCADTransparent"));
	}
	else
	{
		MaterialElement->SetParentLabel(TEXT("M_DatasmithCAD"));
	}

	return MaterialElement;
}

}
