// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshEditingWrapper.h"

#include "MeshAttributes.h"
#include "StaticMeshAttributes.h"




FMeshEditingWrapper::FMeshEditingWrapper(FMeshDescription& InMeshDescription)
	: MeshDescription(InMeshDescription)
{
	UpdateMeshWrapper();
}

void FMeshEditingWrapper::UpdateMeshWrapper()
{
	VertexMetaData.SetNumZeroed(MeshDescription.Vertices().GetArraySize());
	EdgeMetaData.SetNumZeroed(MeshDescription.Edges().GetArraySize());
	TriangleMetaData.SetNumZeroed(MeshDescription.Triangles().GetArraySize());

	// If there are more than one polygon groups, edges between groups are considered feature lines
	for (const FEdgeID EdgeID : MeshDescription.Edges().GetElementIDs())
	{
		DefineEdgeTopology(EdgeID);
	}

	for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
	{
		DefineVertexTopologyApproximation(VertexID);
	}

	VertexInstanceMarker.Init(false, MeshDescription.VertexInstances().GetArraySize());
}


void FMeshEditingWrapper::DefineEdgeTopology(FEdgeID EdgeID)
{
	if (EdgeID == INDEX_NONE)
	{
		return;
	}

	TArrayView<const FTriangleID> EdgeConnectedPolygons = MeshDescription.GetEdgeConnectedTriangleIDs(EdgeID);

	switch (EdgeConnectedPolygons.Num())
	{
	case 0:
		EdgeMetaData[EdgeID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryLine;
		break;
	case 1:
		EdgeMetaData[EdgeID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryBorder;
		break;
	case 2:
		EdgeMetaData[EdgeID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategorySurface;
		break;
	default:
		EdgeMetaData[EdgeID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryNonManifold;
		break;
	}
}

void FMeshEditingWrapper::DefineVertexTopologyApproximation(FVertexID VertexID)
{
	if (VertexID == INDEX_NONE)
	{
		return;
	}

	TArrayView<const FEdgeID> VertexConnectedEdgeIDs = MeshDescription.GetVertexConnectedEdgeIDs(VertexID);

	switch (VertexConnectedEdgeIDs.Num())
	{
	case 0:
		VertexMetaData[VertexID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryFree;
		break;

	case 1:
		VertexMetaData[VertexID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryBorder;
		break;
	default:
	{
		int CountPerCategory[(uint16)MeshCategory::EElementCategory::ElementCategoryMax] = { 0 };

		for (const FEdgeID EdgeID : VertexConnectedEdgeIDs)
		{
			CountPerCategory[EdgeMetaData[EdgeID.GetValue()].Category]++;
		}

		if (CountPerCategory[(uint16)MeshCategory::EElementCategory::ElementCategoryNonManifold] > 0)
		{
			VertexMetaData[VertexID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryNonManifold;
		}
		else if (CountPerCategory[(uint16)MeshCategory::EElementCategory::ElementCategoryBorder] > 0)
		{
			VertexMetaData[VertexID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryNonSurface;

		}
		else if (CountPerCategory[(uint16)MeshCategory::EElementCategory::ElementCategoryLine] > 0)
		{
			if (CountPerCategory[(uint16)MeshCategory::EElementCategory::ElementCategoryLine] == 2 && VertexConnectedEdgeIDs.Num() == 2)
			{
				VertexMetaData[VertexID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryLine;
			}
			else
			{
				VertexMetaData[VertexID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryNonManifold;
			}
		}
		else if (CountPerCategory[(uint16)MeshCategory::EElementCategory::ElementCategorySurface] > 0)
		{
			const int32 EdgeCount = VertexConnectedEdgeIDs.Num();
			const FEdgeID FirstEdgeID(VertexConnectedEdgeIDs[0]);
			FEdgeID EdgeID(FirstEdgeID);

			FTriangleID TriangleID(INDEX_NONE);
			int32 TriangleCount = 0;

			do
			{
				TArrayView<const FTriangleID> EdgeConnectedPolygons = MeshDescription.GetEdgeConnectedTriangleIDs(EdgeID);
				// Border edge no more triangles to process, exit from the loop
				if (EdgeConnectedPolygons.Num() < 2)
				{
					break;
				}

				TriangleID = (TriangleID == EdgeConnectedPolygons[0]) ? EdgeConnectedPolygons[1] : EdgeConnectedPolygons[0];
				++TriangleCount;

				TArrayView<const FEdgeID> TriangleEdges = MeshDescription.GetTriangleEdges(TriangleID);

				for (int32 Corner = 0; Corner < 3; ++Corner)
				{
					if (EdgeID != TriangleEdges[Corner])
					{
						const FVertexID EdgeVertexID0 = MeshDescription.GetEdgeVertex(TriangleEdges[Corner], 0);
						const FVertexID EdgeVertexID1 = MeshDescription.GetEdgeVertex(TriangleEdges[Corner], 1);
						if (EdgeVertexID0 == VertexID || EdgeVertexID1 == VertexID)
						{
							EdgeID = TriangleEdges[Corner];
							break;
						}
					}
				}
			} while (EdgeID != FirstEdgeID);

			VertexMetaData[VertexID.GetValue()].Category = TriangleCount == EdgeCount ? (uint16)MeshCategory::EElementCategory::ElementCategorySurface : (uint16)MeshCategory::EElementCategory::ElementCategoryNonManifold;
		}
		else
		{
			// we don`t need to know exactly if the node is a border or non manifold node. Non surface is enough
			VertexMetaData[VertexID.GetValue()].Category = (uint16)MeshCategory::EElementCategory::ElementCategoryNonSurface;
		}
	}
	break;
	}
}

FMeshEditingWrapper::~FMeshEditingWrapper()
{

}

void FMeshEditingWrapper::ResetTriangleMarkerRecursively(FTriangleID Triangle)
{
	MeshEditingWrapperUtils::ResetElementMarker(TriangleMetaData[Triangle.GetValue()], ELEMENT_FIRST_MARKER_MASK);
	TArrayView<const FEdgeID> EdgeSet = MeshDescription.GetTriangleEdges(Triangle);
	for (int32 IEdge = 0; IEdge < 3; IEdge++)
	{
		MeshEditingWrapperUtils::ResetElementMarker(TriangleMetaData[EdgeSet[IEdge].GetValue()], ELEMENT_FIRST_MARKER_MASK);
	}
}

bool FMeshEditingWrapper::IsTriangleMarked(FTriangleID Triangle) const
{
	return MeshEditingWrapperUtils::IsElementMarkerSet(TriangleMetaData[Triangle.GetValue()], ELEMENT_FIRST_MARKER_MASK);
}

void FMeshEditingWrapper::SetTriangleMarked(FTriangleID Triangle)
{
	return MeshEditingWrapperUtils::SetElementMarker(TriangleMetaData[Triangle.GetValue()], ELEMENT_FIRST_MARKER_MASK);
}

MeshCategory::EElementCategory FMeshEditingWrapper::GetEdgeCategory(FEdgeID Edge) const
{
	return (MeshCategory::EElementCategory) EdgeMetaData[Edge.GetValue()].Category;
}

bool FMeshEditingWrapper::IsEdgeOfCategory(FEdgeID Edge, MeshCategory::EElementCategory Category) const
{
	return EdgeMetaData[Edge.GetValue()].Category == Category;
}

bool FMeshEditingWrapper::IsVertexOfCategory(FVertexInstanceID VertexInstanceID, MeshCategory::EElementCategory Category) const
{
	FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceID);
	return VertexMetaData[VertexID.GetValue()].Category == Category;
}

// Triangle
void FMeshEditingWrapper::GetTriangleBoundingBox(FTriangleID Triangle, FVector& MinCorner, FVector& MaxCorner, FVertexInstanceID HighestVertex[3], FVertexInstanceID LowestVertex[3]) const
{
	TArrayView < const FVertexInstanceID> Vertices = MeshDescription.GetTriangleVertexInstances(Triangle);
	for (int32 Index = 0; Index < 3; Index++)
	{
		GetVertexBoundingBox(Vertices[Index], MinCorner, MaxCorner, HighestVertex, LowestVertex);
	}
}

void FMeshEditingWrapper::GetVertexBoundingBox(FVertexInstanceID VertexInstanceID, FVector& MinCorner, FVector& MaxCorner, FVertexInstanceID HighestVertex[3], FVertexInstanceID LowestVertex[3]) const
{
	FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceID);
	const FVector VertexPosition = (FVector)MeshDescription.GetVertexPositions()[VertexID];

	if (MaxCorner[0] < VertexPosition[0])
	{
		HighestVertex[0] = VertexInstanceID;
		MaxCorner[0] = VertexPosition[0];
	}
	if (MaxCorner[1] < VertexPosition[1])
	{
		HighestVertex[1] = VertexInstanceID;
		MaxCorner[1] = VertexPosition[1];
	}
	if (MaxCorner[2] < VertexPosition[2])
	{
		HighestVertex[2] = VertexInstanceID;
		MaxCorner[2] = VertexPosition[2];
	}

	if (MinCorner[0] > VertexPosition[0])
	{
		LowestVertex[0] = VertexInstanceID;
		MinCorner[0] = VertexPosition[0];
	}
	if (MinCorner[1] > VertexPosition[1])
	{
		LowestVertex[1] = VertexInstanceID;
		MinCorner[1] = VertexPosition[1];
	}
	if (MinCorner[2] > VertexPosition[2])
	{
		LowestVertex[2] = VertexInstanceID;
		MinCorner[2] = VertexPosition[2];
	}
}

void FMeshEditingWrapper::SwapTriangleOrientation(FTriangleID Triangle)
{
	MeshDescription.ReverseTriangleFacing(Triangle);
	TArrayView<const FVertexInstanceID> TriVertexInstances = MeshDescription.GetTriangleVertexInstances(Triangle);
	for (int32 IVertex = 0; IVertex < 3; IVertex++)
	{
		const FVertexInstanceID& InstanceID = TriVertexInstances[IVertex];
		if(!VertexInstanceMarker[InstanceID])
		{
			SwapVertexNormal(InstanceID);
			VertexInstanceMarker[InstanceID] = true;
		}
	}
}

// Edge
const FTriangleID FMeshEditingWrapper::GetOtherTriangleAtEdge(FEdgeID EdgeID, FTriangleID Triangle) const
{
	TArrayView<const FTriangleID> EdgeConnectedPolygons = MeshDescription.GetEdgeConnectedTriangleIDs(EdgeID);
	return EdgeConnectedPolygons.Num() < 2 ? INDEX_NONE : (EdgeConnectedPolygons[0] == Triangle ? EdgeConnectedPolygons[1] : EdgeConnectedPolygons[0]);
}

bool FMeshEditingWrapper::GetEdgeDirectionInTriangle(FEdgeID EdgeID, int32 TriangleIndex) const
{
	TArrayView<const FTriangleID> EdgeConnectedTriangles = MeshDescription.GetEdgeConnectedTriangleIDs(EdgeID);

	if (EdgeConnectedTriangles.Num() > TriangleIndex)
	{
		const FTriangleID TriangleID = EdgeConnectedTriangles[TriangleIndex];
		TArrayView<const FVertexInstanceID> VertexInstanceIDs = MeshDescription.GetTriangleVertexInstances(TriangleID);
		TArrayView<const FEdgeID> TriangleEdges = MeshDescription.GetTriangleEdges(TriangleID);

		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			if (TriangleEdges[Corner] == EdgeID)
			{
				const FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceIDs[Corner]);
				return MeshDescription.GetEdgeVertex(EdgeID, 0) == VertexID ? true : false;
			}
		}
	}
	return true;
}

// Vertex
void FMeshEditingWrapper::SwapVertexNormal(FVertexInstanceID VertexInstanceID)
{
	FStaticMeshAttributes StaticMeshAttributes( MeshDescription );
	FVector3f Normal = StaticMeshAttributes.GetVertexInstanceNormals()[ VertexInstanceID ];
	Normal *= -1;

	StaticMeshAttributes.GetVertexInstanceNormals()[ VertexInstanceID ] = Normal;
}

