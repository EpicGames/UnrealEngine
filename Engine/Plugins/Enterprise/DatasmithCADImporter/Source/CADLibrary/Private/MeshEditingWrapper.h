// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"

#include "MeshTypes.h"
#include "MeshDescription.h"


#define EDGE_IS_HARD      0x01
#define EDGE_IS_UV_SEAM   0x02

#define ELEMENT_STATUS_MASK			0x01
#define ELEMENT_FIRST_MARKER_MASK	0x02
#define ELEMENT_SECOND_MARKER_MASK	0x04

#define ELEMENT_CRITICAL_ZONE_MASK		0x01
#define ELEMENT_PARTITION_BORDER_MASK	0x02

struct FElementMetaData
{
	uint16 Category:4;
	uint16 Markers:4;
	uint16 Extras:4;
};

namespace MeshCategory
{
	enum EElementCategory : uint8_t
	{
		ElementCategoryUnused = 0,
		ElementCategoryFree = 1,
		ElementCategoryLine = 2,
		ElementCategorySurface = 3,
		ElementCategoryBorder = 4,
		ElementCategoryNonManifold = 5,
		ElementCategoryNonSurface = 6,
		ElementCategoryMax = 7
	};
}

namespace MeshEditingWrapperUtils
{
	const FName Debug( "Debug" );
	const FName FeatureLine( "FeatureLine" );
	const FName EdgeLength( "EdgeLength" );

	FORCEINLINE bool IsElementMarkerSet(const FElementMetaData& ElementMetaData, const uint16 Mask)
	{
		return (ElementMetaData.Markers & Mask) == Mask;
	}

	FORCEINLINE void SetElementMarker(FElementMetaData& ElementMetaData, const uint16 Mask)
	{
		ElementMetaData.Markers |= Mask;
	}

	FORCEINLINE void ResetElementMarker(FElementMetaData& ElementMetaData, const uint16 Mask)
	{
		ElementMetaData.Markers &= ~Mask;
	}

	FORCEINLINE bool IsElementExtraSet(const FElementMetaData& ElementMetaData, const uint16 Mask)
	{
		return (ElementMetaData.Extras & Mask) == Mask;
	}

	FORCEINLINE void SetElementExtra(FElementMetaData& ElementMetaData, const bool Value, const uint16 Mask)
	{
		if (Value)
		{
			ElementMetaData.Extras |= Mask;
		}
		else
		{
			ElementMetaData.Extras &= ~Mask;
		}
	}

	FORCEINLINE void ResetElementExtra(FElementMetaData& ElementMetaData, const uint16 Mask)
	{
		ElementMetaData.Extras &= ~Mask;
	}

	FORCEINLINE void ResetElementData(FElementMetaData& ElementMetaData)
	{
		ElementMetaData.Category = (uint16)MeshCategory::EElementCategory::ElementCategoryUnused;
		ElementMetaData.Markers = 0;
		ElementMetaData.Extras = 0;
	}
}

class FMeshEditingWrapper
{

private:
	FMeshDescription& MeshDescription;

	TArray<FElementMetaData> VertexMetaData;
	TArray<FElementMetaData> EdgeMetaData;
	TArray<FElementMetaData> TriangleMetaData;

	// for orientation purpose
	TBitArray<> VertexInstanceMarker;

private:


public:
	FMeshEditingWrapper(FMeshDescription& InMeshDescription);
	 ~FMeshEditingWrapper();

	 void UpdateMeshWrapper();

	 void ResetTriangleMarkerRecursively(FTriangleID Triangle);
	 bool IsTriangleMarked(FTriangleID Triangle) const;
	 void SetTriangleMarked(FTriangleID Triangle);

	 MeshCategory::EElementCategory GetEdgeCategory(FEdgeID Edge) const;
	 bool IsEdgeOfCategory(FEdgeID Edge, MeshCategory::EElementCategory Category) const;
	 bool IsVertexOfCategory(FVertexInstanceID Vertex, MeshCategory::EElementCategory Category) const;

	 // Triangle
	 void GetTriangleBoundingBox(FTriangleID Triangle, FVector& MinCorner, FVector& MaxCorner, FVertexInstanceID HighestVertex[3], FVertexInstanceID LowestVertex[3]) const;

	 void SwapTriangleOrientation(FTriangleID Triangle);

	 // Edge
	 const FTriangleID GetOtherTriangleAtEdge(FEdgeID Edge, FTriangleID Triangle) const;

	 /**
	  * @param Edge           The edge defined by its FEdgeID to get direction in its connected triangle 
	  * @param TriangleIndex  The triangle connected to the edge, 0 for the first, 1 for the second, ...
	  *
	  */
	 bool GetEdgeDirectionInTriangle(FEdgeID Edge, int32 TriangleIndex) const;
	 void DefineEdgeTopology(FEdgeID EdgeID);

	 // Vertex
	 void SwapVertexNormal(FVertexInstanceID VertexID);
	 void GetVertexBoundingBox(FVertexInstanceID Vertex, FVector& MinCorner, FVector& MaxCorner, FVertexInstanceID HighestVertex[3], FVertexInstanceID LowestVertex[3]) const;

	 void DefineVertexTopologyApproximation(FVertexID VertexID);
};


