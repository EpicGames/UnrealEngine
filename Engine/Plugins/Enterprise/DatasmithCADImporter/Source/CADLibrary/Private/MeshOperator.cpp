// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshOperator.h"

#include "MeshEditingWrapper.h"

#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshAttributes.h"
#include "CADMeshDescriptionHelper.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "MeshElementArray.h"
#include "Operations/MeshResolveTJunctions.h"
#include "StaticMeshAttributes.h"

void MeshOperator::RecomputeNullNormal(FMeshDescription& MeshDescription)
{
	//using namespace UE::MeshCategory;
	constexpr float SquareNormalThreshold = KINDA_SMALL_NUMBER * KINDA_SMALL_NUMBER;

	FStaticMeshAttributes StaticMeshAttributes = FStaticMeshAttributes(MeshDescription);
	TVertexInstanceAttributesRef<FVector3f> Normals = StaticMeshAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVertexID> Indices = StaticMeshAttributes.GetVertexInstanceVertexIndices();
	TVertexAttributesRef<FVector3f> VertexPositions = StaticMeshAttributes.GetVertexPositions();

	for (FTriangleID Triangle : MeshDescription.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexInstanceID> Vertices = MeshDescription.GetTriangleVertexInstances(Triangle);
		for (int32 VertexIndex = 0; VertexIndex < 3; VertexIndex++)
		{
			FVector3f& Normal = Normals[Vertices[VertexIndex]];

			if (Normal.IsNearlyZero(KINDA_SMALL_NUMBER))
			{
				Normal = FVector3f::ZeroVector;

				const FVertexInstanceID VertexInstanceID = Vertices[VertexIndex];
				FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceID);
				const TArrayView<const FTriangleID> VertexConnectedTriangles = MeshDescription.GetVertexInstanceConnectedTriangleIDs(VertexInstanceID);

				int32 TriangleCount = 0;

				const int32 TriangleIndex[3][3] = { {0,1,2},{1,2,0},{2,0,1} };

				// compute the weighted sum of normals of the "partition star" according to the corner angle
				for(const FTriangleID& TriangleID : VertexConnectedTriangles)
				{
					// compute face normal at the vertex
					TArrayView<const FVertexID> TriangleVertices = MeshDescription.GetTriangleVertices(TriangleID);

					int32 ApexIndex = 0;
					for (; ApexIndex < 3; ++ApexIndex)
					{
						if (TriangleVertices[ApexIndex] == VertexID)
						{
							break;
						}
					}

					const FVector3f Position0 = VertexPositions[TriangleVertices[TriangleIndex[ApexIndex][0]]];
					FVector3f DPosition1 = VertexPositions[TriangleVertices[TriangleIndex[ApexIndex][1]]] - Position0;
					FVector3f DPosition2 = VertexPositions[TriangleVertices[TriangleIndex[ApexIndex][2]]] - Position0;

					// to avoid numerical issue due to small vector
					DPosition1.Normalize();
					DPosition2.Normalize();

					// We have a left-handed coordinate system, but a counter-clockwise winding order
					// Hence normal calculation has to take the triangle vectors cross product in reverse.
					FVector3f TriangleNormal = FVector3f::CrossProduct(DPosition2, DPosition1);
					double SinOfApexAngle = TriangleNormal.Length();
					double ApexAngle = FMath::Asin(SinOfApexAngle);

					if (TriangleNormal.Normalize(SquareNormalThreshold))
					{
						Normal += (TriangleNormal * ApexAngle);
						TriangleCount++;
					}
				}

				if (TriangleCount)
				{
					if (Normal.Normalize(SquareNormalThreshold))
					{
						continue;
					}
				}

				// the vertex is a vertex of degenerated triangles, the vertex normal doesn't matter as the triangle is flat, but a non null normal is needed.
				Normal = FVector3f::UpVector;
			}
		}
	}
}

bool MeshOperator::OrientMesh(FMeshDescription& MeshDescription)
{
	using namespace MeshCategory;

	FMeshEditingWrapper MeshWrapper(MeshDescription);

	TQueue<FTriangleID> Front;
	TQueue<FTriangleID> BadOrientationFront;

	FTriangleID AdjacentTriangle;
	FEdgeID Edge;

	TArrayView < const FVertexInstanceID> Vertices;

	FVector MaxCorner;
	FVector MinCorner;
	FVertexInstanceID HighestVertex[3];
	FVertexInstanceID LowestVertex[3];

	uint32 NbTriangles = MeshDescription.Triangles().Num();

	TArray<FTriangleID> ConnectedTriangles;
	ConnectedTriangles.Reserve(NbTriangles);

	for (FTriangleID Triangle : MeshDescription.Triangles().GetElementIDs())
	{
		if (MeshWrapper.IsTriangleMarked(Triangle))
		{
			continue;
		}

		HighestVertex[0] = INDEX_NONE;
		HighestVertex[1] = INDEX_NONE;
		HighestVertex[2] = INDEX_NONE;
		LowestVertex[0] = INDEX_NONE;
		LowestVertex[1] = INDEX_NONE;
		LowestVertex[2] = INDEX_NONE;

		MaxCorner[0] = -MAX_flt;
		MaxCorner[1] = -MAX_flt;
		MaxCorner[2] = -MAX_flt;
		MinCorner[0] = MAX_flt;
		MinCorner[1] = MAX_flt;
		MinCorner[2] = MAX_flt;

		MeshWrapper.SetTriangleMarked(Triangle);

		MeshWrapper.GetTriangleBoundingBox(Triangle, MinCorner, MaxCorner, HighestVertex, LowestVertex);

		Front.Enqueue(Triangle);
		ConnectedTriangles.Add(Triangle);

		int32 NbConnectedFaces = 1;
		int32 NbBorderEdges = 0;
		int32 NbSurfaceEdges = 0;
		int32 NbSwappedTriangles = 0;
		while (!Front.IsEmpty())
		{
			while (!Front.IsEmpty())
			{
				Front.Dequeue(Triangle);

				TArrayView<const FEdgeID> EdgeSet = MeshDescription.GetTriangleEdges(Triangle);

				for (int32 IEdge = 0; IEdge < 3; IEdge++)
				{
					Edge = EdgeSet[IEdge];

					if (!MeshWrapper.IsEdgeOfCategory(Edge, EElementCategory::ElementCategorySurface))
					{
						NbBorderEdges++;
						continue;
					}

					AdjacentTriangle = MeshWrapper.GetOtherTriangleAtEdge(Edge, Triangle);
					if (MeshWrapper.IsTriangleMarked(AdjacentTriangle))
					{
						continue;
					}

					NbSurfaceEdges++;
					NbConnectedFaces++;

					ConnectedTriangles.Add(AdjacentTriangle);

					MeshWrapper.SetTriangleMarked(AdjacentTriangle);
					MeshWrapper.GetTriangleBoundingBox(AdjacentTriangle, MinCorner, MaxCorner, HighestVertex, LowestVertex);

					if (MeshWrapper.GetEdgeDirectionInTriangle(Edge, 0) == MeshWrapper.GetEdgeDirectionInTriangle(Edge, 1))
					{
						MeshWrapper.SwapTriangleOrientation(AdjacentTriangle);
						NbSwappedTriangles++;
						BadOrientationFront.Enqueue(AdjacentTriangle);
					}
					else
					{
						Front.Enqueue(AdjacentTriangle);
					}
				}
			}

			while (!BadOrientationFront.IsEmpty())
			{
				BadOrientationFront.Dequeue(Triangle);

				TArrayView<const FEdgeID> EdgeSet = MeshDescription.GetTriangleEdges(Triangle);

				for (int32 IEdge = 0; IEdge < 3; IEdge++)
				{
					Edge = EdgeSet[IEdge];

					if (!MeshWrapper.IsEdgeOfCategory(Edge, EElementCategory::ElementCategorySurface))
					{
						NbBorderEdges++;
						continue;
					}

					AdjacentTriangle = MeshWrapper.GetOtherTriangleAtEdge(Edge, Triangle);
					if (MeshWrapper.IsTriangleMarked(AdjacentTriangle))
					{
						continue;
					}

					NbSurfaceEdges++;
					NbConnectedFaces++;

					ConnectedTriangles.Add(AdjacentTriangle);

					MeshWrapper.SetTriangleMarked(AdjacentTriangle);
					MeshWrapper.GetTriangleBoundingBox(AdjacentTriangle, MinCorner, MaxCorner, HighestVertex, LowestVertex);
					if (MeshWrapper.GetEdgeDirectionInTriangle(Edge, 0) == MeshWrapper.GetEdgeDirectionInTriangle(Edge, 1))
					{
						BadOrientationFront.Enqueue(AdjacentTriangle);
						MeshWrapper.SwapTriangleOrientation(AdjacentTriangle);
						NbSwappedTriangles++;
					}
					else
					{
						Front.Enqueue(AdjacentTriangle);
					}
				}
			}
		}

		// Check if the mesh orientation need to be swapped
		int NbInverted = 0;
		int NbNotInverted = 0;
		if (NbBorderEdges == 0 || NbBorderEdges * 20 < NbSurfaceEdges)  // NbBorderEdges * 20 < nbSurfaceEdges => basic rule to define if a mesh is a surface mesh or if the mesh is a volume mesh with gaps 
		{
			// case of volume mesh
			// A vertex can have many normal (one by vertex instance) e.g. the corner of a box that have 3 normal and could be the highest 
			// vertex of a mesh. At the highest vertex, a folding of the mesh can give a vertex with two opposite normal.
			// The normal most parallel to the axis is preferred
			// To avoid mistake, we check for each highest vertex and trust the majority 

			if (HighestVertex[0] != INDEX_NONE)
			{
				FStaticMeshConstAttributes StaticMeshAttributes(MeshDescription);
				TVertexInstanceAttributesConstRef<FVector3f> Normals = StaticMeshAttributes.GetVertexInstanceNormals();

				for (int32 VertexIndex = 0; VertexIndex < 3; VertexIndex++)
				{
					if (MeshWrapper.IsVertexOfCategory(HighestVertex[VertexIndex], EElementCategory::ElementCategorySurface))
					{
						FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(HighestVertex[VertexIndex]);
						TArrayView<const FVertexInstanceID> CoincidentVertexInstanceIdSet = MeshDescription.GetVertexVertexInstanceIDs(VertexID);
						float MaxComponent = 0;
						for (const FVertexInstanceID VertexInstanceID : CoincidentVertexInstanceIdSet)
						{
							FVector Normal = (FVector)Normals[VertexInstanceID];
							if (FMath::Abs(MaxComponent) < FMath::Abs(Normal[VertexIndex]))
							{
								MaxComponent = Normal[VertexIndex];
							}
						}

						if (0 > MaxComponent)
						{
							NbInverted++;
						}
						else
						{
							NbNotInverted++;
						}
					}

					if (MeshWrapper.IsVertexOfCategory(LowestVertex[VertexIndex], EElementCategory::ElementCategorySurface))
					{
						FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(LowestVertex[VertexIndex]);
						TArrayView<const FVertexInstanceID> CoincidentVertexInstanceIdSet = MeshDescription.GetVertexVertexInstanceIDs(VertexID);
						float MaxComponent = 0;
						for (const FVertexInstanceID VertexInstanceID : CoincidentVertexInstanceIdSet)
						{
							FVector Normal = (FVector)Normals[VertexInstanceID];
							if (FMath::Abs(MaxComponent) < FMath::Abs(Normal[VertexIndex]))
							{
								MaxComponent = Normal[VertexIndex];
							}
						}

						if (0 < MaxComponent)
						{
							NbInverted++;
						}
						else
						{
							NbNotInverted++;
						}
					}
				}
			}
		}
		else if (NbSwappedTriangles * 2 > NbConnectedFaces)
		{
			// case of surface mesh
			// this means that more triangles of surface shape have been swapped than no swapped, the good orientation has been reversed.
			// The mesh need to be re swapped
			NbInverted++;
		}

		// if needed swap all the mesh
		if (NbInverted > NbNotInverted)
		{
			for (auto Tri : ConnectedTriangles)
			{
				MeshWrapper.SwapTriangleOrientation(Tri);
			}
		}
		ConnectedTriangles.Empty(NbTriangles);
	}

	return true;
}

void MeshOperator::ResolveTJunctions(FMeshDescription& MeshDescription, double Tolerance)
{
	UE::Geometry::FDynamicMesh3 DynamicMesh(UE::Geometry::EMeshComponents::All);
	DynamicMesh.EnableAttributes();

	{
		FMeshDescriptionToDynamicMesh ConverterToDynamicMesh;
		ConverterToDynamicMesh.Convert(&MeshDescription, DynamicMesh);
	}

	// Check if there are boundary edges
	if (auto It(DynamicMesh.BoundaryEdgeIndicesItr()); It.begin() == It.end())
	{
		return;
	}

	UE::Geometry::FMeshResolveTJunctions MeshResolveTJunctions(&DynamicMesh);
	MeshResolveTJunctions.DistanceTolerance = Tolerance;
	bool bResolveOK = MeshResolveTJunctions.Apply();

	if (bResolveOK && MeshResolveTJunctions.NumSplitEdges > 0)
	{
		UE::Geometry::FMergeCoincidentMeshEdges MergeCoincidentEdges(&DynamicMesh);
		MergeCoincidentEdges.MergeVertexTolerance = Tolerance;
		MergeCoincidentEdges.MergeSearchTolerance = 2 * Tolerance;
		MergeCoincidentEdges.Apply();
	}

	CADLibrary::FMeshDescriptionDataCache MeshDescriptionDataCache(MeshDescription);

	{
		FConversionToMeshDescriptionOptions ConversionOptions;
		ConversionOptions.bSetPolyGroups = true;
		ConversionOptions.bUpdatePositions = true;
		ConversionOptions.bUpdateNormals = true;
		ConversionOptions.bUpdateTangents = false;
		ConversionOptions.bUpdateUVs = true;
		ConversionOptions.bUpdateVtxColors = false;
		ConversionOptions.bTransformVtxColorsSRGBToLinear = false;

		FDynamicMeshToMeshDescription ConverterToMeshDescription(ConversionOptions);
		ConverterToMeshDescription.Convert(&DynamicMesh, MeshDescription);
	}

	MeshDescriptionDataCache.RestoreMaterialSlotNames(MeshDescription);
}


void MeshOperator::FixNonManifoldMesh(FMeshDescription& MeshDescription)
{
	using namespace UE::Geometry;

	FDynamicMesh3 DynamicMesh(UE::Geometry::EMeshComponents::All);
	DynamicMesh.EnableAttributes();

	TFunction<void()> ToDynamicMesh = [&MeshDescription, &DynamicMesh]()
	{
		DynamicMesh.Clear();

		FMeshDescriptionToDynamicMesh ConverterToDynamicMesh;
		ConverterToDynamicMesh.Convert(&MeshDescription, DynamicMesh);
	};

	TFunction<void()> ToMeshDescription = [&MeshDescription, &DynamicMesh]()
	{
		CADLibrary::FMeshDescriptionDataCache MeshDescriptionDataCache(MeshDescription);

		{
			FConversionToMeshDescriptionOptions ConversionOptions;
			ConversionOptions.bSetPolyGroups = true;
			ConversionOptions.bUpdatePositions = true;
			ConversionOptions.bUpdateNormals = true;
			ConversionOptions.bUpdateTangents = false;
			ConversionOptions.bUpdateUVs = true;
			ConversionOptions.bUpdateVtxColors = false;
			ConversionOptions.bTransformVtxColorsSRGBToLinear = false;

			FDynamicMeshToMeshDescription ConverterToMeshDescription(ConversionOptions);
			ConverterToMeshDescription.Convert(&DynamicMesh, MeshDescription);
		}

		MeshDescriptionDataCache.RestoreMaterialSlotNames(MeshDescription);
	};

	ToDynamicMesh();

	// Nothing to do if mesh is manifold (all edges are shared by at most 2 triangles)
	if (DynamicMesh.CheckValidity(FDynamicMesh3::FValidityOptions(false, true), EValidityCheckFailMode::ReturnOnly))
	{
		return;
	}

	// Although the conversion from MeshDescription to DynamicMesh3 handles non-manifold edges
	// Some edges may be shared by more than 3 triangles. To solve this, we must perform
	// the conversion back and forth until all non-manifold edges have been eliminated.
	do 
	{
		ToMeshDescription();
		ToDynamicMesh();
	} while (DynamicMesh.CheckValidity(FDynamicMesh3::FValidityOptions(false, true), EValidityCheckFailMode::ReturnOnly));

	ToMeshDescription();
}