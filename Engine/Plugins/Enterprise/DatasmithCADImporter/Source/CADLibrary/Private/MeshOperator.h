// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "MeshTypes.h"
#include "MeshDescription.h"

namespace MeshOperator
{

void FixNonManifoldMesh(FMeshDescription& MeshDescription);

bool OrientMesh(FMeshDescription& MeshDescription);

void RecomputeNullNormal(FMeshDescription& MeshDescription);

void ResolveTJunctions(FMeshDescription& MeshDescription, double Tolerance);

}