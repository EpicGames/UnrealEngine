// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADOptions.h"

namespace UE::CADKernel
{
class FBody;
class FBodyMesh;
class FFaceMesh;
class FTopologicalShapeEntity;
class FModelMesh;
}

struct FMeshDescription;

namespace CADLibrary
{
class FImportParameters;
struct FMeshParameters;

struct FMeshConversionContext;

class CADLIBRARY_API FCADKernelTools
{
public:
	static void DefineMeshCriteria(UE::CADKernel::FModelMesh& MeshModel, const FImportParameters& ImportParameters, double GeometricTolerance);

	static void GetBodyTessellation(const UE::CADKernel::FModelMesh& ModelMesh, const UE::CADKernel::FBody& Body, FBodyMesh& OutBodyMesh);

	/**
	 * Tessellate a CADKernel entity and update the MeshDescription
	 */
	static bool Tessellate(UE::CADKernel::FTopologicalShapeEntity& InCADKernelEntity, FMeshConversionContext& Context, FMeshDescription& OutMeshDescription);

	static uint32 GetFaceTessellation(UE::CADKernel::FFaceMesh& FaceMesh, FBodyMesh& OutBodyMesh, FObjectDisplayDataId FaceMaterial);
};
}
