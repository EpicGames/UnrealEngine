// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADOptions.h"
#include "MeshDescription.h"

struct FCTMesh;
class FBodyMesh;
class FCADMaterial;
class IDatasmithMaterialIDElement;
class IDatasmithUEPbrMaterialElement;
class IDatasmithScene;

#define LAST_CT_MATERIAL_ID 0x00ffffff

namespace CADLibrary
{

using TColorMap = TMap<uint32, uint32>;

struct FMeshConversionContext
{
	const FImportParameters& ImportParameters;
	const FMeshParameters& MeshParameters;
	TArray<int32> VertexIds;
	TArray<int32> SymmetricVertexIds;
	TSet<int32> PatchesToMesh;
	double GeometricTolerance = 0.001;

	FMeshConversionContext(const FImportParameters& InImportParams, const FMeshParameters& InMeshParameters, double InGeometricTolerance = 0.001)
		: ImportParameters(InImportParams)
		, MeshParameters(InMeshParameters)
		, GeometricTolerance(InGeometricTolerance)
	{
	}
};

/**
 * Macro data of MeshDescription.
 * Needed to update a MeshDesction after a conversion of a DynamicMesh To a MeshDescription.
 * Indeed, PolygonGroupId can be modify during the process of DynamicMesh and material slot names are not saved in a DynamicMesh.
 * The only persistent data is the patchGroupId
 * 
 * This class save the PatchGroupToPolygonGroup map and the MaterialSlotNames of the MeshDescription before the conversion into a DynamicMesh 
 * This information allows to restor the material slot name used by each PolygonGroup
 *
 * The call of UpdateMaterialSlotNames restor the information in the new or update MeshDescription
 */
class CADLIBRARY_API FMeshDescriptionDataCache
{
private:
	/** the mapping between PatchId and PolygonGroupId */
	TMap<int32, FPolygonGroupID> PatchGroupToPolygonGroup;

	/** the mapping between PolygonGroupId and SlotNames */
	TArray<FName> MaterialSlotNames;

	FName EmptyName;

	const FPolygonGroupID* Find(const int32 PatchGroupId) const 
	{
		return PatchGroupToPolygonGroup.Find(PatchGroupId);
	}

	const FName& GetSlotName(const FPolygonGroupID& GroupID) const
	{
		if(GroupID < MaterialSlotNames.Num())
		{
			return MaterialSlotNames[GroupID];
		}
		return EmptyName;
	}

public:

	FMeshDescriptionDataCache(FMeshDescription& Mesh);

	void RestoreMaterialSlotNames(FMeshDescription& MeshDestination) const;
};

CADLIBRARY_API TSharedPtr<IDatasmithUEPbrMaterialElement> CreateDefaultUEPbrMaterial();
CADLIBRARY_API TSharedPtr<IDatasmithUEPbrMaterialElement> CreateUEPbrMaterialFromColor(const FColor& InColor);
CADLIBRARY_API TSharedPtr<IDatasmithUEPbrMaterialElement> CreateUEPbrMaterialFromMaterial(FCADMaterial& InMaterial, TSharedRef<IDatasmithScene> Scene);

CADLIBRARY_API bool ConvertBodyMeshToMeshDescription(const FMeshConversionContext& MeshConversionContext, FBodyMesh& Body, FMeshDescription& MeshDescription);

/** 
 * Enable per-triangle integer attribute named PolyTriGroups 
 * This integer defines the identifier of the PolyTriGroup containing the triangle.
 * In case of mesh coming from a CAD file, a PolyTriGroup is associated to a CAD topological face
 */
CADLIBRARY_API TPolygonAttributesRef<int32> EnableCADPatchGroups(FMeshDescription& OutMeshDescription);
CADLIBRARY_API void GetExistingPatches(FMeshDescription& Mesh, TSet<int32>& OutPatchIdSet);

} // namespace CADLibrary
