// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class CADTools : ModuleRules
	{
		public CADTools(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
                    "Core",
					"DatasmithCore"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
                }
			);
        }
    }
}
