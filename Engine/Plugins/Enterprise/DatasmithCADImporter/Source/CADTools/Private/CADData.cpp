// Copyright Epic Games, Inc. All Rights Reserved.
#include "CADData.h"

#include "CADOptions.h"

#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

uint32 MeshArchiveMagic = 345612;

namespace CADLibrary
{

FMaterialUId BuildColorFastUId(uint32 ColorId, uint8 Alpha)
{
	if (Alpha == 0)
	{
		Alpha = 1;
	}
	uint32 FastColorId =  ColorId | Alpha << 24;
	return FMath::Abs((int32) ::GetTypeHash(FastColorId));
}

FMaterialUId BuildColorUId(const FColor& Color)
{
	return FMath::Abs((int32) GetTypeHash(Color));
}

FMaterialUId BuildMaterialUId(const FCADMaterial& Material)
{
	using ::GetTypeHash;

	uint32 MaterialUId = 0;
	if (!Material.MaterialName.IsEmpty())
	{
		MaterialUId = FCrc::Strihash_DEPRECATED(*Material.MaterialName); // we add material name because it could be used by the end user so two material with same parameters but different name are different.
	}

	MaterialUId = HashCombine(MaterialUId, GetTypeHash(Material.Diffuse));
	MaterialUId = HashCombine(MaterialUId, GetTypeHash(Material.Ambient));
	MaterialUId = HashCombine(MaterialUId, GetTypeHash(Material.Specular));
	MaterialUId = HashCombine(MaterialUId, GetTypeHash((int)(Material.Shininess * 255.0)));
	MaterialUId = HashCombine(MaterialUId, GetTypeHash((int)(Material.Transparency * 255.0)));
	MaterialUId = HashCombine(MaterialUId, GetTypeHash((int)(Material.Reflexion * 255.0)));

	if (!Material.TextureName.IsEmpty())
	{
		MaterialUId = HashCombine(MaterialUId, FCrc::Strihash_DEPRECATED(*Material.TextureName));
	}
	return FMath::Abs((int32)MaterialUId);
}

FArchive& operator<<(FArchive& Ar, FArchiveGraphicProperties& Object)
{
	Ar << Object.ColorUId;
	Ar << Object.MaterialUId;
	Ar << Object.bIsRemoved;
	Ar << Object.bShow;
	Ar << Object.Inheritance;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FCADMaterial& Material)
{
	Ar << Material.MaterialName;
	Ar << Material.Diffuse;
	Ar << Material.Ambient;
	Ar << Material.Specular;
	Ar << Material.Shininess;
	Ar << Material.Transparency;
	Ar << Material.Reflexion;
	Ar << Material.TextureName;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FFileDescriptor& File)
{
	Ar << File.SourceFilePath;
	Ar << File.CacheFilePath;
	Ar << File.Name;
	Ar << File.Configuration;
	Ar << File.Format;
	Ar << File.RootFolder;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FTessellationData& TessellationData)
{
	Ar << TessellationData.PositionArray;

	Ar << TessellationData.PositionIndices;
	Ar << TessellationData.VertexIndices;

	Ar << TessellationData.NormalArray;
	Ar << TessellationData.TexCoordArray;

	Ar << TessellationData.ColorUId;
	Ar << TessellationData.MaterialUId;

	Ar << TessellationData.PatchId;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FBodyMesh& BodyMesh)
{
	Ar << BodyMesh.bIsFromCad;

	Ar << BodyMesh.VertexArray;
	Ar << BodyMesh.Faces;

	Ar << BodyMesh.TriangleCount;
	Ar << BodyMesh.BodyID;
	Ar << BodyMesh.MeshActorUId;

	Ar << BodyMesh.MaterialSet;
	Ar << BodyMesh.ColorSet;

	return Ar;
}

void SerializeBodyMeshSet(const TCHAR* Filename, TArray<FBodyMesh>& InBodySet)
{
	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileWriter(Filename));

	uint32 MagicNumber = MeshArchiveMagic;
	*Archive << MagicNumber;

	*Archive << InBodySet;

	Archive->Close();
}

void DeserializeBodyMeshFile(const TCHAR* Filename, TArray<FBodyMesh>& OutBodySet)
{
	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileReader(Filename));

	uint32 MagicNumber = 0;
	*Archive << MagicNumber;
	if (MagicNumber != MeshArchiveMagic)
	{
		Archive->Close();
		return;
	}

	*Archive << OutBodySet;
	Archive->Close();
}

// Duplicated with FDatasmithUtils::GetCleanFilenameAndExtension, to delete as soon as possible
void GetCleanFilenameAndExtension(const FString& InFilePath, FString& OutFilename, FString& OutExtension)
{
	if (InFilePath.IsEmpty())
	{
		OutFilename.Empty();
		OutExtension.Empty();
		return;
	}

	FString BaseFile = FPaths::GetCleanFilename(InFilePath);
	BaseFile.Split(TEXT("."), &OutFilename, &OutExtension, ESearchCase::CaseSensitive, ESearchDir::FromEnd);

	if (!OutExtension.IsEmpty() && FCString::IsNumeric(*OutExtension))
	{
		BaseFile = OutFilename;
		FString NewExtension;
		BaseFile.Split(TEXT("."), &OutFilename, &NewExtension, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (!NewExtension.IsEmpty())
		{
			OutExtension = NewExtension + TEXT(".*");
		}
	}
}

FString GetExtension(const FString& InFilePath)
{
	if (InFilePath.IsEmpty())
	{
		return FString();
	}

	FString Extension;
	FString BaseFileWithoutExt;

	FString BaseFile = FPaths::GetCleanFilename(InFilePath);
	BaseFile.Split(TEXT("."), &BaseFileWithoutExt, &Extension, ESearchCase::CaseSensitive, ESearchDir::FromEnd);

	if (!Extension.IsEmpty() && FCString::IsNumeric(*Extension))
	{
		FString NewExtension;
		BaseFileWithoutExt.Split(TEXT("."), &BaseFile, &NewExtension, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (!NewExtension.IsEmpty())
		{
			NewExtension += TEXT(".*");
			return NewExtension;
		}
	}
	return Extension;
}

uint32 GetTypeHash(const FFileDescriptor& FileDescriptor)
{
	using ::GetTypeHash;
	FFileStatData FileStatData = IFileManager::Get().GetStatData(*FileDescriptor.SourceFilePath);

	uint32 DescriptorHash = FCrc::Strihash_DEPRECATED(*FileDescriptor.Name);
	DescriptorHash = HashCombine(DescriptorHash, FCrc::Strihash_DEPRECATED(*FileDescriptor.Configuration));
	DescriptorHash = HashCombine(DescriptorHash, GetTypeHash(FileStatData.FileSize));
	DescriptorHash = HashCombine(DescriptorHash, GetTypeHash(FileStatData.ModificationTime));

	return DescriptorHash;
}

void FFileDescriptor::SetFileFormat(const FString& Extension)
{
	if (Extension == TEXT("catpart") )
	{
		Format = ECADFormat::CATIA;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("catproduct"))
	{
		Format = ECADFormat::CATIA;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("catshape"))
	{
		Format = ECADFormat::CATIA;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("cgr"))
	{
		Format = ECADFormat::CATIA_CGR;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("iges") || Extension == TEXT("igs"))
	{
		Format = ECADFormat::IGES;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("step") || Extension == TEXT("stp")
		|| Extension == TEXT("stpz") || Extension == TEXT("stpx") || Extension == TEXT("stpxz"))
	{
		Format = ECADFormat::STEP;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("ipt"))
	{
		Format = ECADFormat::INVENTOR;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("iam"))
	{
		// Importer.m_sLoadData.m_sIncremental not supported
		Format = ECADFormat::INVENTOR;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("jt"))
	{
		// Importer.m_sLoadData.m_sIncremental not supported
		Format = ECADFormat::JT;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("model"))
	{
		Format = ECADFormat::CATIAV4;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("exp") || Extension == TEXT("session") || Extension == TEXT("dlv"))
	{
		Format = ECADFormat::CATIAV4;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("prt.*")
		|| Extension == TEXT("xpr"))
	{
		Format = ECADFormat::CREO;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("asm.*")
		|| Extension == TEXT("creo") || Extension == TEXT("creo.*")
		|| Extension == TEXT("neu") || Extension == TEXT("neu.*")
		|| Extension == TEXT("xas"))
	{
		Format = ECADFormat::CREO;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("prt"))
	{
		Format = ECADFormat::N_X;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("sat") || Extension == TEXT("sab"))
	{
		Format = ECADFormat::ACIS;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("sldprt") )
	{
		Format = ECADFormat::SOLIDWORKS;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("sldasm"))
	{
		Format = ECADFormat::SOLIDWORKS;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("x_t") || Extension == TEXT("x_b")
		|| Extension == TEXT("xmt") || Extension == TEXT("xmt_txt"))
	{
		Format = ECADFormat::PARASOLID;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("3dxml") || Extension == TEXT("3drep"))
	{
		// Importer.m_sLoadData.m_sIncremental not supported
		Format = ECADFormat::CATIA_3DXML;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("par") )
	{
		Format = ECADFormat::SOLID_EDGE;
		bCanReferenceOtherFiles = false;
	}
	else if (Extension == TEXT("psm"))
	{
		Format = ECADFormat::SOLID_EDGE;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("dwg") || Extension == TEXT("dxf"))
	{
		Format = ECADFormat::AUTOCAD;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("ifc") || Extension == TEXT("ifczip"))
	{
		Format = ECADFormat::IFC;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("dgn"))
	{
		Format = ECADFormat::MICROSTATION;
		bCanReferenceOtherFiles = true;
	}
	else if (Extension == TEXT("hsf") || Extension == TEXT("prc"))
	{
		Format = ECADFormat::TECHSOFT;
		bCanReferenceOtherFiles = false;
	}
	else
	{
		Format = ECADFormat::OTHER;
		bCanReferenceOtherFiles = true;
	}
}

}
