// Copyright Epic Games, Inc. All Rights Reserved.
#include "CADOptions.h"

namespace CADLibrary
{

int32 GMaxImportThreads = 0;
FAutoConsoleVariableRef GCADTranslatorMaxImportThreads(
	TEXT("ds.CADTranslator.MaxImportThreads"),
	GMaxImportThreads,
	TEXT("\
CAD file parallel processing\n\
Default is MaxImportThreads = 0\n\
0: multi-processing, n : multi-processing limited to n process. EnableCADCache is mandatory.\n\
1: -if EnableCADCache is true, the scene is read in a sequential mode with cache i.e. cache is used for sub-files already read,\n\
   -if EnableCADCache is false, the scene is read all at once\n"),
	ECVF_Default);

bool FImportParameters::bGDisableCADKernelTessellation = false;
FAutoConsoleVariableRef GCADTranslatorDisableCADKernelTessellation(
	TEXT("ds.CADTranslator.DisableCADKernelTessellation"),
	FImportParameters::bGDisableCADKernelTessellation,
	TEXT("\
Disable to use CAD import library tessellator.\n\
Default is enable\n"),
	ECVF_Default);

bool FImportParameters::bGEnableCADCache = true;
FAutoConsoleVariableRef GCADTranslatorEnableCADCache(
	TEXT("ds.CADTranslator.EnableCADCache"),
	FImportParameters::bGEnableCADCache,
	TEXT("\
Enable/disable temporary CAD processing file cache. These file will be use in a next import to avoid CAD file processing.\n\
If MaxImportThreads != 1, EnableCADCache value is ignored\n\
Default is enable\n"),
	ECVF_Default);

bool FImportParameters::bGOverwriteCache = false;
FAutoConsoleVariableRef GCADTranslatorOverwriteCache(
	TEXT("ds.CADTranslator.OverwriteCache"),
	FImportParameters::bGOverwriteCache,
	TEXT("Overwrite any existing cache associated with the file being imported.\n"),
	ECVF_Default);

bool FImportParameters::bGEnableTimeControl = true;
FAutoConsoleVariableRef GCADTranslatorEnableTimeControl(
	TEXT("ds.CADTranslator.EnableTimeControl"),
	FImportParameters::bGEnableTimeControl,
	TEXT("Enable the timer that kill the worker if the import time is unusually long. With this time control, the load of the corrupted file is canceled but the rest of the scene is imported.\n"),
	ECVF_Default);

bool FImportParameters::bGPreferJtFileEmbeddedTessellation = false;
FAutoConsoleVariableRef GCADTranslatorJtFileEmbeddedTessellation(
	TEXT("ds.CADTranslator.PreferJtFileEmbeddedTessellation"),
	FImportParameters::bGPreferJtFileEmbeddedTessellation,
	TEXT("If both (tessellation and BRep) exist in the file, import embedded tessellation instead of meshing BRep.\n"),
	ECVF_Default);

float FImportParameters::GStitchingTolerance = 0.001f;
FAutoConsoleVariableRef GCADTranslatorStitchingTolerance(
	TEXT("ds.CADTranslator.StitchingTolerance"),
	FImportParameters::GStitchingTolerance,
	TEXT("Welding threshold for Heal/Sew stitching methods in cm\n\
Default value of StitchingTolerance is 0.001 cm\n"),
ECVF_Default);

bool FImportParameters::bGStitchingForceSew = true;
FAutoConsoleVariableRef GCADTranslatorStitchingForceSew(
	TEXT("ds.CADTranslator.Stitching.ForceSew"),
	FImportParameters::bGStitchingForceSew,
	TEXT("During the welding process, a second step is performed to try to fix last cracks with a largest threshold.\n\
For this second step, the threshold is increased by ds.CADTranslator.Stitching.ForceFactor.\n\
Only available with CADKernel.\n\
Default value of ForceSew is true\n"),
ECVF_Default);

float FImportParameters::GStitchingForceFactor = 5.f;
FAutoConsoleVariableRef GCADTranslatorStitchingForceFactor(
	TEXT("ds.CADTranslator.Stitching.ForceFactor"),
	FImportParameters::GStitchingForceFactor,
	TEXT("Factor to increase the tolerance during the welding process.\n\
Only available with CADKernel.\n\
Default value of ForceFactor is 5.\n"),
ECVF_Default);

bool FImportParameters::bGStitchingRemoveThinFaces = false;
FAutoConsoleVariableRef GCADTranslatorStitchingRemoveThinFaces(
	TEXT("ds.CADTranslator.Stitching.RemoveThinFaces"),
	FImportParameters::bGStitchingRemoveThinFaces,
	TEXT("During the welding process, Thin faces are removed before the stiching process. The width of the thin faces is equal to the force sew tolerance.\n\
Only available with CADKernel.\n\
Default value of RemoveThinFaces is true\n"),
ECVF_Default);

bool FImportParameters::bGStitchingRemoveDuplicatedFaces = false;
FAutoConsoleVariableRef GCADTranslatorStitchingRemoveDuplicatedFaces(
	TEXT("ds.CADTranslator.Stitching.RemoveDuplicatedFaces"),
	FImportParameters::bGStitchingRemoveDuplicatedFaces,
	TEXT("During the welding process, duplicated faces (i.e. faces with the same loops) are removed. \n\
Only available with CADKernel.\n\
Default value of RemoveDuplicatedFaces is true\n"),
ECVF_Default);

bool FImportParameters::bGActivateThinZoneMeshing = false;
FAutoConsoleVariableRef GActivateThinZoneMeshing(
	TEXT("ds.CADTranslator.Meshing.ActivateThinZoneMeshing"),
	FImportParameters::bGActivateThinZoneMeshing,
	TEXT("During the meshing process, faces with thin zones are meshed with a specific process for the thin zones. \n\
Only available with CADKernel.\n\
Default value of ActivateThinZoneMeshing is true\n"),
ECVF_Default);

float FImportParameters::GUnitScale = 1.f;
FAutoConsoleVariableRef GCADTranslatorUnitScale(
	TEXT("ds.CADTranslator.UnitScale"),
	FImportParameters::GUnitScale,
	TEXT("Scale factor to change the unit of the DMU (Only applies to TechSoft import.)\n\
Default value of UnitScale is 1 i.e. unit = cm\n"),
ECVF_Default);

bool FImportParameters::bGSewMeshIfNeeded = false;
FAutoConsoleVariableRef GCADTranslatorSewMeshIfNeeded(
	TEXT("ds.CADTranslator.SewMeshIfNeeded"),
	FImportParameters::bGSewMeshIfNeeded,
	TEXT("Perform a welding of the mesh to try to stitch mesh cracks\n\
This welding is performed respecting the ds.CADTranslator.StitchingTolerance\n\
Default value is false\n"),
ECVF_Default);

bool FImportParameters::bGRemoveDuplicatedTriangle = false;
FAutoConsoleVariableRef GCADTranslatorRemoveDuplicatedTriangle(
	TEXT("ds.CADTranslator.RemoveDuplicatedTriangle"),
	FImportParameters::bGRemoveDuplicatedTriangle,
	TEXT("Avoid duplicated triangles in a mesh (i.e. two triangle with the same triplet of vertices)\n\
This problem appears when a surface patch is duplicated. Due to the two identical mesh of the patches, the global mesh has duplicated triangles\n\
The time cost of this process is about 20% for 10e6 triangles with a complexity in nlog(n) due to the use of a map\n\
As this problem is rare, and to avoid the cost of the process, this option is disable by default. To fix StaticMesh with duplicated triangles, the StaticMesh can be remesh with this option.\n\
Default value is false. \n"),
ECVF_Default);

float FImportParameters::GMeshingParameterFactor = 1;
FAutoConsoleVariableRef GCADTranslatorMeshingParameterFactor(
	TEXT("ds.CADTranslator.MeshingParameterFactor"),
	FImportParameters::GMeshingParameterFactor,
	TEXT("Factor to allow to use smaller value than the defined minimal value of metric meshing parameters (i.e. Chord error > 0.005 cm, Max Edge Length > 1. cm) \n\
The used value of the meshing parameter is value * MeshingParameterFactor \n\
Default value is 1.\n"),
ECVF_Default);

int32 FImportParameters::GMaxMaterialCountPerMesh = 256;
FAutoConsoleVariableRef GCADTranslatorMaxMaterialCountPerMesh(
	TEXT("ds.CADTranslator.MaxMaterialCountPerMesh"),
	FImportParameters::GMaxMaterialCountPerMesh,
	TEXT("The main UE5 rendering systems do not support more than 256 materials per mesh. This CVar allow to define the max material count per mesh.\n\
Default value is 256\n"),
ECVF_Default);

bool FImportParameters::bValidationProcess = false;

uint32 GetTypeHash(const FImportParameters& ImportParameters)
{
	uint32 ParametersHash = ::GetTypeHash(ImportParameters.bGDisableCADKernelTessellation);
	ParametersHash = HashCombine(ParametersHash, ::GetTypeHash(ImportParameters.ChordTolerance));
	ParametersHash = HashCombine(ParametersHash, ::GetTypeHash(ImportParameters.MaxEdgeLength));
	ParametersHash = HashCombine(ParametersHash, ::GetTypeHash(ImportParameters.MaxNormalAngle));
	ParametersHash = HashCombine(ParametersHash, ::GetTypeHash(ImportParameters.StitchingTechnique));
	return ParametersHash;
}

}
