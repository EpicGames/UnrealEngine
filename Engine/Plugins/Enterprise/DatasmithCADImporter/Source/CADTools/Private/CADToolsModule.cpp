// Copyright Epic Games, Inc. All Rights Reserved.
#include "CADToolsModule.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FCADToolsModule"

FCADToolsModule& FCADToolsModule::Get()
{
	return FModuleManager::LoadModuleChecked< FCADToolsModule >(CADTOOLS_MODULE_NAME);
}

bool FCADToolsModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(CADTOOLS_MODULE_NAME);
}

uint32 FCADToolsModule::GetCacheVersion()
{
	return 580;
}

IMPLEMENT_MODULE(FCADToolsModule, CADTools);

#undef LOCTEXT_NAMESPACE

