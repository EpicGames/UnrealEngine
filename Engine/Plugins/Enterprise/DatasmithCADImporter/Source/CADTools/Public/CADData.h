// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Containers/Array.h"
#include "Math/Color.h"
#include "Misc/Paths.h"

class FArchive;

using FCadId = uint32; // Identifier defined in the input CAD file
using FMaterialUId = int32; // Material of Color unique Identifier defined based on the material or color properties
using FCadUuid = uint32;  // Unique identifier that be used for the unreal asset name (Actor, Material)

namespace CADLibrary
{

enum class ECADFormat
{
	ACIS,
	AUTOCAD,
	CATIA,
	CATIA_CGR,
	CATIA_3DXML,
	CATIAV4,
	CREO,
	DWG,
	DGN,
	TECHSOFT,
	IFC,
	IGES,
	INVENTOR,
	JT,
	N_X,  
	MICROSTATION,
	PARASOLID,
	SOLID_EDGE,
	SOLIDWORKS,
	STEP,
	OTHER
};

enum class ECADParsingResult : uint8
{
	Unknown,
	Running,
	UnTreated,
	ProcessOk,
	ProcessFailed,
	FileNotFound,
};

enum EComponentType : uint8
{
	Reference = 0,
	Instance,
	OverriddeOccurence,
	Body,
	LastType
};

enum class ECADGraphicPropertyInheritance : uint8
{
	Unset,
	FatherHerit,
	ChildHerit,
};

// TODO: Remove from here and replace by DatasmithUtils::GetCleanFilenameAndExtension... But need to remove DatasmithCore dependancies 
CADTOOLS_API void GetCleanFilenameAndExtension(const FString& InFilePath, FString& OutFilename, FString& OutExtension);
CADTOOLS_API FString GetExtension(const FString& InFilePath);

class CADTOOLS_API FCADMaterial
{
public:
	friend CADTOOLS_API FArchive& operator<<(FArchive& Ar, FCADMaterial& Material);

public:
	FString MaterialName;
	FColor Diffuse = FColor(255, 255, 255);
	FColor Ambient = FColor(255, 255, 255);
	FColor Specular = FColor(255, 255, 255);
	float Shininess = 0.f;
	float Transparency = 0.f;
	float Reflexion = 0.f;
	FString TextureName;
};

struct CADTOOLS_API FObjectDisplayDataId
{
	FMaterialUId DefaultMaterialUId = 0;
	FMaterialUId MaterialUId = 0;
	FMaterialUId ColorUId = 0;
};

class CADTOOLS_API FArchiveGraphicProperties
{
public:
	FArchiveGraphicProperties() = default;

	FArchiveGraphicProperties(const FArchiveGraphicProperties& Parent)
		: ColorUId(Parent.ColorUId)
		, MaterialUId(Parent.MaterialUId)
	{
	}

	virtual ~FArchiveGraphicProperties() = default;

	friend CADTOOLS_API FArchive& operator<<(FArchive& Ar, FArchiveGraphicProperties& C);

public:
	FMaterialUId ColorUId = 0;
	FMaterialUId MaterialUId = 0;
	bool bIsRemoved = false;
	bool bShow = true;
	ECADGraphicPropertyInheritance Inheritance = ECADGraphicPropertyInheritance::Unset;

	/**
	 * If a graphic property is undefined, define it with Source property
	 */
	void DefineGraphicsPropertiesFromNoOverwrite(const FArchiveGraphicProperties& Source)
	{

		if (!ColorUId && !MaterialUId)
		{
			ColorUId = Source.ColorUId;
			MaterialUId = Source.MaterialUId;
		}
	}

	/**
	 * If a source property is defined, set the property with it
	 */
	void SetGraphicProperties(const FArchiveGraphicProperties& Source)
	{
		if (Source.ColorUId)
		{
			ColorUId = Source.ColorUId;
		}

		if (Source.MaterialUId)
		{
			MaterialUId = Source.MaterialUId;
		}
	}

	bool IsDeleted() const
	{
		return bIsRemoved;
	}

	bool IsShown() const
	{
		return bShow;
	}
};

class CADTOOLS_API FFileDescriptor
{
public:
	FFileDescriptor() = default;
	
	explicit FFileDescriptor(const TCHAR* InFilePath, const TCHAR* InConfiguration = nullptr, const TCHAR* InRootFolder = nullptr)
		: SourceFilePath(InFilePath)
		, Configuration(InConfiguration)
	{
		Name = FPaths::GetCleanFilename(InFilePath);
		SetFileFormat(GetExtension(InFilePath));
		RootFolder = InRootFolder ? InRootFolder : FPaths::GetPath(InFilePath);
	}

	/**
	 * Used define and then load the cache of the CAD File instead of the source file
	 */ 
	void SetCacheFile(const FString& InCacheFilePath)
	{
		CacheFilePath = InCacheFilePath;
	}

	bool operator==(const FFileDescriptor& Other) const
	{
		return (Name.Equals(Other.Name, ESearchCase::IgnoreCase) && (Configuration == Other.Configuration));
	}

	bool IsEmpty()
	{
		return Name.IsEmpty();
	}

	void Empty()
	{
		SourceFilePath.Empty(); 
		CacheFilePath.Empty();
		Name.Empty();
		Configuration.Empty();
		RootFolder.Empty();
		DescriptorHash = 0;
	}

	friend CADTOOLS_API FArchive& operator<<(FArchive& Ar, FFileDescriptor& File);

	friend CADTOOLS_API uint32 GetTypeHash(const FFileDescriptor& FileDescription);

	uint32 GetDescriptorHash() const
	{
		if (!DescriptorHash)
		{
			DescriptorHash = GetTypeHash(*this);
		}
		return DescriptorHash;
	}

	const FString& GetSourcePath() const 
	{
		return SourceFilePath;
	}
	
	bool HasConfiguration() const
	{
		return !Configuration.IsEmpty();
	}

	const FString& GetConfiguration() const
	{
		return Configuration;
	}

	void SetConfiguration(const FString& NewConfiguration)
	{
		Configuration = NewConfiguration;
	}

	ECADFormat GetFileFormat() const
	{
		return Format;
	}

	bool CanReferenceOtherFiles() const
	{
		return bCanReferenceOtherFiles;
	}

	const FString& GetPathOfFileToLoad() const
	{
		if (CacheFilePath.IsEmpty())
		{
			return SourceFilePath;
		}
		else
		{
			return CacheFilePath;
		}
	}

	/** Set the file path if SourceFilePath was not the real path */
	void SetSourceFilePath(const FString& NewFilePath)
	{
		SourceFilePath = NewFilePath;
	}

	const FString& GetRootFolder() const 
	{
		return RootFolder;
	}

	const FString& GetFileName() const
	{
		return Name;
	}

	void ChangePath(const FString& OldPath, const FString& NewPath)
	{
		int32 OldPathLength = OldPath.Len();
		FString SourceFileOldPath = SourceFilePath.Left(OldPathLength);
		FString SourceFileEndPath = SourceFilePath.Right(SourceFilePath.Len() - OldPathLength - 1);
		if (OldPath == SourceFileOldPath)
		{
			SourceFilePath = FPaths::Combine(NewPath, SourceFileEndPath);
		}
		RootFolder = NewPath;
	}

private:

	FString SourceFilePath; // e.g. d:/folder/content.jt
	FString CacheFilePath; // if the file has already been loaded 
	FString Name; // content.jt
	ECADFormat Format; // ECADFormat::JT
	bool bCanReferenceOtherFiles;
	FString Configuration; // dedicated to JT or SW file to read the good configuration (SW) or only a sub-file (JT)
	FString RootFolder; // alternative folder where the file could be if its path is not valid.

	mutable uint32 DescriptorHash = 0;

	void SetFileFormat(const FString& Extension);
};

/**
 * Helper struct to store tessellation data from CoreTech or CADKernel
 *
 * FBodyMesh and FTessellationData are design to manage mesh from CoreTech and CADKernel.
 * FTessellationData is the mesh of a face
 * FBodyMesh is the mesh of a body composed by an array of FTessellationData (one FTessellationData by body face)
 *
 * CoreTech mesh are defined surface by surface. The mesh is not connected
 * CADKernel mesh is connected.
 */
struct CADTOOLS_API FTessellationData : public FArchiveGraphicProperties
{
	friend CADTOOLS_API FArchive& operator<<(FArchive& Ar, FTessellationData& Tessellation);

	/** Empty with CADKernel as set in FBodyMesh, Set by CoreTech (this is only the vertices of the face) */
	TArray<FVector3f> PositionArray;

	/** Index of each vertex in FBody::VertexArray. Empty with CoreTech and filled by FillKioVertexPosition */
	TArray<int32> PositionIndices;

	/** Index of Vertices of each face in the local Vertices set (i.e. VerticesBodyIndex for CADKernel) */
	TArray<int32> VertexIndices;

	/** Normal of each vertex */
	TArray<FVector3f> NormalArray;

	/** UV coordinates of each vertex */
	TArray<FVector2f> TexCoordArray;

	int32 PatchId;
};

class CADTOOLS_API FBodyMesh
{
public:
	FBodyMesh(FCadId InBodyID = 0) : BodyID(InBodyID)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FBodyMesh& BodyMesh);

public:

	void AddGraphicPropertiesFrom(const FArchiveGraphicProperties& GraphicProperties)
	{
		if (GraphicProperties.ColorUId)
		{
			ColorSet.Add(GraphicProperties.ColorUId);
		}
		if (GraphicProperties.MaterialUId)
		{
			MaterialSet.Add(GraphicProperties.MaterialUId);
		}
	}

	bool bIsFromCad = true;

	TArray<FVector3f> VertexArray; // set by CADKernel, filled by FillKioVertexPosition that merges coincident vertices (CoreTechHelper)
	TArray<FTessellationData> Faces;

	uint32 TriangleCount = 0;
	FCadId BodyID = 0;
	FCadUuid MeshActorUId = 0;

	TArray<int32> VertexIds;  // StaticMesh FVertexID NO Serialize, filled by FillKioVertexPosition or FillVertexPosition
	TArray<int32> SymmetricVertexIds; // StaticMesh FVertexID for sym part NO Serialize, filled by FillKioVertexPosition or FillVertexPosition

	TSet<FMaterialUId> MaterialSet;
	TSet<FMaterialUId> ColorSet;
};

CADTOOLS_API FMaterialUId BuildColorFastUId(uint32 ColorId, uint8 Alpha);

CADTOOLS_API FMaterialUId BuildColorUId(const FColor& Color);
CADTOOLS_API FMaterialUId BuildMaterialUId(const FCADMaterial& Material);

CADTOOLS_API void SerializeBodyMeshSet(const TCHAR* Filename, TArray<FBodyMesh>& InBodySet);
CADTOOLS_API void DeserializeBodyMeshFile(const TCHAR* Filename, TArray<FBodyMesh>& OutBodySet);

}

