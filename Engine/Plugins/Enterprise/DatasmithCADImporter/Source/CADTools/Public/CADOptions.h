// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithUtils.h"
#include "HAL/IConsoleManager.h"
#include "Math/Vector.h"
#include "Misc/Paths.h"
#include "Templates/TypeHash.h"

namespace CADLibrary
{
	CADTOOLS_API extern int32 GMaxImportThreads;

	enum class EMesher
	{
		CADKernel,
		TechSoft,
		None
	};

	enum class EFailureReason : uint8
	{
		Curve3D,
		Unknown,
		None
	};

	enum EStitchingTechnique
	{
		StitchingNone = 0,
		StitchingHeal,
		StitchingSew,
	};

	enum class ESewOption : uint8  // Same as UE::CADKernel::ESewOption
	{
		None = 0x00u,	// No flags.

		ForceJoining = 0x01u,
		RemoveThinFaces = 0x02u,
		RemoveDuplicatedFaces = 0x04u,

		All = 0x07u
	};

	ENUM_CLASS_FLAGS(ESewOption);

	class CADTOOLS_API FImportParameters
	{
	private:
		double ChordTolerance;
		double MaxEdgeLength;
		double MaxNormalAngle;
		EStitchingTechnique StitchingTechnique;
		EMesher Mesher;
		FDatasmithUtils::EModelCoordSystem ModelCoordSys;
		
	public:
		static bool bGDisableCADKernelTessellation;
		static bool bGEnableCADCache;
		static bool bGEnableTimeControl;
		static bool bGOverwriteCache;
		static bool bGPreferJtFileEmbeddedTessellation;
		static bool bGSewMeshIfNeeded;
		static bool bGRemoveDuplicatedTriangle;
		static float GStitchingTolerance;
		static bool bGStitchingForceSew;
		static bool bGStitchingRemoveThinFaces;
		static bool bGStitchingRemoveDuplicatedFaces;
		static bool bGActivateThinZoneMeshing;
		static float GStitchingForceFactor;
		static float GUnitScale;
		static float GMeshingParameterFactor;
		static int32 GMaxMaterialCountPerMesh;
		static bool bValidationProcess;

	public:
		FImportParameters(FDatasmithUtils::EModelCoordSystem NewCoordinateSystem = FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded)
			: ChordTolerance(0.2)
			, MaxEdgeLength(0.0)
			, MaxNormalAngle(20.0)
			, StitchingTechnique(EStitchingTechnique::StitchingNone)
			, Mesher(EMesher::TechSoft)
			, ModelCoordSys(NewCoordinateSystem)
		{
		}
	
		FImportParameters(const FImportParameters& InParamneters, EMesher InMesher)
			: ChordTolerance(InParamneters.ChordTolerance)
			, MaxEdgeLength(InParamneters.MaxEdgeLength)
			, MaxNormalAngle(InParamneters.MaxNormalAngle)
			, StitchingTechnique(InParamneters.StitchingTechnique)
			, Mesher(InMesher)
			, ModelCoordSys(InParamneters.ModelCoordSys)
		{
		}

		void SetTesselationParameters(double InChordTolerance, double InMaxEdgeLength, double InMaxNormalAngle, EStitchingTechnique InStitchingTechnique)
		{
			ChordTolerance = InChordTolerance * GMeshingParameterFactor;
			MaxEdgeLength = InMaxEdgeLength * GMeshingParameterFactor;
			MaxNormalAngle = InMaxNormalAngle;
			StitchingTechnique = InStitchingTechnique;
			Mesher = FImportParameters::bGDisableCADKernelTessellation ? EMesher::TechSoft : EMesher::CADKernel;
		}

		void SetTesselationParameters(double InChordTolerance, double InMaxEdgeLength, double InMaxNormalAngle, EStitchingTechnique InStitchingTechnique, EMesher InMesher)
		{
			SetTesselationParameters(InChordTolerance, InMaxEdgeLength, InMaxNormalAngle, InStitchingTechnique);
			Mesher = InMesher;
		}

		uint32 GetHash() const
		{
			uint32 Hash = 0;
			for (double Param : {ChordTolerance, MaxEdgeLength, MaxNormalAngle, (double) GStitchingForceFactor})
			{
				Hash = HashCombine(Hash, ::GetTypeHash(Param));
			}
			for (uint32 Param : {uint32(StitchingTechnique), uint32(Mesher), uint32(ModelCoordSys)})
			{
				Hash = HashCombine(Hash, ::GetTypeHash(Param));
			}
			for (bool Param : {bGStitchingForceSew, bGStitchingRemoveThinFaces, bGStitchingRemoveDuplicatedFaces, bGDisableCADKernelTessellation, bGActivateThinZoneMeshing, bGPreferJtFileEmbeddedTessellation})
			{
				Hash = HashCombine(Hash, ::GetTypeHash(Param));
			}
			return Hash;
		}

		friend FArchive& operator<<(FArchive& Ar, FImportParameters& ImportParameters)
		{
			Ar << ImportParameters.ChordTolerance;
			Ar << ImportParameters.MaxEdgeLength;
			Ar << ImportParameters.MaxNormalAngle;
			Ar << (uint32&) ImportParameters.StitchingTechnique;
			Ar << (uint8&) ImportParameters.Mesher;
			Ar << (uint8&) ImportParameters.ModelCoordSys;

			// these static variables have to be serialized to be transmitted to CADWorkers
			// because CADWorker has not access to CVars
			Ar << ImportParameters.bGDisableCADKernelTessellation;
			Ar << ImportParameters.bGEnableCADCache;
			Ar << ImportParameters.bGEnableTimeControl;
			Ar << ImportParameters.bGOverwriteCache;
			Ar << ImportParameters.bGPreferJtFileEmbeddedTessellation;
			Ar << ImportParameters.bGRemoveDuplicatedTriangle;
			Ar << ImportParameters.bGSewMeshIfNeeded;
			Ar << ImportParameters.bGStitchingForceSew;
			Ar << ImportParameters.bGStitchingRemoveDuplicatedFaces;
			Ar << ImportParameters.bGStitchingRemoveThinFaces;
			Ar << ImportParameters.bGActivateThinZoneMeshing;
			Ar << ImportParameters.bValidationProcess;
			Ar << ImportParameters.GMaxMaterialCountPerMesh;
			Ar << ImportParameters.GMeshingParameterFactor;
			Ar << ImportParameters.GStitchingForceFactor;
			Ar << ImportParameters.GStitchingTolerance;
			Ar << ImportParameters.GUnitScale;

			return Ar;
		}

		double GetChordTolerance() const
		{
			return ChordTolerance;
		}

		double GetMaxNormalAngle() const
		{
			return MaxNormalAngle;
		}

		double GetMaxEdgeLength() const
		{
			return MaxEdgeLength;
		}

		EStitchingTechnique GetStitchingTechnique() const
		{
			return StitchingTechnique;
		}

		EMesher GetMesher() const
		{
			return Mesher;
		}

		FDatasmithUtils::EModelCoordSystem GetModelCoordSys() const
		{
			return ModelCoordSys;
		}

		void SetModelCoordinateSystem(FDatasmithUtils::EModelCoordSystem NewCoordinateSystem)
		{
			ModelCoordSys = NewCoordinateSystem;
		}

		CADTOOLS_API friend uint32 GetTypeHash(const FImportParameters& ImportParameters);
	};

	namespace SewOption
	{

	static ESewOption GetFromImportParameters()
	{
		ESewOption Option = ESewOption::None;

		if (FImportParameters::bGStitchingForceSew)
		{
			Option |= ESewOption::ForceJoining;
		}

		if (FImportParameters::bGStitchingRemoveThinFaces)
		{
			Option |= ESewOption::RemoveThinFaces;
		}

		if (FImportParameters::bGStitchingRemoveDuplicatedFaces)
		{
			Option |= ESewOption::RemoveDuplicatedFaces;
		}

		return Option;
	}

	} // ns SewOption

	inline FString BuildCadCachePath(const TCHAR* CachePath, uint32 FileHash)
	{
		FString FileName = FString::Printf(TEXT("UEx%08x"), FileHash) + TEXT(".prc");
		return FPaths::Combine(CachePath, TEXT("cad"), FileName);
	}

	inline FString BuildCacheFilePath(const TCHAR* CachePath, const TCHAR* Folder, uint32 BodyHash, const EMesher Mesher)
	{
		FString BodyFileName = FString::Printf(TEXT("UEx%08x"), BodyHash);
		FString OutFileName = FPaths::Combine(CachePath, Folder, BodyFileName);

		if (Mesher == EMesher::CADKernel)
		{
			OutFileName += TEXT(".ugeom");
		}
		else
		{
			OutFileName += TEXT(".prc");
		}
		return OutFileName;
	}

	struct FMeshParameters
	{
		bool bNeedSwapOrientation = false;
		bool bIsSymmetric = false;
		FVector3f SymmetricOrigin;
		FVector3f SymmetricNormal;
	};
}
