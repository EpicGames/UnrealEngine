// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#define CADTOOLS_MODULE_NAME TEXT("CADTools")

class CADTOOLS_API FCADToolsModule : public IModuleInterface
{
public:
	static FCADToolsModule& Get();
	static bool IsAvailable();

	static uint32 GetCacheVersion();
};
