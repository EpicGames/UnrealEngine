// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithCADTranslator : ModuleRules
	{
		public DatasmithCADTranslator(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CADKernelSurface",
					"CADInterfaces",
					"CADLibrary",
					"CADTools",
					"DatasmithCore",
					"DatasmithContent",
					"DatasmithDispatcher",
					"DatasmithTranslator",
					"Engine",
					"MeshDescription",
					"ParametricSurface",
					"Sockets",
				}
			);
		}
	}
}
