// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCADTranslator.h"

#include "CADFileReader.h"
#include "CADInterfacesModule.h"
#include "CADKernelSurfaceExtension.h"
#include "CADOptions.h"

#include "DatasmithCADTranslatorModule.h"
#include "DatasmithDispatcher.h"
#include "DatasmithMeshBuilder.h"
#include "DatasmithSceneGraphBuilder.h"
#include "DatasmithTranslator.h"
#include "DatasmithUtils.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "IDatasmithSceneElements.h"


DEFINE_LOG_CATEGORY(LogCADTranslator);

namespace DatasmithCADTranslatorImpl
{

static bool bGEnableUnsupportedCADFormats = false;
FAutoConsoleVariableRef GEnableUnsupportedCADFormats(
	TEXT("ds.CADTranslator.EnableUnsupportedCADFormats"),
	bGEnableUnsupportedCADFormats,
	TEXT("\
Enable/disable unsupported CAD formats i.e. formats that can be imported with the CAD library but haven't been selected as officialy supported.\n\
These formats stay \"experimental\" as they are out of our testing coverage. No support will be done on these formats.\n\
Default is disable\n"),
	ECVF_Default);
}

void FDatasmithCADTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
{
	TFunction<bool()> GetCADInterfaceAvailability = []() -> bool
	{
		if (ICADInterfacesModule::GetAvailability() == ECADInterfaceAvailability::Unavailable)
		{
			return false;
		}
		return true;
	};

	static bool bIsCADInterfaceAvailable = GetCADInterfaceAvailability();
	if (!bIsCADInterfaceAvailable)
	{
		OutCapabilities.bIsEnabled = false;
		return;
	}
	OutCapabilities.bIsEnabled = true;

#ifdef CAD_TRANSLATOR_DEBUG
	OutCapabilities.bParallelLoadStaticMeshSupported = true;
#endif
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("CATPart"), TEXT("CATIA Part files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("CATProduct"), TEXT("CATIA Product files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("CATShape"), TEXT("CATIA Shape files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("cgr"), TEXT("CATIA Graphical Representation V5 files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("3dxml"), TEXT("CATIA 3D xml files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("3drep"), TEXT("CATIA 3D xml files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("model"), TEXT("CATIA V4 files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("session"), TEXT("CATIA V4 files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("exp"), TEXT("CATIA V4 files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("dlv"), TEXT("CATIA V4 files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("asm.*"), TEXT("Creo Assembly files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("creo.*"), TEXT("Creo Assembly files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("creo"), TEXT("Creo Assembly files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("neu.*"), TEXT("Creo Assembly files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("neu"), TEXT("Creo Assembly files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("prt.*"), TEXT("Creo Part files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("xas"), TEXT("Creo Assembly files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("xpr"), TEXT("Creo Part files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("iam"), TEXT("Inventor Assembly files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("ipt"), TEXT("Inventor Part files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("iges"), TEXT("IGES files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("igs"), TEXT("IGES files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("jt"), TEXT("JT Open files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("sat"), TEXT("3D ACIS model files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("sab"), TEXT("3D ACIS model files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("SLDASM"), TEXT("SolidWorks Product files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("SLDPRT"), TEXT("SolidWorks Part files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("step"), TEXT("Step files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("stp"), TEXT("Step files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("stpz"), TEXT("Step files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("stpx"), TEXT("Step/XML files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("stpxz"), TEXT("Step/XML files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("xml"), TEXT("AP242 Xml Step files, XPDM files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("x_t"), TEXT("Parasolid files (Text format)") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("x_b"), TEXT("Parasolid files (Binary format)") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("xmt"), TEXT("Parasolid files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("xmt_txt"), TEXT("Parasolid files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("asm"), TEXT("Unigraphics, NX, SolidEdge Assembly files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("prt"), TEXT("Unigraphics, NX Part files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("par"), TEXT("SolidEdge Part files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("psm"), TEXT("SolidEdge Part files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("dwg"), TEXT("AutoCAD model files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("dxf"), TEXT("AutoCAD model files") });

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("ifc"), TEXT("IFC (Industry Foundation Classes)") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("ifczip"), TEXT("IFC (Industry Foundation Classes)") });
	
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("hsf"), TEXT("HOOPS stream files") });
	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("prc"), TEXT("HOOPS stream files") });

	if (DatasmithCADTranslatorImpl::bGEnableUnsupportedCADFormats)
	{
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("3mf"), TEXT("3D Manufacturing Format") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("3ds"), TEXT("Autodesk 3DS") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("dae"), TEXT("Collada") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("dwf"), TEXT("Autodesk DWF") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("dwfx"), TEXT("Autodesk DWF") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("nwd"), TEXT("Autodesk Navisworks") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("mf1"), TEXT("I-Deas") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("arc"), TEXT("I-Deas") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("unv"), TEXT("I-Deas") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("pkg"), TEXT("I-Deas") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("dgn"), TEXT("Microstation") });  // available with Hoops Exchange 2023
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("stl"), TEXT("Stereo Lithography (STL)") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("u3d"), TEXT("U3D (ECMA-363)") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("vda"), TEXT("VDA-FS") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("vrml"), TEXT("VRML") });
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("wrl"), TEXT("VRML") });
	}
}

bool FDatasmithCADTranslator::IsSourceSupported(const FDatasmithSceneSource& Source)
{
	if (Source.GetSourceFileExtension() != TEXT("xml"))
	{
		return true;
	}

	return Datasmith::CheckXMLFileSchema(Source.GetSourceFile(), TEXT("XPDMXML"), TEXT("ns3:Uos"));
}

bool FDatasmithCADTranslator::LoadScene(TSharedRef<IDatasmithScene> DatasmithScene)
{
	using namespace CADLibrary;
	const FDatasmithTessellationOptions& TesselationOptions = GetCommonTessellationOptions();
	CADLibrary::FFileDescriptor FileDescriptor(*FPaths::ConvertRelativePathToFull(GetSource().GetSourceFile()));

	UE_LOGF(LogCADTranslator, Display, "CAD translation [%ls].", *FileDescriptor.GetSourcePath());
	UE_LOGF(LogCADTranslator, Display, " - Parsing Library:      %ls", ICADInterfacesModule::GetLibraryVersion());
	UE_LOGF(LogCADTranslator, Display, " - Tessellation Library: %ls"
		, FImportParameters::bGDisableCADKernelTessellation ? TEXT("TechSoft") : TEXT("CADKernel"));
	UE_LOGF(LogCADTranslator, Display, " - Cache mode:           %ls"
		, FImportParameters::bGEnableCADCache ? (FImportParameters::bGOverwriteCache ? TEXT("Override") : TEXT("Enabled")) : TEXT("Disabled"));
	UE_LOGF(LogCADTranslator, Display, " - Processing:           %ls"
		, FImportParameters::bGEnableCADCache ? (GMaxImportThreads == 1 ? TEXT("Sequencial") : TEXT("Parallel")) : TEXT("Sequencial"));

	TFunction<double(double, double, const TCHAR*)> CheckParameterValue = [](double Value, double MinValue, const TCHAR* ParameterName) -> double
	{
		if (Value < MinValue)
		{
			UE_LOGF(LogCADTranslator, Warning, "%ls value (%lf) of tessellation parameters is smaller than the minimal value %lf. It's value is modified to respect the limit", ParameterName, Value, MinValue);
			return MinValue;
		}
		return Value;
	};
	
	ImportParameters.SetTesselationParameters(CheckParameterValue(TesselationOptions.ChordTolerance, UE::DatasmithTessellation::MinTessellationChord, TEXT("Chord tolerance")),
		FMath::IsNearlyZero(TesselationOptions.MaxEdgeLength) ? 0. : CheckParameterValue(TesselationOptions.MaxEdgeLength, UE::DatasmithTessellation::MinTessellationEdgeLength, TEXT("Max Edge Length")),
		CheckParameterValue(TesselationOptions.NormalTolerance, UE::DatasmithTessellation::MinTessellationAngle, TEXT("Max Angle")),
		(EStitchingTechnique)TesselationOptions.StitchingTechnique);
	ImportParameters.SetModelCoordinateSystem(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded);

	UE_LOGF(LogCADTranslator, Display, " - Import parameters:");
	UE_LOGF(LogCADTranslator, Display, "     - ChordTolerance:     %lf", ImportParameters.GetChordTolerance());
	UE_LOGF(LogCADTranslator, Display, "     - MaxEdgeLength:      %lf", ImportParameters.GetMaxEdgeLength());
	UE_LOGF(LogCADTranslator, Display, "     - MaxNormalAngle:     %lf", ImportParameters.GetMaxNormalAngle());
	FString StitchingTechnique;
	switch(ImportParameters.GetStitchingTechnique())
	{
		case EStitchingTechnique::StitchingHeal:
			StitchingTechnique = TEXT("Heal");
			break;
		case EStitchingTechnique::StitchingSew:
			StitchingTechnique = TEXT("Sew");
			break;
		default:
			StitchingTechnique = TEXT("None");
			break;
	}
	UE_LOGF(LogCADTranslator, Display, "     - StitchingTechnique: %ls", *StitchingTechnique);
	if (!FImportParameters::bGDisableCADKernelTessellation)
	{
		UE_LOGF(LogCADTranslator, Display, "     - Stitching Options:");
		UE_LOGF(LogCADTranslator, Display, "         - ForceSew:              %ls", ImportParameters.bGStitchingForceSew ? TEXT("True") : TEXT("False"));
		UE_LOGF(LogCADTranslator, Display, "         - RemoveThinFaces:       %ls", ImportParameters.bGStitchingRemoveThinFaces ? TEXT("True") : TEXT("False"));
		UE_LOGF(LogCADTranslator, Display, "         - RemoveDuplicatedFaces: %ls", ImportParameters.bGStitchingRemoveDuplicatedFaces ? TEXT("True") : TEXT("False"));
		UE_LOGF(LogCADTranslator, Display, "         - ForceFactor:           %f", ImportParameters.GStitchingForceFactor);
	}

	switch (FileDescriptor.GetFileFormat())
	{
	case CADLibrary::ECADFormat::N_X:
	{
		break;
	}

	case ECADFormat::SOLIDWORKS:
	{
		ImportParameters.SetModelCoordinateSystem(FDatasmithUtils::EModelCoordSystem::YUp_RightHanded);
		break;
	}

	case ECADFormat::INVENTOR:
	case ECADFormat::CREO:
	{
		ImportParameters.SetModelCoordinateSystem(FDatasmithUtils::EModelCoordSystem::YUp_RightHanded);
		break;
	}

	case ECADFormat::DWG:
	{
		break;
	}

	case ECADFormat::IFC:
	{
		ImportParameters.SetModelCoordinateSystem(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy);
		break;
	}

	default:
		break;
	}

	FString CachePath = FDatasmithCADTranslatorModule::Get().GetCacheDir();
	if (!CachePath.IsEmpty())
	{
		CachePath = FPaths::ConvertRelativePathToFull(CachePath);
	}

	// Use sequential translation (multi-processed or not)
	if (FImportParameters::bGEnableCADCache)
	{
		TMap<uint32, FString> CADFileToUEFileMap;
		{
			const EMesher Mesher = FImportParameters::bGDisableCADKernelTessellation ? EMesher::TechSoft : EMesher::CADKernel;
			DatasmithDispatcher::FDatasmithDispatcher Dispatcher(ImportParameters, CachePath, CADFileToUEFileMap, CADFileToUEGeomMap);
			Dispatcher.AddTask(FileDescriptor, Mesher);

			Dispatcher.Process(GMaxImportThreads != 1);
		}

		FDatasmithSceneGraphBuilder SceneGraphBuilder(CADFileToUEFileMap, CachePath, DatasmithScene, GetSource(), ImportParameters);
		SceneGraphBuilder.Build();

		MeshBuilderPtr = MakeUnique<FDatasmithMeshBuilder>(CADFileToUEGeomMap, CachePath, ImportParameters);

		return true;
	}

	FCADFileReader FileReader(ImportParameters, FileDescriptor, *FPaths::EnginePluginsDir(), CachePath);
	if (FileReader.ProcessFile() != ECADParsingResult::ProcessOk)
	{
		// todo: copied from FDatasmithDispatcher::LogMessages
		for (const TPair<uint8, FString>& Message : FileReader.GetCADFileData().GetMessages())
		{
			switch (Message.Key)
			{
			case ELogVerbosity::Fatal:
				UE_LOGF(LogCADTranslator, Fatal, "%ls", *Message.Value);
				break;
			case ELogVerbosity::Error:
				UE_LOGF(LogCADTranslator, Error, "%ls", *Message.Value);
				break;
			case ELogVerbosity::Warning:
				UE_LOGF(LogCADTranslator, Warning, "%ls", *Message.Value);
				break;
			case ELogVerbosity::Display:
				UE_LOGF(LogCADTranslator, Display, "%ls", *Message.Value);
				break;
			case ELogVerbosity::Log:
				UE_LOGF(LogCADTranslator, Log, "%ls", *Message.Value);
				break;
			case ELogVerbosity::Verbose:
				UE_LOGF(LogCADTranslator, Verbose, "%ls", *Message.Value);
				break;
			case ELogVerbosity::VeryVerbose:
			default:
				UE_LOGF(LogCADTranslator, VeryVerbose, "%ls", *Message.Value);
				break;
			}
		}

		return false;
	}

	FCADFileData& CADFileData = FileReader.GetCADFileData();
	FDatasmithSceneBaseGraphBuilder SceneGraphBuilder(&CADFileData.GetSceneGraphArchive(), CachePath, DatasmithScene, GetSource(), ImportParameters);
	SceneGraphBuilder.Build();

	MeshBuilderPtr = MakeUnique<FDatasmithMeshBuilder>(CADFileData.GetBodyMeshes(), ImportParameters);

	return true;
}

void FDatasmithCADTranslator::UnloadScene()
{
	MeshBuilderPtr.Reset();

	CADFileToUEGeomMap.Empty();
}

bool FDatasmithCADTranslator::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	if (!MeshBuilderPtr.IsValid())
	{
		return false;
	}

	CADLibrary::FMeshParameters MeshParameters;

	// Two meshers can be used (TechSoft and CADKernel) during the import indeed in case of failure of one, the other is tried.
	// At this stage, the only clue to know which mesher has been used for the current body is the extension of the rawdata file
	const TCHAR* FileName = MeshElement->GetFile();
	FString Extension = FPaths::GetExtension(FileName);

	if (TOptional< FMeshDescription > Mesh = MeshBuilderPtr->GetMeshDescription(MeshElement, MeshParameters))
	{
		OutMeshPayload.LodMeshes.Add(MoveTemp(Mesh.GetValue()));

		if (Extension == TEXT("prc"))
		{
			ParametricSurfaceUtils::AddSurfaceData(MeshElement->GetFile(), ImportParameters, MeshParameters, GetCommonTessellationOptions(), OutMeshPayload);
		}
		else
		{
			CADKernelSurface::AddSurfaceDataForMesh(MeshElement->GetFile(), ImportParameters, MeshParameters, GetCommonTessellationOptions(), OutMeshPayload);
		}
	}

	return OutMeshPayload.LodMeshes.Num() > 0;
}

