// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CADOptions.h"
#include "DatasmithMeshBuilder.h"
#include "DatasmithSceneGraphBuilder.h"
#include "ParametricSurfaceTranslator.h"
#include "UObject/ObjectMacros.h"

class IDatasmithMeshElement;
class IDatasmithScene;
struct FDatasmithMeshElementPayload;

DECLARE_LOG_CATEGORY_EXTERN(LogCADTranslator, Log, All);


class FDatasmithCADTranslator : public FParametricSurfaceTranslator
{
public:
	virtual FName GetFName() const override { return "DatasmithCADTranslator"; };

	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;
	virtual bool IsSourceSupported(const FDatasmithSceneSource& Source) override;

	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;

	virtual void UnloadScene() override;

	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) override;

private:
	TMap<uint32, FString> CADFileToUEGeomMap;

	CADLibrary::FImportParameters ImportParameters;

	TUniquePtr<FDatasmithMeshBuilder> MeshBuilderPtr;
};

