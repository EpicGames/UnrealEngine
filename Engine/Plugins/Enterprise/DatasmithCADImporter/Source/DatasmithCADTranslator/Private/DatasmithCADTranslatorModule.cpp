// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCADTranslatorModule.h"

#include "CADOptions.h"
#include "CADToolsModule.h"
#include "DatasmithCADTranslator.h"

#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


IMPLEMENT_MODULE(FDatasmithCADTranslatorModule, DatasmithCADTranslator);

void FDatasmithCADTranslatorModule::StartupModule()
{
	const int32 CacheVersion = FCADToolsModule::Get().GetCacheVersion();

	// Delete incompatible cache directory 
	const FString Request = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DatasmithCADCache"), TEXT("*")));
	const FString CurrentCadCacheName = FString::FromInt(CacheVersion);

	TArray<FString> CadCacheContents;
	IFileManager::Get().FindFiles(CadCacheContents, *Request, false, true);

	for (const FString& Directory : CadCacheContents)
	{
		if (Directory != CurrentCadCacheName)
		{
			const FString OldCacheDirectory = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DatasmithCADCache"), *Directory));
			IFileManager::Get().DeleteDirectory(*OldCacheDirectory, true, true);
		}
	}

	// Create root cache directory which will be used by cad library sdk to store import data
	CacheDir = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DatasmithCADCache"), *FString::FromInt(CacheVersion)));
	if (!IFileManager::Get().MakeDirectory(*CacheDir, true))
	{
		CacheDir.Empty();
		CADLibrary::FImportParameters::bGEnableCADCache = false; // very weak protection: user could turn that on later, while the cache path is invalid
	}

	// Create body cache directory since this one is used even if bGEnableCADCache is false
	if (!CacheDir.IsEmpty())
	{
		IFileManager::Get().MakeDirectory(*FPaths::Combine(CacheDir, TEXT("body")), true);
	}

	Datasmith::RegisterTranslator<FDatasmithCADTranslator>();
}

void FDatasmithCADTranslatorModule::ShutdownModule()
{
	Datasmith::UnregisterTranslator<FDatasmithCADTranslator>();
}


FString FDatasmithCADTranslatorModule::GetCacheDir() const
{
	return CacheDir;
}

