// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithMeshBuilder.h"

#include "CADData.h"
#include "IDatasmithSceneElements.h"
#include "CADMeshDescriptionHelper.h"
#include "Utility/DatasmithMeshHelper.h"

#include "HAL/FileManager.h"
#include "MeshDescription.h"
#include "Misc/Paths.h"

FDatasmithMeshBuilder::FDatasmithMeshBuilder(TMap<uint32, FString>& CADFileToMeshFile, const FString& InCachePath, const CADLibrary::FImportParameters& InImportParameters)
	: CachePath(InCachePath)
	, ImportParameters(InImportParameters)
{
	LoadMeshFiles(CADFileToMeshFile);
}

void FDatasmithMeshBuilder::LoadMeshFiles(TMap<uint32, FString>& CADFileToMeshFile)
{
	BodyMeshes.Reserve(CADFileToMeshFile.Num());

	for (const auto& FilePair : CADFileToMeshFile)
	{
		FString MeshFile = FPaths::Combine(CachePath, TEXT("mesh"), FilePair.Value + TEXT(".gm"));
		if (!IFileManager::Get().FileExists(*MeshFile))
		{
			continue;
		}
		TArray<CADLibrary::FBodyMesh>& BodyMeshSet = BodyMeshes.Emplace_GetRef();
		DeserializeBodyMeshFile(*MeshFile, BodyMeshSet);
		for (CADLibrary::FBodyMesh& Body : BodyMeshSet)
		{
			MeshActorNameToBodyMesh.Emplace(Body.MeshActorUId, &Body);
		}
	}
}

TOptional<FMeshDescription> FDatasmithMeshBuilder::GetMeshDescription(TSharedRef<IDatasmithMeshElement> OutMeshElement, CADLibrary::FMeshParameters& OutMeshParameters)
{
	const TCHAR* NameLabel = OutMeshElement->GetName();
	FCadUuid BodyUuid = (FCadUuid) FCString::Atoi64(OutMeshElement->GetName() + 2);  // +2 to remove 2 first char (Ox)
	if (BodyUuid == 0)
	{
		return TOptional<FMeshDescription>();
	}

	CADLibrary::FBodyMesh** PPBody = MeshActorNameToBodyMesh.Find(BodyUuid);
	if(PPBody == nullptr || *PPBody == nullptr)
	{
		return TOptional<FMeshDescription>();
	}

	CADLibrary::FBodyMesh& Body = **PPBody;

	// FDatasmithSceneBaseGraphBuilder::BuildBody is performing a special treatment for
	// FBodyMesh without color. Replicate the treatment here too.
	int32 Num = Body.ColorSet.Num() + Body.MaterialSet.Num();
	if (!Num)
	{
		ensure(OutMeshElement->GetMaterialSlotCount() == 1);
		int32 MaterialSlotId = OutMeshElement->GetMaterialSlotAt(0)->GetId();

		Body.ColorSet.Add(MaterialSlotId);

		for (CADLibrary::FTessellationData& Face : Body.Faces)
		{
			Face.ColorUId = MaterialSlotId;
		}
	}

	FMeshDescription MeshDescription;
	DatasmithMeshHelper::PrepareAttributeForStaticMesh(MeshDescription);

	CADLibrary::FMeshConversionContext ConversionContext(ImportParameters, OutMeshParameters);

	if (ConvertBodyMeshToMeshDescription(ConversionContext, Body, MeshDescription))
	{
		return MoveTemp(MeshDescription);
	}

	return TOptional<FMeshDescription>();
}
