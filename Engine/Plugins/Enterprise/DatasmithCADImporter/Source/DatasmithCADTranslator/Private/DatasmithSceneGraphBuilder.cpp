// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithSceneGraphBuilder.h"

#include "CADData.h"
#include "CADSceneGraph.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneSource.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"
#include "CADMeshDescriptionHelper.h"
#include "Misc/FileHelper.h"

namespace DatasmithSceneGraphBuilderImpl
{

/**
 * For more details:
 * @see https://docs.techsoft3d.com/exchange/latest/build/managing_attribute_inheritance.html
 */
void SpreadGraphicProperties(const CADLibrary::FArchiveCADObject& Component, FActorData& ComponentData)
{
	if (!ComponentData.ColorUId)
	{
		ComponentData.ColorUId = Component.ColorUId;
	}
	if (!ComponentData.MaterialUId)
	{
		ComponentData.MaterialUId = Component.MaterialUId;
	}

	if (Component.Inheritance == CADLibrary::ECADGraphicPropertyInheritance::ChildHerit)
	{
		if (Component.ColorUId)
		{
			ComponentData.ColorUId = Component.ColorUId;
		}
		if (Component.MaterialUId)
		{
			ComponentData.MaterialUId = Component.MaterialUId;
		}
		ComponentData.Inheritance = CADLibrary::ECADGraphicPropertyInheritance::ChildHerit;
	}
	if (ComponentData.Inheritance == CADLibrary::ECADGraphicPropertyInheritance::ChildHerit)
	{
		return;
	}

	if (Component.Inheritance == CADLibrary::ECADGraphicPropertyInheritance::FatherHerit)
	{
		if (Component.ColorUId)
		{
			ComponentData.ColorUId = Component.ColorUId;
		}
		if (Component.MaterialUId)
		{
			ComponentData.MaterialUId = Component.MaterialUId;
		}
		ComponentData.Inheritance = CADLibrary::ECADGraphicPropertyInheritance::FatherHerit;
	}
	if (ComponentData.Inheritance == CADLibrary::ECADGraphicPropertyInheritance::FatherHerit)
	{
		return;
	}

	if (!ComponentData.ColorUId)
	{
		ComponentData.ColorUId = Component.ColorUId;
	}
	if (!ComponentData.MaterialUId)
	{
		ComponentData.MaterialUId = Component.MaterialUId;
	}
}

void AddTransformToActor(const CADLibrary::FArchiveCADObject& Object, TSharedPtr<IDatasmithActorElement> Actor, const CADLibrary::FImportParameters& ImportParameters)
{
	if (!Actor.IsValid())
	{
		return;
	}

	FTransform LocalTransform(Object.TransformMatrix);
	FTransform LocalUETransform = FDatasmithUtils::ConvertTransform(ImportParameters.GetModelCoordSys(), LocalTransform);

	Actor->SetTranslation(LocalUETransform.GetTranslation());
	Actor->SetScale(LocalUETransform.GetScale3D());
	Actor->SetRotation(LocalUETransform.GetRotation());
}

void GetNodeUuidAndLabel(CADLibrary::FArchiveInstance& Instance, CADLibrary::FArchiveCADObject& Reference, const TCHAR* InParentUEUUID, FString& OutUEUUID, FString& OutLabel)
{
	if (Instance.IsNameDefined())
	{
		OutLabel = Instance.Label;
	}
	else if (Reference.IsNameDefined())
	{
		OutLabel = Reference.Label;
	}
	else
	{
		OutLabel = TEXT("NoName");
	}

	FCadUuid UeUuid = HashCombine(FCrc::Strihash_DEPRECATED(InParentUEUUID), GetTypeHash(Instance.Id));
	UeUuid = HashCombine(UeUuid, GetTypeHash(Reference.Id));

	OutUEUUID = FString::Printf(TEXT("0x%08x"), UeUuid);
}
}

FDatasmithSceneGraphBuilder::FDatasmithSceneGraphBuilder(
	TMap<uint32, FString>& InCADFileToUnrealFileMap,
	const FString& InCachePath,
	TSharedRef<IDatasmithScene> InScene,
	const FDatasmithSceneSource& InSource,
	const CADLibrary::FImportParameters& InImportParameters)
	: FDatasmithSceneBaseGraphBuilder(nullptr, InCachePath, InScene, InSource, InImportParameters)
	, CADFileToSceneGraphDescriptionFile(InCADFileToUnrealFileMap)
{
}

bool FDatasmithSceneGraphBuilder::Build()
{
	LoadSceneGraphDescriptionFiles();

	uint32 RootHash = RootFileDescription.GetDescriptorHash();
	SceneGraph = CADFileToSceneGraphArchive.FindRef(RootHash);

	if (!SceneGraph)
	{
		return false;
	}

	AncestorSceneGraphHash.Add(RootHash);

	return FDatasmithSceneBaseGraphBuilder::Build();
}

void FDatasmithSceneGraphBuilder::LoadSceneGraphDescriptionFiles()
{
	ArchiveMockUps.Reserve(CADFileToSceneGraphDescriptionFile.Num());
	CADFileToSceneGraphArchive.Reserve(CADFileToSceneGraphDescriptionFile.Num());

	for (const auto& FilePair : CADFileToSceneGraphDescriptionFile)
	{
		FString MockUpDescriptionFile = FPaths::Combine(CachePath, TEXT("scene"), FilePair.Value + TEXT(".sg"));

		CADLibrary::FArchiveSceneGraph& MockUpDescription = ArchiveMockUps.Emplace_GetRef();

		CADFileToSceneGraphArchive.Add(FilePair.Key, &MockUpDescription);

		MockUpDescription.DeserializeMockUpFile(*MockUpDescriptionFile);

		for (const auto& ColorPair : MockUpDescription.ColorHIdToColor)
		{
			ColorUIdToColorArchive.Emplace(ColorPair.Value.UEMaterialUId, ColorPair.Value);
		}

		for (const auto& MaterialPair : MockUpDescription.MaterialHIdToMaterial)
		{
			MaterialUIdToMaterialArchive.Emplace(MaterialPair.Value.UEMaterialUId, MaterialPair.Value);
		}

	}
}

void FDatasmithSceneGraphBuilder::FillAnchorActor(const TSharedRef<IDatasmithActorElement>& ActorElement, const FString& CleanFilenameOfCADFile)
{
	CADLibrary::FFileDescriptor AnchorDescription(*CleanFilenameOfCADFile);

	uint32 AnchorHash = AnchorDescription.GetDescriptorHash();
	SceneGraph = CADFileToSceneGraphArchive.FindRef(AnchorHash);

	if (!SceneGraph)
	{
		return;
	}

	FActorData Data(TEXT(""));

	FActorData ParentData(ActorElement->GetName());

	CADLibrary::FArchiveReference* Reference = SceneGraph->GetRootReference();
	if (!ensureMsgf(Reference, TEXT("FDatasmithSceneGraphBuilder::FillAnchorActor expected root Reference in the Scene Graph")))
	{
		return;
	}

	CADLibrary::FArchiveInstance EmptyInstance;
	FString ActorUUID;
	FString ActorLabel;
	DatasmithSceneGraphBuilderImpl::GetNodeUuidAndLabel(EmptyInstance, *Reference, ParentData.Uuid, ActorUUID, ActorLabel);

	AddMetaData(ActorElement, EmptyInstance, *Reference);

	FActorData ComponentData(*ActorUUID, ParentData);
	//DatasmithSceneGraphBuilderImpl::GetMainMaterial(Reference.MetaData, ComponentData, bMaterialPropagationIsTopDown);

	AddChildren(ActorElement, *Reference, ComponentData);

	ActorElement->SetLabel(*ActorLabel);
}

FDatasmithSceneBaseGraphBuilder::FDatasmithSceneBaseGraphBuilder(CADLibrary::FArchiveSceneGraph* InSceneGraph, const FString& InCachePath, TSharedRef<IDatasmithScene> InScene, const FDatasmithSceneSource& InSource, const CADLibrary::FImportParameters& InImportParameters)
	: SceneGraph(InSceneGraph)
	, CachePath(InCachePath)
	, DatasmithScene(InScene)
	, ImportParameters(InImportParameters)
	, ImportParametersHash(ImportParameters.GetHash())
	, RootFileDescription(*InSource.GetSourceFile())
{
	if (InSceneGraph)
	{
		ColorUIdToColorArchive.Reserve(SceneGraph->ColorHIdToColor.Num());
		for (const auto& ColorPair : SceneGraph->ColorHIdToColor)
		{
			ColorUIdToColorArchive.Emplace(ColorPair.Value.UEMaterialUId, ColorPair.Value);
		}

		MaterialUIdToMaterialArchive.Reserve(SceneGraph->MaterialHIdToMaterial.Num());
		for (const auto& MaterialPair : SceneGraph->MaterialHIdToMaterial)
		{
			MaterialUIdToMaterialArchive.Emplace(MaterialPair.Value.UEMaterialUId, MaterialPair.Value);
		}
	}
}

bool FDatasmithSceneBaseGraphBuilder::Build()
{
	FActorData Data(TEXT(""));
	CADLibrary::FArchiveReference* Reference = SceneGraph->GetRootReference();
	if (!ensureMsgf(Reference, TEXT("FDatasmithSceneBaseGraphBuilder::Build: expected root Reference in the Scene Graph")))
	{
		return false;
	}
	TSharedPtr<IDatasmithActorElement> RootActor = BuildReference(*Reference, Data);
	DatasmithScene->AddActor(RootActor);

	// Set ProductName, ProductVersion in DatasmithScene for Analytics purpose
	// application_name is something like "Catia V5"
	DatasmithScene->SetVendor(TEXT("Techsoft"));

	FString ProductVersion;
	if (Reference->MetaData.RemoveAndCopyValue(TEXT("TechsoftVersion"), ProductVersion))
	{
		DatasmithScene->SetProductVersion(*ProductVersion);
	}

	// SetProductName
	{
		switch (RootFileDescription.GetFileFormat())
		{
		case CADLibrary::ECADFormat::JT:
			DatasmithScene->SetProductName(TEXT("Jt"));
			break;
		case CADLibrary::ECADFormat::SOLIDWORKS:
			DatasmithScene->SetProductName(TEXT("SolidWorks"));
			break;
		case CADLibrary::ECADFormat::ACIS:
			DatasmithScene->SetProductName(TEXT("3D ACIS"));
			break;
		case CADLibrary::ECADFormat::CATIA:
			DatasmithScene->SetProductName(TEXT("CATIA V5"));
			break;
		case CADLibrary::ECADFormat::CATIA_CGR:
			DatasmithScene->SetProductName(TEXT("CATIA V5"));
			break;
		case CADLibrary::ECADFormat::CATIAV4:
			DatasmithScene->SetProductName(TEXT("CATIA V4"));
			break;
		case CADLibrary::ECADFormat::CATIA_3DXML:
			DatasmithScene->SetProductName(TEXT("3D XML"));
			break;
		case CADLibrary::ECADFormat::CREO:
			DatasmithScene->SetProductName(TEXT("Creo"));
			break;
		case CADLibrary::ECADFormat::IGES:
			DatasmithScene->SetProductName(TEXT("IGES"));
			break;
		case CADLibrary::ECADFormat::INVENTOR:
			DatasmithScene->SetProductName(TEXT("Inventor"));
			break;
		case CADLibrary::ECADFormat::N_X:
			DatasmithScene->SetProductName(TEXT("N")TEXT("X"));
			break;
		case CADLibrary::ECADFormat::PARASOLID:
			DatasmithScene->SetProductName(TEXT("Parasolid"));
			break;
		case CADLibrary::ECADFormat::STEP:
			DatasmithScene->SetProductName(TEXT("STEP"));
			break;
		case CADLibrary::ECADFormat::DWG:
			DatasmithScene->SetProductName(TEXT("AutoCAD"));
			break;
		case CADLibrary::ECADFormat::DGN:
			DatasmithScene->SetProductName(TEXT("Micro Station"));
			break;
		default:
			DatasmithScene->SetProductName(TEXT("Unknown"));
			break;
		}
	}

	return true;
}

void FDatasmithSceneBaseGraphBuilder::OverrideInstance(CADLibrary::FArchiveOverrideOccurrence* OverrideOccurrence, CADLibrary::FArchiveInstance& Instance, FActorData& InstanceData)
{
	if (OverrideOccurrence == nullptr)
	{
		return;
	}

	if (OverrideOccurrence->OverridenChildren.Num())
	{
		InstanceData.OverridenChildren = MoveTemp(OverrideOccurrence->OverridenChildren);
	}

	if (OverrideOccurrence->TransformMatrix != FMatrix::Identity)
	{
		Instance.TransformMatrix = OverrideOccurrence->TransformMatrix;
	}

	Instance.bShow = OverrideOccurrence->bShow;
}

TSharedPtr<IDatasmithActorElement>  FDatasmithSceneBaseGraphBuilder::BuildInstance(FCadId InstanceId, const FActorData& ParentData)
{
	CADLibrary::FArchiveReference* Reference = nullptr;
	CADLibrary::FArchiveReference EmptyReference;

	CADLibrary::FArchiveInstance& Instance = SceneGraph->GetInstance(InstanceId);

	CADLibrary::FArchiveOverrideOccurrence* OverrideInstancePtr = nullptr;
	uint32 OverridenSceneGraphId = 0;

	if(ParentData.OverridenChildren.Num() > 0)
	{
		CADLibrary::FArchiveSceneGraph* ParentSceneGraph = CADFileToSceneGraphArchive.FindRef(ParentData.SceneGraphId);
		if(ParentSceneGraph)
		{
			for (const FCadId& OverrideOccurrenceId : ParentData.OverridenChildren)
			{
				CADLibrary::FArchiveOverrideOccurrence& OverrideOccurrence = ParentSceneGraph->GetOverrideOccurrence(OverrideOccurrenceId);
				if (OverrideOccurrence.Label == Instance.Label)
				{
					OverrideInstancePtr = &OverrideOccurrence;
					OverridenSceneGraphId = ParentData.SceneGraphId;
					break;
				}
			}
		}
	}

	CADLibrary::FArchiveSceneGraph* InstanceSceneGraph = SceneGraph;
	if (Instance.bIsExternalReference)
	{
		CADLibrary::FArchiveUnloadedReference& UnloadedReference = SceneGraph->GetUnloadedReference(Instance.ReferenceNodeId);

		if (!UnloadedReference.ExternalFile.GetSourcePath().IsEmpty())
		{
			uint32 InstanceSceneGraphHash = UnloadedReference.ExternalFile.GetDescriptorHash();
			SceneGraph = CADFileToSceneGraphArchive.FindRef(InstanceSceneGraphHash);

			if (SceneGraph)
			{
				if (AncestorSceneGraphHash.Find(InstanceSceneGraphHash) == INDEX_NONE)
				{
					AncestorSceneGraphHash.Add(InstanceSceneGraphHash);

					Reference = SceneGraph->GetRootReference();
					if (ensureMsgf(Reference, TEXT("FDatasmithSceneBaseGraphBuilder::BuildInstance expected root Reference in the Scene Graph of an external reference")))
					{
						Reference->SetGraphicProperties(UnloadedReference);
					}
				}
			}
		}

		if (!Reference)
		{
			SceneGraph = InstanceSceneGraph;
			EmptyReference.CopyMetaData(UnloadedReference);
			Reference = &EmptyReference;
		}
	}
	else
	{
		Reference = &(SceneGraph->GetReference(Instance.ReferenceNodeId));
	}

	if (!Reference) // Should never append
	{
		Reference = &EmptyReference;
	}

	FString ActorUUID;
	FString ActorLabel;
	DatasmithSceneGraphBuilderImpl::GetNodeUuidAndLabel(Instance, *Reference, ParentData.Uuid, ActorUUID, ActorLabel);

	TSharedPtr<IDatasmithActorElement> Actor = CreateActor(*ActorUUID, *ActorLabel);
	if (Actor.IsValid())
	{
		AddMetaData(Actor, Instance, *Reference);

		FActorData InstanceData(*ActorUUID, ParentData);

		DatasmithSceneGraphBuilderImpl::SpreadGraphicProperties(Instance, InstanceData);
		DatasmithSceneGraphBuilderImpl::SpreadGraphicProperties(*Reference, InstanceData);
		if (OverrideInstancePtr && OverrideInstancePtr->OverridenChildren.Num())
		{
			InstanceData.OverridenChildren = OverrideInstancePtr->OverridenChildren;
			InstanceData.SceneGraphId = OverridenSceneGraphId;
		}
		else if (Instance.OverridenChildren.Num())
		{
			InstanceData.OverridenChildren = MoveTemp(Instance.OverridenChildren);
			InstanceData.SceneGraphId = InstanceSceneGraph->SceneGraphId;
		}

		OverrideInstance(OverrideInstancePtr, Instance, InstanceData);
		if(Instance.bShow)
		{
			AddChildren(Actor, *Reference, InstanceData);
			DatasmithSceneGraphBuilderImpl::AddTransformToActor(Instance, Actor, ImportParameters);
		}
		else
		{
			Actor.Reset();
		}
	}

	if (SceneGraph != InstanceSceneGraph)
	{
		SceneGraph = InstanceSceneGraph;
		AncestorSceneGraphHash.Pop();
	}
	return Actor;
}

TSharedPtr<IDatasmithActorElement>  FDatasmithSceneBaseGraphBuilder::CreateActor(const TCHAR* InEUUID, const TCHAR* InLabel)
{
	TSharedPtr<IDatasmithActorElement> Actor = FDatasmithSceneFactory::CreateActor(InEUUID);
	if (Actor.IsValid())
	{
		Actor->SetLabel(InLabel);
		return Actor;
	}
	return TSharedPtr<IDatasmithActorElement>();
}

TSharedPtr<IDatasmithActorElement> FDatasmithSceneBaseGraphBuilder::BuildReference(CADLibrary::FArchiveReference& Reference, const FActorData& ParentData)
{
	TMap<FString, FString> InstanceNodeMetaDataMap;

	CADLibrary::FArchiveInstance EmptyInstance;
	FString ActorUUID;
	FString ActorLabel;
	DatasmithSceneGraphBuilderImpl::GetNodeUuidAndLabel(EmptyInstance, Reference, ParentData.Uuid, ActorUUID, ActorLabel);

	TSharedPtr<IDatasmithActorElement> Actor = CreateActor(*ActorUUID, *ActorLabel);
	if (!Actor.IsValid())
	{
		return TSharedPtr<IDatasmithActorElement>();
	}

	AddMetaData(Actor, EmptyInstance, Reference);

	FActorData ReferenceData(*ActorUUID, ParentData);
	DatasmithSceneGraphBuilderImpl::SpreadGraphicProperties(Reference, ReferenceData);

	AddChildren(Actor, Reference, ReferenceData);

	DatasmithSceneGraphBuilderImpl::AddTransformToActor(Reference, Actor, ImportParameters);

	return Actor;
}

TSharedPtr<IDatasmithActorElement> FDatasmithSceneBaseGraphBuilder::BuildBody(FCadId BodyId, const FActorData& ParentData)
{
	CADLibrary::FArchiveInstance EmptyInstance;
	CADLibrary::FArchiveBody& Body = SceneGraph->GetBody(BodyId);

	if (Body.IsDeleted())
	{
		return TSharedPtr<IDatasmithActorElement>();
	}

	FString BodyUUID;
	FString BodyLabel;
	DatasmithSceneGraphBuilderImpl::GetNodeUuidAndLabel(EmptyInstance, Body, ParentData.Uuid, BodyUUID, BodyLabel);

	// Apply materials on the current part
	FActorData BodyData(*BodyUUID, ParentData);
	DatasmithSceneGraphBuilderImpl::SpreadGraphicProperties(Body, BodyData);
	FMaterialUId MaterialUId = (BodyData.Inheritance == CADLibrary::ECADGraphicPropertyInheritance::Unset) ? -1 : BodyData.MaterialUId ? BodyData.MaterialUId : BodyData.ColorUId;

	if (!(Body.ColorFaceSet.Num() + Body.MaterialFaceSet.Num()))
	{
		Body.ColorFaceSet.Add(MaterialUId);
	}

	TSharedPtr<IDatasmithMeshElement> MeshElement = FindOrAddMeshElement(Body);
	if (!MeshElement.IsValid())
	{
		return TSharedPtr<IDatasmithActorElement>();
	}


	TSharedPtr<IDatasmithMeshActorElement> ActorElement = FDatasmithSceneFactory::CreateMeshActor(*BodyUUID);
	if (!ActorElement.IsValid())
	{
		return TSharedPtr<IDatasmithActorElement>();
	}

	ActorElement->SetLabel(*BodyLabel);
	ActorElement->SetStaticMeshPathName(MeshElement->GetName());

	DatasmithSceneGraphBuilderImpl::AddTransformToActor(Body, ActorElement, ImportParameters);

	return ActorElement;
}

TSharedPtr<IDatasmithMeshElement> FDatasmithSceneBaseGraphBuilder::FindOrAddMeshElement(CADLibrary::FArchiveBody& Body)
{
	FString ShellUuidName = FString::Printf(TEXT("0x%012u"), Body.MeshActorUId);

	// Look if geometry has not been already processed, return it if found
	TSharedPtr<IDatasmithMeshElement>* MeshElementPtr = BodyUuidToMeshElement.Find(Body.MeshActorUId);
	if (MeshElementPtr != nullptr)
	{
		return *MeshElementPtr;
	}

	TSharedPtr<IDatasmithMeshElement> MeshElement = FDatasmithSceneFactory::CreateMesh(*ShellUuidName);
	MeshElement->SetLabel(*Body.Label);
	MeshElement->SetLightmapSourceUV(-1);

	// Set MeshElement FileHash used for re-import task 
	FMD5 MD5; // unique Value that define the mesh
	MD5.Update(reinterpret_cast<const uint8*>(&ImportParametersHash), sizeof ImportParametersHash);
	// the scene graph archive name that is define by the name and the stat of the file (creation date, size)
	MD5.Update(reinterpret_cast<const uint8*>(SceneGraph->ArchiveFileName.GetCharArray().GetData()), SceneGraph->ArchiveFileName.GetCharArray().Num());
	// MeshActorName
	MD5.Update(reinterpret_cast<const uint8*>(&Body.MeshActorUId), sizeof Body.MeshActorUId);

	FMD5Hash Hash;
	Hash.Set(MD5);
	MeshElement->SetFileHash(Hash);

	TFunction<void(TSet<FMaterialUId>&)> SetMaterialToDatasmithMeshElement = [&](TSet<FMaterialUId>& MaterialSet)
	{
		for (FMaterialUId MaterialSlotId : MaterialSet)
		{
			TSharedPtr<IDatasmithMaterialIDElement> PartMaterialIDElement;
			PartMaterialIDElement = FindOrAddMaterial(MaterialSlotId);
			MeshElement->SetMaterial(PartMaterialIDElement->GetName(), MaterialSlotId);
		}
	};

	SetMaterialToDatasmithMeshElement(Body.ColorFaceSet);
	SetMaterialToDatasmithMeshElement(Body.MaterialFaceSet);

	DatasmithScene->AddMesh(MeshElement);

	BodyUuidToMeshElement.Add(Body.MeshActorUId, MeshElement);

	FString BodyCachePath = CADLibrary::BuildCacheFilePath(*CachePath, TEXT("body"), Body.MeshActorUId, Body.Mesher);
	MeshElement->SetFile(*BodyCachePath);

	return MeshElement;
}

TSharedPtr<IDatasmithUEPbrMaterialElement> FDatasmithSceneBaseGraphBuilder::GetDefaultMaterial()
{
	if (!DefaultMaterial.IsValid())
	{
		DefaultMaterial = CADLibrary::CreateDefaultUEPbrMaterial();
		DatasmithScene->AddMaterial(DefaultMaterial);
	}

	return DefaultMaterial;
}

TSharedPtr<IDatasmithMaterialIDElement> FDatasmithSceneBaseGraphBuilder::FindOrAddMaterial(FMaterialUId MaterialUuid)
{
	TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement;

	TSharedPtr<IDatasmithUEPbrMaterialElement>* MaterialPtr = MaterialUuidMap.Find(MaterialUuid);
	if (MaterialPtr != nullptr)
	{
		MaterialElement = *MaterialPtr;
	}
	else if (MaterialUuid > 0)
	{
		if (CADLibrary::FArchiveColor* Color = ColorUIdToColorArchive.Find(MaterialUuid))
		{
			MaterialElement = CADLibrary::CreateUEPbrMaterialFromColor(Color->Color);
		}
		else if (CADLibrary::FArchiveMaterial* Material = MaterialUIdToMaterialArchive.Find(MaterialUuid))
		{
			MaterialElement = CADLibrary::CreateUEPbrMaterialFromMaterial(Material->Material, DatasmithScene);
		}

		if (MaterialElement.IsValid())
		{
			DatasmithScene->AddMaterial(MaterialElement);
			MaterialUuidMap.Add(MaterialUuid, MaterialElement);
		}
	}

	if (!MaterialElement.IsValid())
	{
		MaterialElement = GetDefaultMaterial();
		MaterialUuidMap.Add(MaterialUuid, MaterialElement);
	}

	TSharedPtr<IDatasmithMaterialIDElement> MaterialIDElement = FDatasmithSceneFactory::CreateMaterialId(MaterialElement->GetName());

	return MaterialIDElement;
}

void FDatasmithSceneBaseGraphBuilder::AddMetaData(TSharedPtr<IDatasmithActorElement> ActorElement, const CADLibrary::FArchiveCADObject& Instance, const CADLibrary::FArchiveCADObject& Reference)
{

	const TMap<FString, FString>& InstanceNodeAttributeSetMap = Instance.MetaData;
	const TMap<FString, FString>& ReferenceNodeAttributeSetMap = Reference.MetaData;

	TSharedRef<IDatasmithMetaDataElement> MetaDataRefElement = FDatasmithSceneFactory::CreateMetaData(ActorElement->GetName());
	MetaDataRefElement->SetAssociatedElement(ActorElement);

	TFunction<void(const CADLibrary::FArchiveCADObject&, const TCHAR*)> AddMetaData = [&](const CADLibrary::FArchiveCADObject& Component, const TCHAR* PostName)
	{
		for (auto& Attribute : Component.MetaData)
		{
			if (Attribute.Value.IsEmpty())
			{
				continue;
			}

			FString MetaName = PostName;
			MetaName += Attribute.Key;
			TSharedRef<IDatasmithKeyValueProperty> KeyValueProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*MetaName);

			KeyValueProperty->SetValue(*Attribute.Value);
			KeyValueProperty->SetPropertyType(EDatasmithKeyValuePropertyType::String);

			MetaDataRefElement->AddProperty(KeyValueProperty);
		}

		// Add name
		if(!Component.Label.IsEmpty())
		{
			FString MetaName = PostName;
			MetaName += TEXT(" Name");
			TSharedRef<IDatasmithKeyValueProperty> KeyValueProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*MetaName);
			KeyValueProperty->SetValue(*Component.Label);
			KeyValueProperty->SetPropertyType(EDatasmithKeyValuePropertyType::String);
			MetaDataRefElement->AddProperty(KeyValueProperty);
		}
	};

	AddMetaData(Reference, TEXT("Reference "));
	AddMetaData(Instance, TEXT("Instance "));

	DatasmithScene->AddMetaData(MetaDataRefElement);

}

bool FDatasmithSceneBaseGraphBuilder::DoesActorHaveChildrenOrIsAStaticMesh(const TSharedPtr<IDatasmithActorElement>& ActorElement)
{
	if (ActorElement != nullptr)
	{
		if (ActorElement->GetChildrenCount() > 0)
		{
			return true;
		}
		else if (ActorElement->IsA(EDatasmithElementType::StaticMeshActor))
		{
			const TSharedPtr<IDatasmithMeshActorElement>& MeshActorElement = StaticCastSharedPtr<IDatasmithMeshActorElement>(ActorElement);
			return FCString::Strlen(MeshActorElement->GetStaticMeshPathName()) > 0;
		}
	}
	return false;
}

void FDatasmithSceneBaseGraphBuilder::AddChildren(TSharedPtr<IDatasmithActorElement> Actor, const CADLibrary::FArchiveReference& Reference, const FActorData& ParentData)
{
	for (const FCadId& ChildId : Reference.Children)
	{
		if (SceneGraph->IsAInstance(ChildId))
		{
			TSharedPtr<IDatasmithActorElement> ChildActor = BuildInstance(ChildId, ParentData);
			if (ChildActor.IsValid() && DoesActorHaveChildrenOrIsAStaticMesh(ChildActor))
			{
				Actor->AddChild(ChildActor);
			}
		}
		else if (SceneGraph->IsABody(ChildId))
		{
			TSharedPtr<IDatasmithActorElement> ChildActor = BuildBody(ChildId, ParentData);
			if (ChildActor.IsValid() && DoesActorHaveChildrenOrIsAStaticMesh(ChildActor))
			{
				Actor->AddChild(ChildActor);
			}
		}
	}
}
