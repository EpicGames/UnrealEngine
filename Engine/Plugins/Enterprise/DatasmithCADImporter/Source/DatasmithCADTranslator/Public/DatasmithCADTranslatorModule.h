// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define DATASMITHCADTRANSLATOR_MODULE_NAME TEXT("DatasmithCADTranslator")

/**
 * Datasmith Translator for .wire files.
 */
class FDatasmithCADTranslatorModule : public IModuleInterface
{
public:

    static FDatasmithCADTranslatorModule& Get()
    {
        return FModuleManager::LoadModuleChecked< FDatasmithCADTranslatorModule >(DATASMITHCADTRANSLATOR_MODULE_NAME);
    }

    static bool IsAvailable()
    {
        return FModuleManager::Get().IsModuleLoaded(DATASMITHCADTRANSLATOR_MODULE_NAME);
    }

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	FString GetCacheDir() const;

private:
	FString TempDir;
	FString CacheDir;
};
