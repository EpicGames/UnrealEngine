// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADOptions.h"

#include "MeshDescription.h"


class IDatasmithMeshElement;


class DATASMITHCADTRANSLATOR_API FDatasmithMeshBuilder
{
public:
	FDatasmithMeshBuilder(TMap<uint32, FString>& InCADFileToMeshFileMap, const FString& InCachePath, const CADLibrary::FImportParameters& InImportParameters);

	FDatasmithMeshBuilder(TArray<CADLibrary::FBodyMesh>& InBodyMeshSet, const CADLibrary::FImportParameters& InImportParameters)
		: ImportParameters(InImportParameters)
	{
		TArray<CADLibrary::FBodyMesh>& BodyMeshSet = BodyMeshes.Add_GetRef(MoveTemp(InBodyMeshSet));

		for (CADLibrary::FBodyMesh& Body : BodyMeshSet)
		{
			MeshActorNameToBodyMesh.Emplace(Body.MeshActorUId, &Body);
		}
	}

	TOptional<FMeshDescription> GetMeshDescription(TSharedRef<IDatasmithMeshElement> OutMeshElement, CADLibrary::FMeshParameters& OutMeshParameters);

protected:
	FString CachePath;

	void LoadMeshFiles(TMap<uint32, FString>& CADFileToMeshFile);

	TArray<TArray<CADLibrary::FBodyMesh>> BodyMeshes;
	TMap<FCadUuid, CADLibrary::FBodyMesh*> MeshActorNameToBodyMesh;

	CADLibrary::FImportParameters ImportParameters;
};

