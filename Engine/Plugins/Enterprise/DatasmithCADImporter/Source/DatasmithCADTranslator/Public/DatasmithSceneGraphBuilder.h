// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADSceneGraph.h"
#include "DatasmithImportOptions.h"
#include "CADMeshDescriptionHelper.h"

#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Misc/Paths.h"

namespace CADLibrary
{
class FArchiveCADObject;
}

class FDatasmithSceneSource;
class IDatasmithActorElement;
class IDatasmithMeshElement;
class IDatasmithScene;
class IDatasmithUEPbrMaterialElement;

class FActorData
{
public:
	FActorData(const TCHAR* NodeUuid, const FActorData& ParentData)
		: Uuid(NodeUuid)
		, Inheritance(ParentData.Inheritance)
		, MaterialUId(ParentData.MaterialUId)
		, ColorUId(ParentData.ColorUId)
	{
	}

	FActorData(const TCHAR* NodeUuid)
		: Uuid(NodeUuid)
		, Inheritance(CADLibrary::ECADGraphicPropertyInheritance::Unset)
		, MaterialUId(0)
		, ColorUId(0)
	{
	}

	const TCHAR* Uuid;
	CADLibrary::ECADGraphicPropertyInheritance Inheritance = CADLibrary::ECADGraphicPropertyInheritance::Unset;
	FMaterialUId MaterialUId;
	FMaterialUId ColorUId;

	uint32 SceneGraphId = 0;
	TArray<FCadId> OverridenChildren;
};

class DATASMITHCADTRANSLATOR_API FDatasmithSceneBaseGraphBuilder
{
public:
	FDatasmithSceneBaseGraphBuilder(
		CADLibrary::FArchiveSceneGraph* InSceneGraph,
		const FString& InCachePath,
		TSharedRef<IDatasmithScene> InScene, 
		const FDatasmithSceneSource& InSource, 
		const CADLibrary::FImportParameters& InImportParameters);

	virtual ~FDatasmithSceneBaseGraphBuilder() {}

	virtual bool Build();

protected:
	TSharedPtr<IDatasmithActorElement> BuildInstance(FCadId InstanceId, const FActorData& ParentData);
	TSharedPtr<IDatasmithActorElement> BuildReference(CADLibrary::FArchiveReference& Reference, const FActorData& ParentData);
	TSharedPtr<IDatasmithActorElement> BuildBody(FCadId BodyId, const FActorData& ParentData);

	void OverrideInstance(CADLibrary::FArchiveOverrideOccurrence* OverrideOccurrence, CADLibrary::FArchiveInstance& Instance, FActorData& InstanceData);

	void AddMetaData(TSharedPtr<IDatasmithActorElement> ActorElement, const CADLibrary::FArchiveCADObject& Instance, const CADLibrary::FArchiveCADObject& Reference);
	void AddChildren(TSharedPtr<IDatasmithActorElement> Actor, const CADLibrary::FArchiveReference& Reference, const FActorData& ParentData);
	bool DoesActorHaveChildrenOrIsAStaticMesh(const TSharedPtr< IDatasmithActorElement >& ActorElement);

	TSharedPtr<IDatasmithUEPbrMaterialElement> GetDefaultMaterial();
	TSharedPtr<IDatasmithMaterialIDElement> FindOrAddMaterial(FMaterialUId MaterialUuid);

	TSharedPtr<IDatasmithActorElement> CreateActor(const TCHAR* ActorUUID, const TCHAR* ActorLabel);
	virtual TSharedPtr<IDatasmithMeshElement> FindOrAddMeshElement(CADLibrary::FArchiveBody& Body);

	CADLibrary::FArchiveSceneGraph* FindSceneGraphArchive(const CADLibrary::FFileDescriptor& File, uint32& FileHash) const;

protected:
	CADLibrary::FArchiveSceneGraph* SceneGraph;
	const FString& CachePath;
	TSharedRef<IDatasmithScene> DatasmithScene;
	const CADLibrary::FImportParameters& ImportParameters;
	const uint32 ImportParametersHash;
	CADLibrary::FFileDescriptor RootFileDescription;

	TArray<CADLibrary::FArchiveSceneGraph> ArchiveMockUps;
	TMap<uint32, CADLibrary::FArchiveSceneGraph*> CADFileToSceneGraphArchive;

	TMap<FCadUuid, TSharedPtr< IDatasmithMeshElement>> BodyUuidToMeshElement;

	TMap<FMaterialUId, TSharedPtr< IDatasmithUEPbrMaterialElement>> MaterialUuidMap;
	TSharedPtr<IDatasmithUEPbrMaterialElement> DefaultMaterial;

	TMap<FMaterialUId, CADLibrary::FArchiveColor> ColorUIdToColorArchive; 
	TMap<FMaterialUId, CADLibrary::FArchiveMaterial> MaterialUIdToMaterialArchive; 

	TArray<uint32> AncestorSceneGraphHash;
};

class DATASMITHCADTRANSLATOR_API FDatasmithSceneGraphBuilder : public FDatasmithSceneBaseGraphBuilder
{
public:
	FDatasmithSceneGraphBuilder(
		TMap<uint32, FString>& InCADFileToUE4FileMap, 
		const FString& InCachePath, 
		TSharedRef<IDatasmithScene> InScene, 
		const FDatasmithSceneSource& InSource, 
		const CADLibrary::FImportParameters& InImportParameters);

	virtual ~FDatasmithSceneGraphBuilder() {}

	virtual bool Build() override;

	void LoadSceneGraphDescriptionFiles();

	void FillAnchorActor(const TSharedRef< IDatasmithActorElement >& ActorElement, const FString& CleanFilenameOfCADFile);

protected:
	TMap<uint32, FString>& CADFileToSceneGraphDescriptionFile;
};
