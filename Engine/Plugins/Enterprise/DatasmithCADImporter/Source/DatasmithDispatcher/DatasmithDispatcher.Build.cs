// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithDispatcher : ModuleRules
	{
		public DatasmithDispatcher(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
				new string[]
				{
                    "Core",
					"Sockets",
					"CADInterfaces",
					"CADTools",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
                }
			);
        }
    }
}
