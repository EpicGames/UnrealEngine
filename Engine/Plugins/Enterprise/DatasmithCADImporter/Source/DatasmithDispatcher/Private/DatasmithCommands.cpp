// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCommands.h"


#include "Serialization/Archive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

namespace DatasmithDispatcher
{
	TSharedPtr<ICommand> CreateCommand(ECommandId CommandType)
	{
		switch (CommandType)
		{
			case ECommandId::Ping: return MakeShared<FPingCommand>();
			case ECommandId::BackPing: return MakeShared<FBackPingCommand>();
			case ECommandId::RunTask: return MakeShared<FRunTaskCommand>();
			case ECommandId::NotifyEndTask: return MakeShared<FCompletedTaskCommand>();
			case ECommandId::ImportParams: return MakeShared<FImportParametersCommand>();
			case ECommandId::Terminate: return MakeShared<FTerminateCommand>();
		}
		return nullptr;
	}

	void SerializeCommand(ICommand& Command, TArray<uint8>& OutBuffer)
	{
		FMemoryWriter ArWriter(OutBuffer);
		uint8 type = static_cast<uint8>(Command.GetType());
		ArWriter << type;
		ArWriter << Command;
	}

	TSharedPtr<ICommand> DeserializeCommand(const TArray<uint8>& InBuffer)
	{
		FMemoryReader ArReader(InBuffer);
		uint8 type = 0;
		ArReader << type;
		if (TSharedPtr<ICommand> Command = CreateCommand(static_cast<ECommandId>(type)))
		{
			ArReader << *Command;
			return ArReader.IsError() ? nullptr : Command;
		}
		return nullptr;
	}

	void FRunTaskCommand::SerializeImpl(FArchive& Ar)
	{
		Ar << JobFileDescription;
		Ar << Mesher;
		Ar << JobIndex;
	}

  	void FCompletedTaskCommand::SerializeImpl(FArchive& Ar)
	{
		Ar << ExternalReferences;
		Ar << ProcessResult;
		Ar << SceneGraphFileName;
		Ar << GeomFileName;
		Ar << Messages;
	}

	void FImportParametersCommand::SerializeImpl(FArchive& Ar)
	{
		Ar << ImportParameters;
	}

}
