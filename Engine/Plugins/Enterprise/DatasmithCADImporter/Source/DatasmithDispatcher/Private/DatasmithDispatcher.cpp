// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDispatcher.h"

#include "CADFileData.h"
#include "CADFileReader.h"
#include "CADOptions.h"
#include "DatasmithDispatcherConfig.h"
#include "DatasmithDispatcherLog.h"
#include "DatasmithDispatcherTask.h"

#include "Algo/Count.h"
#include "Algo/Find.h"
#include "HAL/FileManager.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "DatasmithDispatcher"

namespace DatasmithDispatcher
{

bool bCheckMemory = true;
bool bEnableMemoryControl = true;
FAutoConsoleVariableRef GCADTranslatorEnableMemoryControl(
	TEXT("ds.CADTranslator.EnableMemoryControl"),
	bEnableMemoryControl,
	TEXT("\
Enable/disable the control of the memory used by static meshes to avoid running out of memory.\n\
if CheckMemoryUsage = true, a warning is displayed allowing the user to interrupt the import and have only data already processed or to continue with the risk of running out of memory.\n\
Default value is true.\n"),
ECVF_Default);


static uint64 EstimationOfMemoryUsedByStaticMeshes = 0;
static uint64 AvailableMemory = 0;
static bool bDisplayWorkerWarning = true;

const EAppReturnType::Type OpenMessageDialog(uint64 AvailableRam, uint64 EstimationOfMemory, bool bWorkerWarning)
{
	const FText StaticMeshSizeTxt = FText::AsMemory(EstimationOfMemoryUsedByStaticMeshes, EMemoryUnitStandard::IEC);
	const FText RamSizeTxt = FText::AsMemory(AvailableRam, EMemoryUnitStandard::IEC);

	const FText WarningMessageTxt = LOCTEXT("WarningNotEnoughRamForLocalImport", "Warning: DatasmithCADWorker are currently not used.");
	const FText MainMessageTxt = LOCTEXT("NotEnoughRamForLocalImport",
		"Estimated memory used by static meshes is greater than available RAM:\n"
		"\t{0} available vs {1} estimated.\n"
		"Click OK to continue importing or CANCEL to get only what have already been processed.\n"
		"If you want to continue, be sure to allocate enough virtual memory to avoid running out of memory.\n"
		"If you cancel, check the output log for more information.\n"
		"{2}");

	const FText Message = FText::Format(MainMessageTxt, RamSizeTxt, StaticMeshSizeTxt, bWorkerWarning ? WarningMessageTxt : FText());
	const EAppReturnType::Type Choice = FMessageDialog::Open(EAppMsgType::OkCancel, Message);

	if (Choice == EAppReturnType::Cancel)
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "Estimated memory used by static meshes is greater than available RAM (%ls available vs %ls estimated). Import process is interrupted.", *RamSizeTxt.ToString(), *StaticMeshSizeTxt.ToString());
		UE_LOGF(LogDatasmithDispatcher, Error, "Import of the files below have been aborted:");
	}
	return Choice;
}


FDatasmithDispatcher::FDatasmithDispatcher(const CADLibrary::FImportParameters& InImportParameters, const FString& InCacheDir, TMap<uint32, FString>& OutCADFileToUnrealFileMap, TMap<uint32, FString>& OutCADFileToUnrealGeomMap)
	: NextTaskIndex(0)
	, CompletedTaskCount(0)
	, CADFileToUnrealFileMap(OutCADFileToUnrealFileMap)
	, CADFileToUnrealGeomMap(OutCADFileToUnrealGeomMap)
	, ProcessCacheFolder(InCacheDir)
	, ImportParameters(InImportParameters)
	, NumberOfWorkers(0)
	, NextWorkerId(0)
{
	constexpr double RecommandedRamPerWorkers = 6.;
	constexpr double OneGigaByte = 1024. * 1024. * 1024.;
	const double AvailableRamGB = (double)(FPlatformMemory::GetStats().AvailablePhysical / OneGigaByte);

	const int32 MaxNumberOfWorkers = FPlatformMisc::NumberOfCores();
	const int32 RecommandedNumberOfWorkers = (int32)(AvailableRamGB / RecommandedRamPerWorkers + 0.5);

	// UE recommendation 
	NumberOfWorkers = FMath::Min(MaxNumberOfWorkers, RecommandedNumberOfWorkers);

	// User choice but limited by the number of cores. More brings nothing
	if (CADLibrary::GMaxImportThreads > 1)
	{
		NumberOfWorkers = FMath::Min(CADLibrary::GMaxImportThreads, MaxNumberOfWorkers);
	}

	if (CADLibrary::FImportParameters::bGEnableCADCache)
	{
		// init cache folders
		IFileManager::Get().MakeDirectory(*FPaths::Combine(ProcessCacheFolder, TEXT("scene")), true);
		IFileManager::Get().MakeDirectory(*FPaths::Combine(ProcessCacheFolder, TEXT("cad")), true);
		IFileManager::Get().MakeDirectory(*FPaths::Combine(ProcessCacheFolder, TEXT("mesh")), true);
		IFileManager::Get().MakeDirectory(*FPaths::Combine(ProcessCacheFolder, TEXT("body")), true);
	}
}

void FDatasmithDispatcher::AddTask(const CADLibrary::FFileDescriptor& InFileDescription, const CADLibrary::EMesher Mesher)
{
	using namespace CADLibrary;
	FScopeLock Lock(&TaskPoolCriticalSection);

	if (Algo::FindByPredicate(TaskPool, [&InFileDescription, &Mesher](const FTask& Task) { return Task.FileDescription == InFileDescription && Task.Mesher == Mesher; }))
	{
		return;
	}

	int32 TaskIndex = TaskPool.Emplace(InFileDescription, Mesher);
	TaskPool[TaskIndex].Index = TaskIndex;
}

void FDatasmithDispatcher::LogMessages(const TArray<TPair<uint8, FString>>& Messages) const
{
	for (const TPair<uint8, FString>& Message : Messages)
	{
		switch (Message.Key)
		{
		case ELogVerbosity::Fatal:
			UE_LOGF(LogDatasmithDispatcher, Fatal, "%ls", *Message.Value);
			break;
		case ELogVerbosity::Error:
			UE_LOGF(LogDatasmithDispatcher, Error, "%ls", *Message.Value);
			break;
		case ELogVerbosity::Warning:
			UE_LOGF(LogDatasmithDispatcher, Warning, "%ls", *Message.Value);
			break;
		case ELogVerbosity::Display:
			UE_LOGF(LogDatasmithDispatcher, Display, "%ls", *Message.Value);
			break;
		case ELogVerbosity::Log:
			UE_LOGF(LogDatasmithDispatcher, Log, "%ls", *Message.Value);
			break;
		case ELogVerbosity::Verbose:
			UE_LOGF(LogDatasmithDispatcher, Verbose, "%ls", *Message.Value);
			break;
		case ELogVerbosity::VeryVerbose:
		default:
			UE_LOGF(LogDatasmithDispatcher, VeryVerbose, "%ls", *Message.Value);
			break;
		}
	}
}

TOptional<FTask> FDatasmithDispatcher::GetNextTask()
{
	FScopeLock Lock(&TaskPoolCriticalSection);

	while (TaskPool.IsValidIndex(NextTaskIndex) && TaskPool[NextTaskIndex].State != ETaskState::UnTreated)
	{
		NextTaskIndex++;
	}

	if (!TaskPool.IsValidIndex(NextTaskIndex))
	{
		return TOptional<FTask>();
	}

	TaskPool[NextTaskIndex].State = ETaskState::Running;
	UE_LOGF(LogDatasmithDispatcher, Display, "Launch %ls (%d / %d)", *TaskPool[NextTaskIndex].FileDescription.GetSourcePath(), NextTaskIndex, TaskPool.Num());

	return TaskPool[NextTaskIndex++];
}

void FDatasmithDispatcher::SetTaskState(int32 TaskIndex, ETaskState TaskState)
{
	CADLibrary::FFileDescriptor FileDescriptor;
	{
		FScopeLock Lock(&TaskPoolCriticalSection);

		if (!ensure(TaskPool.IsValidIndex(TaskIndex)))
		{
			return;
		}

		FTask& Task = TaskPool[TaskIndex];
		Task.State = TaskState;
		FileDescriptor = Task.FileDescription;

		switch (TaskState)
		{
		case ETaskState::Unknown:
		{
			Task.State = ETaskState::ProcessOk;
			CompletedTaskCount++;
			break;
		}

		case ETaskState::ProcessFailed:
		{
			if (Task.Mesher == CADLibrary::EMesher::CADKernel)
			{
				UE_LOGF(LogDatasmithDispatcher, Warning, "   - Task failed with CADKernel: %ls", *Task.FileDescription.GetFileName());
				UE_LOGF(LogDatasmithDispatcher, Warning, "      => Add task to process with Techsoft");
				AddTask(Task.FileDescription, CADLibrary::EMesher::TechSoft);
				TaskState = ETaskState::UnTreated;
			}

			CompletedTaskCount++;
			break;
		}

		case ETaskState::ProcessOk:
		case ETaskState::FileNotFound:
		{
			CompletedTaskCount++;
			break;
		}

		case ETaskState::UnTreated:
		{
			NextTaskIndex = TaskIndex;
			break;
		}

		default:
			break;
		}
	}

	UE_CLOGF(TaskState == ETaskState::ProcessOk, LogDatasmithDispatcher, Verbose, "File processed: %ls", *FileDescriptor.GetFileName());
	UE_CLOGF(TaskState == ETaskState::UnTreated, LogDatasmithDispatcher, Warning, "File resubmitted: %ls", *FileDescriptor.GetFileName());
	UE_CLOGF(TaskState == ETaskState::ProcessFailed, LogDatasmithDispatcher, Error, "File processing failure: %ls", *FileDescriptor.GetFileName());
	UE_CLOGF(TaskState == ETaskState::FileNotFound, LogDatasmithDispatcher, Warning, "file not found: %ls", *FileDescriptor.GetSourcePath());
}

void FDatasmithDispatcher::Process(bool bWithProcessor)
{
	if (TaskPool.Num() == 1 && !TaskPool[0].FileDescription.CanReferenceOtherFiles())
	{
		NumberOfWorkers = 1;
	}

	EstimationOfMemoryUsedByStaticMeshes = 0;
	bCheckMemory = bEnableMemoryControl & !CADLibrary::FImportParameters::bValidationProcess;

	// This function is added in 5.1.1 but is cloned with the lines 370 to 387
	TFunction<bool()> CanImportContinue = [&]() -> bool
	{
		AvailableMemory = FPlatformMemory::GetStats().AvailablePhysical;
		if (AvailableMemory < EstimationOfMemoryUsedByStaticMeshes)
		{
			const EAppReturnType::Type Choice = OpenMessageDialog(AvailableMemory, EstimationOfMemoryUsedByStaticMeshes, bDisplayWorkerWarning);

			if (Choice == EAppReturnType::Cancel)
			{
				while (TOptional<FTask> Task = GetNextTask())
				{
					UE_LOGF(LogDatasmithDispatcher, Error, "   - %ls", *Task->FileDescription.GetFileName());
				}

				CloseHandlers();

				// Log last running tasks as failed
				{
					FScopeLock Lock(&TaskPoolCriticalSection);
					for (FTask& Task : TaskPool)
					{
						if (Task.State == ETaskState::Running)
						{
							UE_LOGF(LogDatasmithDispatcher, Error, "   - %ls", *Task.FileDescription.GetFileName());
						}
					}
					CompletedTaskCount = TaskPool.Num();
				}
				return false;
			}
			bCheckMemory = false;
		}
		return true;
	};

	// Temporary code to validate that DatasmithCADWorker.exe does exist before triggering the multi-processing
	// A static method of FDatasmithWorkerHandler, i.e. CanStart, should provide this information.
	{
		FString WorkerPath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Enterprise/DatasmithCADImporter/Binaries"));

#if PLATFORM_MAC
		WorkerPath = FPaths::Combine(WorkerPath, TEXT("Mac/DatasmithCADWorker"));
#elif PLATFORM_LINUX
		WorkerPath = FPaths::Combine(WorkerPath, TEXT("Linux/DatasmithCADWorker"));
#elif PLATFORM_WINDOWS
		WorkerPath = FPaths::Combine(WorkerPath, TEXT("Win64/DatasmithCADWorker.exe"));
#endif
		bWithProcessor &= FPaths::FileExists(WorkerPath);
	}

	bDisplayWorkerWarning = !bWithProcessor;

	if (bWithProcessor)
	{
		SpawnHandlers();

		bool bLogRestartError = true;
		while (!IsOver())
		{
			if (bCheckMemory && !CanImportContinue())
			{
				break;
			}

			bool bHasAliveWorker = false;
			for (FDatasmithWorkerHandler& Handler : WorkerHandlers)
			{
				// replace dead workers
				if (Handler.IsRestartable())
				{
					int32 WorkerId = GetNextWorkerId();
					if (WorkerId < NumberOfWorkers + Config::MaxRestartAllowed)
					{
						Handler.~FDatasmithWorkerHandler();
						new (&Handler) FDatasmithWorkerHandler(*this, ImportParameters, ProcessCacheFolder, WorkerId);
						UE_LOGF(LogDatasmithDispatcher, Warning, "Restarting worker (new worker: %d)", WorkerId);
					}
					else if (bLogRestartError)
					{
						bLogRestartError = false;
						UE_LOGF(LogDatasmithDispatcher, Warning, "Worker not restarted (Limit reached)");
					}
				}

				bHasAliveWorker = bHasAliveWorker || Handler.IsAlive();
			}

			if (!bHasAliveWorker)
			{
				break;
			}
			FPlatformProcess::Sleep(0.03f);
		}

		CloseHandlers();
	}

	if (!IsOver())
	{
		// Inform user that multi processing was incomplete
		if (bWithProcessor)
		{
			UE_LOGF(LogDatasmithDispatcher, Warning,
				"Begin local processing. (Multi Process failed to consume all the tasks)\n" "See workers logs: %lsPrograms/DatasmithCADWorker/Saved/Logs", *FPaths::ConvertRelativePathToFull(FPaths::EngineDir()));
		}

		ProcessLocal();
	}
	else
	{
		UE_LOGF(LogDatasmithDispatcher, Display, "Multi Process ended and consumed all the tasks");
	}
}

bool FDatasmithDispatcher::IsOver()
{
	FScopeLock Lock(&TaskPoolCriticalSection);
	return CompletedTaskCount == TaskPool.Num();
}

void FDatasmithDispatcher::LinkCTFileToUnrealCacheFile(const CADLibrary::FFileDescriptor& CTFileDescription, const FString& UnrealSceneGraphFile, const FString& UnrealMeshFile)
{
	FScopeLock Lock(&TaskPoolCriticalSection);

	uint32 FileHash = CTFileDescription.GetDescriptorHash();
	if (!UnrealSceneGraphFile.IsEmpty())
	{
		CADFileToUnrealFileMap.Add(FileHash, UnrealSceneGraphFile);
	}
	if (!UnrealMeshFile.IsEmpty())
	{
		constexpr int64 RatioCacheSizeMeshSize = 11; // 11 is based on tests done on several huge DMUs
		CADFileToUnrealGeomMap.Add(FileHash, UnrealMeshFile);
		FString FilePath = FPaths::Combine(ProcessCacheFolder, TEXT("mesh"), UnrealMeshFile + TEXT(".gm"));
		FFileStatData FileStatData = IFileManager::Get().GetStatData(*FilePath);
		EstimationOfMemoryUsedByStaticMeshes += (uint64)(FileStatData.FileSize * RatioCacheSizeMeshSize);
	}

}

int32 FDatasmithDispatcher::GetNextWorkerId()
{
	return NextWorkerId++;
}

void FDatasmithDispatcher::SpawnHandlers()
{
	WorkerHandlers.Reserve(NumberOfWorkers);
	for (int32 Index = 0; Index < NumberOfWorkers; Index++)
	{
		WorkerHandlers.Emplace(*this, ImportParameters, ProcessCacheFolder, GetNextWorkerId());
	}
}

void FDatasmithDispatcher::CloseHandlers()
{
	for (FDatasmithWorkerHandler& Handler : WorkerHandlers)
	{
		Handler.Stop();
	}
	WorkerHandlers.Empty();
}

void FDatasmithDispatcher::ProcessLocal()
{
	using namespace CADLibrary;
	EMesher DefaultMesher = FImportParameters::bGDisableCADKernelTessellation ? EMesher::TechSoft : EMesher::CADKernel;

	while (TOptional<FTask> Task = GetNextTask())
	{
		FFileDescriptor& FileDescription = Task->FileDescription;
		FImportParameters FileImporParameters(ImportParameters, Task->Mesher);

		FCADFileReader FileReader(FileImporParameters, FileDescription, *FPaths::EnginePluginsDir(), ProcessCacheFolder);
		ETaskState ProcessResult = FileReader.ProcessFile();

		ETaskState TaskState = ProcessResult;
		SetTaskState(Task->Index, TaskState);

		switch (TaskState)
		{
		case ETaskState::ProcessOk:
		{
			const FCADFileData& CADFileData = FileReader.GetCADFileData();
			const TArray<FFileDescriptor>& ExternalRefSet = CADFileData.GetExternalRefSet();
			if (ExternalRefSet.Num() > 0)
			{
				for (const FFileDescriptor& ExternalFile : ExternalRefSet)
				{
					AddTask(ExternalFile, DefaultMesher);
				}
			}

			// update the available memory to get into account possible memory leak
			AvailableMemory = FPlatformMemory::GetStats().AvailablePhysical;

			LinkCTFileToUnrealCacheFile(CADFileData.GetCADFileDescription(), CADFileData.GetSceneGraphFileName(), CADFileData.GetMeshFileName());

			// This block is added in 5.1.1 but is cloned with function of the lines 166 to 204
			// This will be refactor in 5.2 (jira UE-172253)  
			if (bCheckMemory && AvailableMemory < EstimationOfMemoryUsedByStaticMeshes)
			{
				const EAppReturnType::Type Choice = OpenMessageDialog(AvailableMemory, EstimationOfMemoryUsedByStaticMeshes, bDisplayWorkerWarning);

				if (Choice == EAppReturnType::Cancel)
				{
					while ((Task = GetNextTask()))
					{
						UE_LOGF(LogDatasmithDispatcher, Error, "   - %ls", *Task->FileDescription.GetFileName());
					}
					break;
				}
				bCheckMemory = false;
			}
			break;
		}
		case ETaskState::ProcessFailed:
		{
			if (Task->Mesher == EMesher::CADKernel)
			{
				AddTask(Task->FileDescription, EMesher::TechSoft);
			}
			break;
		}
		default:
			break;
		}
	}
}

} // ns DatasmithDispatcher

#undef LOCTEXT_NAMESPACE