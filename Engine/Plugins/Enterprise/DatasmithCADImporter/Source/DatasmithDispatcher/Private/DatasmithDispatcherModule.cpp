// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithDispatcherModule.h"

#include "DatasmithDispatcherLog.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"



#define LOCTEXT_NAMESPACE "FDatasmithDispatcherModule"

DEFINE_LOG_CATEGORY(LogDatasmithDispatcher);

FDatasmithDispatcherModule& FDatasmithDispatcherModule::Get()
{
	return FModuleManager::LoadModuleChecked< FDatasmithDispatcherModule >(DATASMITHDISPATCHER_MODULE_NAME);
}

bool FDatasmithDispatcherModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(DATASMITHDISPATCHER_MODULE_NAME);
}

void FDatasmithDispatcherModule::StartupModule()
{
}

IMPLEMENT_MODULE(FDatasmithDispatcherModule, DatasmithDispatcher);

#undef LOCTEXT_NAMESPACE // "DatasmithDispatcherModule"

