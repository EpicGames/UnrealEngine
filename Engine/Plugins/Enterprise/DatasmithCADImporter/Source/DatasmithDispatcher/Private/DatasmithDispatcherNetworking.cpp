// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDispatcherNetworking.h"

#include "DatasmithDispatcherLog.h"

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"


namespace DatasmithDispatcher
{

static FTimespan TimespanFromSeconds(double Seconds)
{
	return FTimespan(int64(Seconds * ETimespan::TicksPerSecond));
}

FNetworkNode::~FNetworkNode()
{
	CloseSocket(ConnectedSocket);
}

bool FNetworkNode::SendMessage(const TArray<uint8>& Message, double Timeout_s)
{
	if (bWriteError)
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "bWriteError flag raised, can't write");
		return false;
	}

	if (ConnectedSocket == nullptr)
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "node not connected, can't write");
		return false;
	}

	ESocketConnectionState State = ConnectedSocket->GetConnectionState();
	if (State == ESocketConnectionState::SCS_ConnectionError)
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "connection state error");
		return false;
	}

	bool bCanWrite = ConnectedSocket->Wait(ESocketWaitConditions::WaitForWrite, TimespanFromSeconds(Timeout_s));
	if (!bCanWrite)
	{
		UE_LOGF(LogDatasmithDispatcher, Warning, "can't write on socket");
		return false;
	}

	ConnectedSocket->SetNonBlocking(false);

	auto SendBuffer = [&](const TArray<uint8>& Buffer) -> bool
	{
		uint32 BufferSize = Buffer.Num();
		const uint8* Data = Buffer.GetData();

		uint32 TotalByteSent = 0;
		bool bSendSucceed = true;
		while (bSendSucceed && TotalByteSent < BufferSize)
		{
			int32 BytesSent = 0;
			bSendSucceed &= ConnectedSocket->Send(Data + TotalByteSent, BufferSize - TotalByteSent, BytesSent);
			TotalByteSent += (uint32)BytesSent;
		}
		return bSendSucceed && TotalByteSent == BufferSize;
	};

	// Send the header
	TArray<uint8> HeaderBuffer;
	FMemoryWriter ArWriter(HeaderBuffer);

	FMessageHeader Header;
	Header.ByteSize = Message.Num();
	ArWriter << Header;

	if (!SendBuffer(HeaderBuffer))
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "can't write header on socket");
		bWriteError = true;
		return false;
	}

	// Send the content
	if (!SendBuffer(Message))
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "can't write content on socket");
		bWriteError = true;
		return false;
	}

	return true;
}

bool FNetworkNode::ReceiveMessage(TArray<uint8>& OutMessage, double Timeout_s)
{
	if (bReadError)
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "ReadError flag raised, can't read");
		return false;
	}

	if (ConnectedSocket == nullptr)
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "node not connected, can't read");
		return false;
	}

	ConnectedSocket->SetNonBlocking(true);

	bool bCanRead = ConnectedSocket->Wait(ESocketWaitConditions::WaitForRead, TimespanFromSeconds(Timeout_s));
	if (!bCanRead)
	{
		return false;
	}

	// Make sure we have a valid header
	if (IncommingMessage.Header.ByteSize < 0)
	{
		static uint32 HeaderByteSize = []()
		{
			FMessageHeader Header;
			TArray<uint8> HeaderBuffer;
			FMemoryWriter ArWriter(HeaderBuffer);
			ArWriter << Header;
			return HeaderBuffer.Num();
		}();

		uint32 PendingByteCount;
		if (!ConnectedSocket->HasPendingData(PendingByteCount) || PendingByteCount < HeaderByteSize)
		{
			return false;
		}

		TArray<uint8> HeaderBuffer;
		HeaderBuffer.AddZeroed(HeaderByteSize);
		int32 BytesRead = -1;
		bool bRecvSucceed = ConnectedSocket->Recv(HeaderBuffer.GetData(), HeaderBuffer.Num(), BytesRead);
		if (!bRecvSucceed || BytesRead != HeaderByteSize)
		{
			UE_LOGF(LogDatasmithDispatcher, Error, "Parsed header failed");
			bReadError = true;
			return false;
		}

		FMemoryReader ArReader(HeaderBuffer);
		ArReader << IncommingMessage.Header;

		if (IncommingMessage.Header.ByteSize < 0)
		{
			UE_LOGF(LogDatasmithDispatcher, Error, "Parsed header failed: bad message size %d", IncommingMessage.Header.ByteSize);
			bReadError = true;
			return false;
		}

		IncommingMessage.Content.Reserve(IncommingMessage.Header.ByteSize);
	}

	// fill the message with available data
	uint32 PendingByteCount;
	while (IncommingMessage.Header.ByteSize >= 0 && ConnectedSocket->HasPendingData(PendingByteCount))
	{
		int32 MissingByteInCurrentMessage = IncommingMessage.Header.ByteSize - IncommingMessage.Content.Num();
		int32 ReadTarget = int32(PendingByteCount) < MissingByteInCurrentMessage ? PendingByteCount : MissingByteInCurrentMessage;
		uint8* Destination = IncommingMessage.Content.GetData() + IncommingMessage.Content.Num();
		IncommingMessage.Content.AddUninitialized(ReadTarget);

		int32 BytesRead = -1;
		bool bRecvSucceed = ConnectedSocket->Recv(Destination, ReadTarget, BytesRead);
		if (BytesRead != ReadTarget)
		{
			bReadError = true;
			UE_LOGF(LogDatasmithDispatcher, Error, "Recv issue");
			return false;
		}

		bool bMessageCompleted = BytesRead == MissingByteInCurrentMessage;
		if (bMessageCompleted)
		{
			OutMessage = MoveTemp(IncommingMessage.Content);
			IncommingMessage = FMessage();
			return true;
		}
	}

	return false;
}


FSocket* FNetworkNode::CreateInternalSocket(const FString& Description)
{
	FSocket* Socket = nullptr;
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	TSharedRef<FInternetAddr> InternetAddress = SocketSubsystem->CreateInternetAddr();
	InternetAddress->SetLoopbackAddress();

	FString a = InternetAddress->ToString(true);

	FName Protocol = NAME_None;
	if (ensure(InternetAddress->IsValid()))
	{
		Protocol = InternetAddress->GetProtocolType();
	}
	FSocket* NewSocket = SocketSubsystem->CreateSocket(NAME_Stream, *Description, Protocol);
	UE_CLOGF(!NewSocket, LogDatasmithDispatcher, Error, "Socket creation failure");
	return NewSocket;
}

void FNetworkNode::CloseSocket(FSocket*& Socket)
{
	if (Socket)
	{
		Socket->Close();
		ISocketSubsystem::Get()->DestroySocket(Socket);
		Socket = nullptr;
	}
}

bool FNetworkNode::IsConnected()
{
	return ConnectedSocket && ConnectedSocket->GetConnectionState() == ESocketConnectionState::SCS_Connected;
}

FNetworkServerNode::FNetworkServerNode()
{
	ListeningSocket = CreateInternalSocket(TEXT("datasmith listening socket"));

	if (!ListeningSocket)
	{
		ConnectedSocketError = SocketErrorCode::Error_Create;
		return;
	}

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	TSharedRef<FInternetAddr> InternetAddress = SocketSubsystem->CreateInternetAddr();
	InternetAddress->SetLoopbackAddress();

	int32 BoundPort = SocketSubsystem->BindNextPort(ListeningSocket, *InternetAddress, 1e3, 1);
	if (BoundPort == 0)
	{
		ConnectedSocketError = SocketErrorCode::Error_Bind;
		UE_LOGF(LogDatasmithDispatcher, Error, "Socket binding failure");
		return;
	}
	ensure(BoundPort == ListeningSocket->GetPortNo());

	if (!ListeningSocket->Listen(0))
	{
		ConnectedSocketError = SocketErrorCode::Error_Listen;
		UE_LOGF(LogDatasmithDispatcher, Error, "Socket listen failure");
		return;
	}
}

FNetworkServerNode::~FNetworkServerNode()
{
	CloseSocket(ListeningSocket);
}

int32 FNetworkServerNode::GetListeningPort()
{
	return ListeningSocket ? ListeningSocket->GetPortNo() : 0;
}

bool FNetworkServerNode::Accept(const FString& Description, double Timeout_s)
{
	CloseSocket(ConnectedSocket);

	// wait until a connection occurs
	ListeningSocket->SetNonBlocking();
	double AcceptLimit_s = FPlatformTime::Seconds() + Timeout_s;
	do
	{
		ConnectedSocket = ListeningSocket->Accept(Description);
		if (ConnectedSocket)
		{
			return true;
		}
		FPlatformProcess::Sleep(0.1f);
	}
	while(FPlatformTime::Seconds() < AcceptLimit_s);

	return false;
}

bool FNetworkClientNode::Connect(const FString& Description, int32 ServerPort, double Timeout_s)
{
	CloseSocket(ConnectedSocket);
	ConnectedSocket = CreateInternalSocket(Description);

	if (!ConnectedSocket)
	{
		ConnectedSocketError = SocketErrorCode::Error_Create;
		return false;
	}

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	TSharedRef<FInternetAddr> InternetAddress = SocketSubsystem->CreateInternetAddr();
	InternetAddress->SetLoopbackAddress();
	InternetAddress->SetPort(ServerPort);

	double ConnectTimeout_s = FPlatformTime::Seconds() + Timeout_s;
	ConnectedSocket->SetNonBlocking(true);
	do
	{
		if (ConnectedSocket->Connect(*InternetAddress))
		{
			UE_LOGF(LogDatasmithDispatcher, Verbose, "Client Node is connected");
			return true;
		}
	}
	while(FPlatformTime::Seconds() < ConnectTimeout_s);

	UE_LOGF(LogDatasmithDispatcher, Error, "Client socket failed to connect");
	CloseSocket(ConnectedSocket);
	return false;
}

void FCommandQueue::SetNetworkInterface(FNetworkNode* InNetworkInterface)
{
	NetworkInterface = InNetworkInterface;
}

TSharedPtr<ICommand> FCommandQueue::GetNextCommand(double Timeout_s)
{
	// consume all available commands
	while (Poll(Timeout_s))
	{
		Timeout_s = 0;
	}

	TSharedPtr<ICommand> OutCommand;
	return InCommands.Dequeue(OutCommand) ? OutCommand : nullptr;
}

bool FCommandQueue::SendCommand(ICommand& Commmand, double Timeout_s)
{
	TArray<uint8> CommandBuffer;
	SerializeCommand(Commmand, CommandBuffer);
	return NetworkInterface && NetworkInterface->SendMessage(CommandBuffer, Timeout_s);
}

void FCommandQueue::Disconnect(double Timeout_s)
{
	// consume all available commands before closing the connection
	while (Poll(Timeout_s))
	{
		Timeout_s = 0;
	}

	NetworkInterface = nullptr;
}

bool FCommandQueue::Poll(double Timeout_s)
{
	TArray<uint8> CommandBuffer;
	if (NetworkInterface && NetworkInterface->ReceiveMessage(CommandBuffer, Timeout_s))
	{
		if (TSharedPtr<ICommand> Command = DeserializeCommand(CommandBuffer))
		{
			InCommands.Enqueue(Command);
			return true;
		}
	}
	return false;
}

} // ns DatasmithDispatcher
