// Copyright Epic Games, Inc. All Rights Reserved.
#include "DatasmithWorkerHandler.h"

#include "CADToolsModule.h"
#include "DatasmithCommands.h"
#include "DatasmithDispatcher.h"
#include "DatasmithDispatcherConfig.h"
#include "DatasmithDispatcherLog.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

namespace DatasmithDispatcher
{

static FString GetWorkerExecutablePath()
{
	static FString ProcessorPath = [&]()
	{
		FString Path = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Enterprise/DatasmithCADImporter/Binaries"));

#if PLATFORM_MAC
		Path = FPaths::Combine(Path, TEXT("Mac/DatasmithCADWorker"));
#elif PLATFORM_LINUX
		Path = FPaths::Combine(Path, TEXT("Linux/DatasmithCADWorker"));
#elif PLATFORM_WINDOWS
		Path = FPaths::Combine(Path, TEXT("Win64/DatasmithCADWorker.exe"));
#endif
		if (!FPaths::FileExists(Path))
		{
			UE_LOGF(LogDatasmithDispatcher, Warning, "CADWorker executable not found. Expected location: %ls", *FPaths::ConvertRelativePathToFull(ProcessorPath));
		}
		return Path;
	}();

	return ProcessorPath;
}

FDatasmithWorkerHandler::FDatasmithWorkerHandler(FDatasmithDispatcher& InDispatcher, const CADLibrary::FImportParameters& InImportParameters, FString& InCachePath, uint32 Id)
	: Dispatcher(InDispatcher)
	, WorkerState(EWorkerState::Uninitialized)
	, ErrorState(EWorkerErrorState::Ok)
	, CachePath(InCachePath)
	, ImportParametersCommand(InImportParameters)
	, bShouldTerminate(false)
{
	ThreadName = FString(TEXT("DatasmithWorkerHandler_")) + FString::FromInt(Id);
	IOTask = UE::Tasks::Launch(*ThreadName, [this]() { Run(); } );
}

FDatasmithWorkerHandler::~FDatasmithWorkerHandler()
{
	IOTask.Wait();
}

void FDatasmithWorkerHandler::StartWorkerProcess()
{
	uint32 WorkerProcessId = 0;

	ensure(ErrorState == EWorkerErrorState::Ok);
	FString ProcessorPath = GetWorkerExecutablePath();
	if (FPaths::FileExists(ProcessorPath))
	{
		int32 ListenPort = NetworkInterface.GetListeningPort();
		if (ListenPort == 0)
		{
			ErrorState = EWorkerErrorState::ConnectionFailed_NotBound;
			return;
		}

		FString CommandToProcess;

		// Manually set worker BaseDir of worker process (as automatic deduction is broken in our case,
		// probably due to the abuse of ExeBinariesSubFolder in DatasmithCADWorker.Target.cs)
		// This fixes paths of logs generated by the worker.
		CommandToProcess += TEXT("-BaseDir=\"") + FString(FPlatformProcess::BaseDir()) + TEXT('"');

		CommandToProcess += TEXT(" -ServerPID ") + FString::FromInt(FPlatformProcess::GetCurrentProcessId());
		CommandToProcess += TEXT(" -ServerPort ") + FString::FromInt(ListenPort);
		CommandToProcess += TEXT(" -CacheDir \"") + CachePath + TEXT('"');
		CommandToProcess += TEXT(" -CacheVersion ") + FString::FromInt(FCADToolsModule::GetCacheVersion());
		CommandToProcess += TEXT(" -EnginePluginsDir \"") + FPaths::EngineDir() + TEXT('"');
		UE_LOGF(LogDatasmithDispatcher, Verbose, "CommandToProcess: %ls", *CommandToProcess);

		WorkerHandle = FPlatformProcess::CreateProc(*ProcessorPath, *CommandToProcess, true, false, false, &WorkerProcessId, 0, nullptr, nullptr);
	}

	if (!WorkerHandle.IsValid() || !FPlatformProcess::IsProcRunning(WorkerHandle) || !FPlatformProcess::IsApplicationRunning(WorkerProcessId))
	{
		ErrorState = EWorkerErrorState::WorkerProcess_CantCreate;
		return;
	}
}

void FDatasmithWorkerHandler::ValidateConnection()
{
	if (!NetworkInterface.IsValid())
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "NetworkInterface lost");
		WorkerState = EWorkerState::Closing;
		ErrorState = EWorkerErrorState::ConnectionLost;
	}
	else if (WorkerHandle.IsValid() && !FPlatformProcess::IsProcRunning(WorkerHandle))
	{
		UE_LOGF(LogDatasmithDispatcher, Error, "Worker lost");
		WorkerState = EWorkerState::Closing;
		ErrorState = EWorkerErrorState::WorkerProcess_Lost;
	}
}

void FDatasmithWorkerHandler::Run()
{
	WorkerState = EWorkerState::Uninitialized;
	RunInternal();
	WorkerState = EWorkerState::Terminated;
	UE_CLOGF(ErrorState != EWorkerErrorState::Ok, LogDatasmithDispatcher, Warning, "Handler ended with error: %ls", EWorkerErrorStateAsString(ErrorState));
}

void FDatasmithWorkerHandler::RunInternal()
{
	while (IsAlive())
	{
		switch (WorkerState)
		{
			case EWorkerState::Uninitialized:
			{
				ErrorState = EWorkerErrorState::Ok;

				StartWorkerProcess();

				if (ErrorState != EWorkerErrorState::Ok)
				{
					WorkerState = EWorkerState::Terminated;
					break;
				}

				// The Accept() call on the server blocks until a connection is initiated from a client
				static const FString SocketDescription = TEXT("DatasmithWorkerHandler");
				if (!NetworkInterface.Accept(SocketDescription, Config::AcceptTimeout_s))
				{
					ErrorState = EWorkerErrorState::ConnectionFailed_NoClient;
				}
				else
				{
					CommandIO.SetNetworkInterface(&NetworkInterface);
				}

				if (ErrorState != EWorkerErrorState::Ok)
				{
					WorkerState = EWorkerState::Closing;
					break;
				}

				if (CommandIO.SendCommand(ImportParametersCommand, Config::SendCommandTimeout_s))
				{
					UE_LOGF(LogDatasmithDispatcher, Verbose, "Import Parameters sent");
				}

				WorkerState = EWorkerState::Idle;
				break;
			}

			case EWorkerState::Idle:
			{
				// Fetch a new task
				ensureMsgf(CurrentTask.IsSet() == false, TEXT("We should not have a current task when fetching a new one"));
				CurrentTask = Dispatcher.GetNextTask();

				if (CurrentTask.IsSet())
				{
					UE_LOGF(LogDatasmithDispatcher, Log, "   - Next task: %ls", *CurrentTask->FileDescription.GetFileName());
					FRunTaskCommand NewTask(CurrentTask.GetValue());

					if (CommandIO.SendCommand(NewTask, Config::SendCommandTimeout_s))
					{
						UE_LOGF(LogDatasmithDispatcher, Verbose, "New task command sent");
						WorkerState = EWorkerState::Processing;
					}
					else
					{
						// Signal that the Task was not processed
						Dispatcher.SetTaskState(CurrentTask->Index, ETaskState::UnTreated);

						UE_LOGF(LogDatasmithDispatcher, Error, "New task command issue");
						WorkerState = EWorkerState::Closing;
						ErrorState = EWorkerErrorState::ConnectionLost_SendFailed;
					}
				}
				else if (bShouldTerminate)
				{
					UE_LOGF(LogDatasmithDispatcher, Verbose, "Exit loop gracefully");
					WorkerState = EWorkerState::Closing;
				}
				else
				{
					ValidateConnection();

					// consume
					if (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(Config::IdleLoopDelay))
					{
						ProcessCommand(*Command);
					}
				}

				break;
			}

			case EWorkerState::Processing:
			{
				if (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(Config::ProcessingLoopDelay))
				{
					ProcessCommand(*Command);

					bool bProcessingOver = CurrentTask.IsSet() == false;
					if (bProcessingOver && bShouldTerminate)
					{
						WorkerState = EWorkerState::Closing;
					}
				}
				else
				{
					ValidateConnection();
					if (ErrorState == EWorkerErrorState::WorkerProcess_Lost || ErrorState == EWorkerErrorState::ConnectionLost)
					{
						if (CurrentTask.IsSet())
						{
							Dispatcher.SetTaskState(CurrentTask->Index, ETaskState::ProcessFailed);
							CurrentTask.Reset();
						}

						WorkerState = EWorkerState::Closing;
					}
				}
				break;
			}

			case EWorkerState::Closing:
			{
				// try to close the process gracefully
				if (WorkerHandle.IsValid())
				{
					bool CloseByCommand = Config::CloseProcessByCommand && CommandIO.IsValid() && FPlatformProcess::IsProcRunning(WorkerHandle);

					bool bClosed = false;
					if (CloseByCommand)
					{
						FTerminateCommand Terminate;
						CommandIO.SendCommand(Terminate, 0);

						for (int32 i = 0; i < int32(10. * Config::TerminateTimeout_s); ++i)
						{
							if (!FPlatformProcess::IsProcRunning(WorkerHandle))
							{
								bClosed = true;
								break;
							}
							FPlatformProcess::Sleep(0.1f);
						}
					}

					if (!bClosed)
					{
						FPlatformProcess::TerminateProc(WorkerHandle, true);
					}
				}

				// Process commands still in input queue
				CommandIO.Disconnect(0);
				while (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(0))
				{
					ProcessCommand(*Command);
				}

				WorkerState = EWorkerState::Terminated;
				break;
			}

			default:
			{
				ensureMsgf(false, TEXT("missing case handling"));
			}
		}
	}
}

void FDatasmithWorkerHandler::Stop()
{
	bShouldTerminate = true;
}

bool FDatasmithWorkerHandler::IsAlive() const
{
	return WorkerState != EWorkerState::Terminated;
}

bool FDatasmithWorkerHandler::IsRestartable() const
{
	return !IsAlive() && (
		ErrorState == EWorkerErrorState::WorkerProcess_Lost ||
		ErrorState == EWorkerErrorState::ConnectionLost
	);
}

void FDatasmithWorkerHandler::ProcessCommand(ICommand& Command)
{
	switch (Command.GetType())
	{
		case ECommandId::Ping:
			ProcessCommand(StaticCast<FPingCommand&>(Command));
			break;

		case ECommandId::NotifyEndTask:
			ProcessCommand(StaticCast<FCompletedTaskCommand&>(Command));
			break;

		default:
			break;
	}
}

void FDatasmithWorkerHandler::ProcessCommand(FPingCommand& PingCommand)
{
	DatasmithDispatcher::FBackPingCommand BackPing;
	CommandIO.SendCommand(BackPing, 0);
}

void FDatasmithWorkerHandler::ProcessCommand(FCompletedTaskCommand& CompletedTaskCommand)
{
	using namespace CADLibrary;
	if (!CurrentTask.IsSet())
	{
		return;
	}

	const EMesher DefaultMesher = FImportParameters::bGDisableCADKernelTessellation ? EMesher::TechSoft : EMesher::CADKernel;

	if(CompletedTaskCommand.ProcessResult == ETaskState::ProcessOk)
	{
		for (const FFileDescriptor& ExternalReferenceFile : CompletedTaskCommand.ExternalReferences)
		{
			if (ExternalReferenceFile.GetFileName().IsEmpty())
			{
				continue;
			}
			Dispatcher.AddTask(ExternalReferenceFile, DefaultMesher);
		}
		UE_LOGF(LogDatasmithDispatcher, Log, "   - Task completed: %ls", *CurrentTask->FileDescription.GetFileName());
	}
	else if (CompletedTaskCommand.ProcessResult == ETaskState::ProcessFailed && DefaultMesher == EMesher::CADKernel)
	{
		UE_LOGF(LogDatasmithDispatcher, Log, "   - Task failed with CADKernel: %ls", *CurrentTask->FileDescription.GetFileName());
		UE_LOGF(LogDatasmithDispatcher, Log, "      => Add task to process with Techsoft");
		Dispatcher.AddTask(CurrentTask->FileDescription, EMesher::TechSoft);
	}

	Dispatcher.SetTaskState(CurrentTask->Index, CompletedTaskCommand.ProcessResult);


	Dispatcher.LinkCTFileToUnrealCacheFile(CurrentTask->FileDescription, CompletedTaskCommand.SceneGraphFileName, CompletedTaskCommand.GeomFileName);
	Dispatcher.LogMessages(CompletedTaskCommand.Messages);
	CurrentTask.Reset();

	if (CompletedTaskCommand.ProcessResult == ETaskState::Unknown)
	{
		// try to close the process gracefully
		if (WorkerHandle.IsValid())
		{
			FPlatformProcess::TerminateProc(WorkerHandle, true);
		}

		// Process commands still in input queue
		CommandIO.Disconnect(0);
		while (TSharedPtr<ICommand> Command = CommandIO.GetNextCommand(0))
		{
			ProcessCommand(*Command);
		}
		WorkerState = EWorkerState::Uninitialized;
	}
	else
	{
		WorkerState = EWorkerState::Idle;
	}
}

const TCHAR* FDatasmithWorkerHandler::EWorkerErrorStateAsString(EWorkerErrorState Error)
{
	switch (Error)
	{
		case EWorkerErrorState::Ok: return TEXT("Ok");
		case EWorkerErrorState::ConnectionFailed_NotBound: return TEXT("Connection failed (socket not bound)");
		case EWorkerErrorState::ConnectionFailed_NoClient: return TEXT("Connection failed (No connection from client)");
		case EWorkerErrorState::ConnectionLost: return TEXT("Connection lost");
		case EWorkerErrorState::ConnectionLost_SendFailed: return TEXT("Connection lost (send failed)");
		case EWorkerErrorState::WorkerProcess_CantCreate: return TEXT("Worker process issue (cannot create worker process)");
		case EWorkerErrorState::WorkerProcess_Lost: return TEXT("Worker process issue (worker lost)");
		default: return TEXT("unknown");
	}
}

} // ns DatasmithDispatcher
