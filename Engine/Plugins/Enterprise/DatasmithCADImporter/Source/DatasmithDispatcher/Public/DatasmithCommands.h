// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CADData.h"
#include "CADOptions.h"
#include "DatasmithDispatcherTask.h"

class FArchive;

namespace DatasmithDispatcher
{

enum class ECommandId : uint8
{
	Invalid,
	Ping,
	BackPing,
	RunTask,
	NotifyEndTask,
	ImportParams,
	Terminate,
	Last
};



class ICommand
{
public:
	virtual ~ICommand() = default;

	virtual ECommandId GetType() const = 0;

	friend void operator<<(FArchive& Ar, ICommand& C) { C.SerializeImpl(Ar); }

protected:
	virtual void SerializeImpl(FArchive&) {}
};
	


// Create a new command from its type
TSharedPtr<ICommand> CreateCommand(ECommandId CommandType);

// Converts a command into a byte buffer
void SerializeCommand(ICommand& Command, TArray<uint8>& OutBuffer);

// Converts byte buffer back into a Command
// returns nullptr in case of error
TSharedPtr<ICommand> DeserializeCommand(const TArray<uint8>& InBuffer);



class FTerminateCommand : public ICommand
{
public:
	virtual ECommandId GetType() const override { return ECommandId::Terminate; }
};


class FPingCommand : public ICommand
{
public:
	virtual ECommandId GetType() const override { return ECommandId::Ping; }
};


class FBackPingCommand : public ICommand
{
public:
	virtual ECommandId GetType() const override { return ECommandId::BackPing; }
};


class FRunTaskCommand : public ICommand
{
public:
	FRunTaskCommand() = default;
	FRunTaskCommand(const FTask& Task) 
		: JobFileDescription(Task.FileDescription)
		, Mesher(Task.Mesher)
		, JobIndex(Task.Index) 
	{}
	virtual ECommandId GetType() const override { return ECommandId::RunTask; }

protected:
	virtual void SerializeImpl(FArchive&) override;

public:
	CADLibrary::FFileDescriptor JobFileDescription;
	CADLibrary::EMesher Mesher;

	int32 JobIndex = -1;
};

class FCompletedTaskCommand : public ICommand
{
public:
	virtual ECommandId GetType() const override { return ECommandId::NotifyEndTask; }

protected:
	virtual void SerializeImpl(FArchive&) override;

public:
	TArray<CADLibrary::FFileDescriptor> ExternalReferences;
	FString SceneGraphFileName;
	FString GeomFileName;
	ETaskState ProcessResult = ETaskState::Unknown;
	TArray<TPair<uint8, FString>> Messages;
};

class FImportParametersCommand : public ICommand
{
public:
	FImportParametersCommand(const CADLibrary::FImportParameters& InImportParameters)
		: ImportParameters(InImportParameters)
	{
	}

	FImportParametersCommand()
	{
	}

	virtual ECommandId GetType() const override { return ECommandId::ImportParams; }

protected:
	virtual void SerializeImpl(FArchive&) override;

public:
	CADLibrary::FImportParameters ImportParameters;
};

} // ns DatasmithDispatcher
