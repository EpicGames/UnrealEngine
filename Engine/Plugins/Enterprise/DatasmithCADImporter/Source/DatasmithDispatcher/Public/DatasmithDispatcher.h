// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CADOptions.h"
#include "DatasmithDispatcherTask.h"
#include "DatasmithWorkerHandler.h"

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/Thread.h"

namespace DatasmithDispatcher
{

// Handle a list of tasks, and a set of external workers to consume them.
// Concept of task is currently tightly coupled with cad usage...
class DATASMITHDISPATCHER_API FDatasmithDispatcher
{
public:
	FDatasmithDispatcher(const CADLibrary::FImportParameters& InImportParameters, const FString& InCacheDir, TMap<uint32, FString>& CADFileToUnrealFileMap, TMap<uint32, FString>& CADFileToUnrealGeomMap);

	void AddTask(const CADLibrary::FFileDescriptor & FileDescription, CADLibrary::EMesher Mesher);
	TOptional<FTask> GetNextTask();
	void SetTaskState(int32 TaskIndex, ETaskState TaskState);

	void Process(bool bWithProcessor);
	bool IsOver();

	void LinkCTFileToUnrealCacheFile(const CADLibrary::FFileDescriptor& CTFileDescription, const FString& UnrealSceneGraphFile, const FString& UnrealGeomFile);

	void LogMessages(const TArray<TPair<uint8, FString>>& Warnings) const;

	void SetWorkerCount(const int32 InNumberOfWorkers)
	{
		CADLibrary::GMaxImportThreads = InNumberOfWorkers;
	}

private:
	void SpawnHandlers();
	int32 GetNextWorkerId();
	void CloseHandlers();

	void ProcessLocal();

private:
	// Tasks
	FCriticalSection TaskPoolCriticalSection;
	TArray<FTask> TaskPool;
	int32 NextTaskIndex;
	int32 CompletedTaskCount;

	// Scene wide state
	TMap<uint32, FString>& CADFileToUnrealFileMap;
	TMap<uint32, FString>& CADFileToUnrealGeomMap;
	FString ProcessCacheFolder;
	CADLibrary::FImportParameters ImportParameters;

	// Workers
	int32 NumberOfWorkers;
	int32 NextWorkerId;
	TArray<FDatasmithWorkerHandler> WorkerHandlers;
};

} // NS DatasmithDispatcher
