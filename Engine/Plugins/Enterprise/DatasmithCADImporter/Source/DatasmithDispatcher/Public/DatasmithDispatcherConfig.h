// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace DatasmithDispatcher
{
namespace Config
{

// Server side timeout for connection with the client
constexpr double AcceptTimeout_s = 6.0;

// Client side timeout for connection with the server
constexpr double ConnectTimeout_s = 3.0;

// Maximum time we accept to wait when a command is sent.
// If the command is not sent by this delay, the operation is considered as failed
constexpr double SendCommandTimeout_s = 1.0;

// When true, a Terminate command is sent to the processor and we wait for
// TerminateTimeout_s seconds for it to close, before a hard TerminateProc.
// When false, the process is directly closed with a TerminateProc.
constexpr bool CloseProcessByCommand = true;

// Maximum time we accept to wait for the process to close after a Terminate
// command has been sent.
constexpr double TerminateTimeout_s = 1.0;

// Delay between successive checks for a new task, and validation of the connection
constexpr double IdleLoopDelay = 0.1;

// Delay between successive validation of the connection
constexpr double ProcessingLoopDelay = 0.5;

// Number of worker instance that can be restarted when a crash occurs in en external process
constexpr int32 MaxRestartAllowed = 1000;

} // ns Config
} // ns DatasmithDispatcher
