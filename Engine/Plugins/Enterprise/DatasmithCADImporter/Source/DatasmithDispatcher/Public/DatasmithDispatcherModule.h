// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#define DATASMITHDISPATCHER_MODULE_NAME TEXT("DatasmithDispatcher")

/**
 * This module exposes additional features for assets containing CoreTech data.
 */
class FDatasmithDispatcherModule : public IModuleInterface
{
public:
	static FDatasmithDispatcherModule& Get();
	static bool IsAvailable();

private:
	virtual void StartupModule() override;
};
