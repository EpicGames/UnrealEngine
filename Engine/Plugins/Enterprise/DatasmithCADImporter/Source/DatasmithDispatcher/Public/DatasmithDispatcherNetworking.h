// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithCommands.h"

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "Sockets.h"


namespace DatasmithDispatcher
{

enum class SocketErrorCode
{
	NoError = 0,
	Error_Create,
	Error_Bind,
	Error_Listen,
	UnableToReadOnSocket,
	UnableToSendData,
	CouldNotStartWSA,
	UnableToGetLocalAddress,
	ConnectionToServerFailed,
};


// uint8 hash[20];
// FSHA1::HashBuffer(const void* Data, uint64 DataSize, uint8* OutHash);

struct FMessageHeader
{
	int32 ByteSize = -1;

	friend void operator<< (FArchive& Ar, FMessageHeader& H)
	{
		FString Guard = Ar.IsLoading() ? TEXT("") : TEXT("guard");
		Ar << Guard;
		Ar << H.ByteSize;
		ensure(Guard == TEXT("guard"));
	}
};


class FNetworkNode
{
public:
	virtual ~FNetworkNode();

	// Send a byte buffer as an atomic message
	// returns true when a message is successfully sent
	bool SendMessage(const TArray<uint8>& Buffer, double Timeout_s);

	// Receive a byte buffer as an atomic message
	// returns true when a message is fetched. In that case, OutBuffer contains the message.
	bool ReceiveMessage(TArray<uint8>& OutBuffer, double Timeout_s);

	bool IsValid() const { return !bReadError && !bWriteError; }

protected:
	struct FMessage
	{
		FMessageHeader Header;
		TArray<uint8> Content;
	};
	FSocket* CreateInternalSocket(const FString& Description);
	void CloseSocket(FSocket*& Socket);
	bool IsConnected();

protected:
	FSocket* ConnectedSocket = nullptr;
	SocketErrorCode ConnectedSocketError = SocketErrorCode::NoError;

	FMessage IncommingMessage;

	bool bReadError = false;
	bool bWriteError = false;
};



// Create, Bind, Listen, Accept sequence
class FNetworkServerNode : public FNetworkNode
{
public:
	FNetworkServerNode();
	~FNetworkServerNode();
	int32 GetListeningPort();
	bool Accept(const FString& Description, double Timeout_s);

private:
	FSocket* ListeningSocket;
};



// Create, Connect sequence
class FNetworkClientNode : public FNetworkNode
{
public:
	bool Connect(const FString& Description, int32 ServerPort, double Timeout_s);
};




class FCommandQueue
{
public:
	void SetNetworkInterface(FNetworkNode* InNetworkInterface);

	TSharedPtr<ICommand> GetNextCommand(double Timeout_s);
	bool SendCommand(ICommand& Commmand, double Timeout_s);

	bool IsValid() { return NetworkInterface ? NetworkInterface->IsValid() : true;}

	void Disconnect(double Timeout_s);

private:
	bool Poll(double Timeout_s);
	TQueue<TSharedPtr<ICommand>> InCommands;

private:
	FNetworkNode* NetworkInterface = nullptr;
};


} // ns DatasmithDispatcher
