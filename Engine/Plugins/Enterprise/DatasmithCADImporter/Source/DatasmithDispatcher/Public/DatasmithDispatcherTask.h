// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CADData.h"
#include "CADOptions.h"

namespace DatasmithDispatcher
{
using ETaskState = CADLibrary::ECADParsingResult;

struct FTask
{
	FTask() = default;

	FTask(const CADLibrary::FFileDescriptor& InFile, const CADLibrary::EMesher InMesher)
		: FileDescription(InFile)
		, Mesher(InMesher)
		, State(ETaskState::UnTreated)
	{
	}

	CADLibrary::FFileDescriptor FileDescription;
	CADLibrary::EMesher Mesher;
	int32 Index = -1;
	ETaskState State = ETaskState::Unknown;

};

} // NS DatasmithDispatcher
