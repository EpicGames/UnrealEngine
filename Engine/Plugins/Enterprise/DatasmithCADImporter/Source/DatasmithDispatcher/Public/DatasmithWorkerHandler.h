// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CADOptions.h"
#include "DatasmithCommands.h"
#include "DatasmithDispatcherNetworking.h"
#include "DatasmithDispatcherTask.h"

#include "HAL/PlatformProcess.h"
#include "Tasks/Task.h"

namespace DatasmithDispatcher
{

class FDatasmithDispatcher;

//Handle a Worker by socket communication
class DATASMITHDISPATCHER_API FDatasmithWorkerHandler
{
	enum class EWorkerState
	{
		Uninitialized,
		Idle, // Initialized, available for processing
		Processing, // Currently processing a task
		Closing, // in the process of terminating
		Terminated, // aka. Not Alive
	};

	enum class EWorkerErrorState
	{
		Ok,
		ConnectionFailed_NotBound,
		ConnectionFailed_NoClient,
		ConnectionLost,
		ConnectionLost_SendFailed,
		WorkerProcess_CantCreate,
		WorkerProcess_Lost,
	};

public:
	FDatasmithWorkerHandler(FDatasmithDispatcher& InDispatcher, const CADLibrary::FImportParameters& InImportParameters, FString& InCachePath, uint32 Id);
	~FDatasmithWorkerHandler();

	void Run();
	bool IsAlive() const;
	bool IsRestartable() const;
	void Stop();

private:
	void RunInternal();
	void StartWorkerProcess();
	void ValidateConnection();

	void ProcessCommand(ICommand& Command);
	void ProcessCommand(FPingCommand& PingCommand);
	void ProcessCommand(FCompletedTaskCommand& RunTaskCommand);
	const TCHAR* EWorkerErrorStateAsString(EWorkerErrorState e);

private:
	FDatasmithDispatcher& Dispatcher;

	// Send and receive commands
	FNetworkServerNode NetworkInterface;
	FCommandQueue CommandIO;
	UE::Tasks::FTask IOTask;
	FString ThreadName;

	// External process
	FProcHandle WorkerHandle;
	TAtomic<EWorkerState> WorkerState;
	EWorkerErrorState ErrorState;

	// self
	FString CachePath;
	FImportParametersCommand ImportParametersCommand;
	TOptional<FTask> CurrentTask;
	bool bShouldTerminate;

};
} // namespace DatasmithDispatcher
