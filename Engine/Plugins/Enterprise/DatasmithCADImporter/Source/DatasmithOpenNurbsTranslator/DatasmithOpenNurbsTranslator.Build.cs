// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System;

namespace UnrealBuildTool.Rules
{
	public class DatasmithOpenNurbsTranslator : ModuleRules
	{
		public DatasmithOpenNurbsTranslator(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CADInterfaces",
					"CADKernel",
					"CADKernelSurface",
					"CADLibrary",
					"CADTools",
					"DatasmithCore",
					"DatasmithContent",
					"DatasmithTranslator",
					"Engine",
					"MeshDescription",
					"ParametricSurface",
					"StaticMeshDescription",
				}
			);

			if (System.Type.GetType("OpenNurbs6") != null)
			{
				PrivateDependencyModuleNames.Add("OpenNurbs6");
			}

            if (Target.Type == TargetType.Editor)
            {
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
	 					"MessageLog",
	 					"UnrealEd",
					}
				);
			}

		}
    }
}
