// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithOpenNurbsImportOptions.h"

#include "CADInterfacesModule.h"
#include "DatasmithOpenNurbsTranslatorModule.h"

#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "DatasmithOpenNurbsImportPlugin"

#if WITH_EDITOR

bool UDatasmithOpenNurbsImportOptions::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty) || !InProperty)
	{
		return false;
	}

	FName PropertyFName = InProperty->GetFName();

	if (PropertyFName == GET_MEMBER_NAME_CHECKED(FDatasmithOpenNurbsOptions, Geometry))
	{
		if (ICADInterfacesModule::GetAvailability() == ECADInterfaceAvailability::Unavailable)
		{
			return false;
		}
		return true;
	}
	else if (PropertyFName == GET_MEMBER_NAME_CHECKED(FDatasmithOpenNurbsOptions, ChordTolerance)
		|| PropertyFName == GET_MEMBER_NAME_CHECKED(FDatasmithOpenNurbsOptions, MaxEdgeLength)
		|| PropertyFName == GET_MEMBER_NAME_CHECKED(FDatasmithOpenNurbsOptions, NormalTolerance)
		|| PropertyFName == GET_MEMBER_NAME_CHECKED(FDatasmithOpenNurbsOptions, StitchingTechnique)
		)
	{
		// Enable tessellation options only when using CAD library to tessellate
		return Options.Geometry == EDatasmithOpenNurbsBrepTessellatedSource::UseUnrealNurbsTessellation;
	}

	return true;
}
#endif //WITH_EDITOR


#undef LOCTEXT_NAMESPACE
