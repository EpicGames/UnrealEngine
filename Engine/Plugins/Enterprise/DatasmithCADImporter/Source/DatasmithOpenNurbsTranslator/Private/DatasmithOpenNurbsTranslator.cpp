// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithOpenNurbsTranslator.h"
#include "DatasmithOpenNurbsTranslatorModule.h"

#ifdef USE_OPENNURBS // The whole translation unit is skipped without OpenNurbs TPS library

#include "CADInterfacesModule.h"
#include "CADKernelSurfaceExtension.h"
#include "CADModelConverter.h"
#include "CADOptions.h"
#include "DatasmithImportOptions.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithMaterialsUtils.h"
#include "DatasmithMesh.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneSource.h"
#include "DatasmithUtils.h"
#include "OpenNurbsBRepConverter.h"
#include "OpenNurbsBRepToCADKernelConverter.h"
#include "OpenNurbsBRepToTechSoftConverter.h" // requires Techsoft as public dependency
#include "Utility/DatasmithMeshHelper.h"

#if WITH_EDITOR
#include "Editor.h"
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#endif
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

// Disable macro redefinition warning
#pragma warning(push)
#pragma warning(disable:4005)
#include "opennurbs.h"
#pragma warning(pop)

#include <deque>
#include <map>
#include <set>

DEFINE_LOG_CATEGORY_STATIC(LogDatasmithOpenNurbsTranslator, Log, All);

#define LOCTEXT_NAMESPACE "DatasmithOpenNurbsTranslator"

// Cache for already processed data (only linked file references for now)
class FTranslationCache
{
public:
	TSharedPtr<IDatasmithActorElement> GetElementForLinkedFileReference(const FString& FilePath)
	{
		TSharedPtr<IDatasmithActorElement>* Element = LinkedFileReferenceToElements.Find(FilePath);
		if (Element)
		{
			return *Element;
		}
		return TSharedPtr<IDatasmithActorElement>();
	}

	void AddElementForLinkedFileReference(const FString& FilePath, const TSharedPtr<IDatasmithActorElement>& Element)
	{
		LinkedFileReferenceToElements.Add(FilePath, Element);
	}

private:
	TMap<FString, TSharedPtr<IDatasmithActorElement>> LinkedFileReferenceToElements;
};

TSharedPtr<IDatasmithActorElement> DuplicateActorElement(TSharedPtr<IDatasmithActorElement> ActorElement, const FString& DuplicateName)
{
	TSharedPtr<IDatasmithActorElement> DuplicatedElement;
	FString DuplicatedActorName = DuplicateName + ActorElement->GetName();
	if (ActorElement->IsA(EDatasmithElementType::StaticMeshActor))
	{
		const TSharedPtr< IDatasmithMeshActorElement >& MeshActorElement = StaticCastSharedPtr< IDatasmithMeshActorElement >(ActorElement);

		TSharedPtr<IDatasmithMeshActorElement> DuplicatedMeshActorElement = FDatasmithSceneFactory::CreateMeshActor(*DuplicatedActorName);
		DuplicatedMeshActorElement->SetStaticMeshPathName(MeshActorElement->GetStaticMeshPathName());
		DuplicatedElement = DuplicatedMeshActorElement;
	}
	else if (ActorElement->IsA(EDatasmithElementType::Actor))
	{
		DuplicatedElement = FDatasmithSceneFactory::CreateActor(*DuplicatedActorName);
	}

	DuplicatedElement->SetLabel(ActorElement->GetLabel());
	DuplicatedElement->SetTranslation(ActorElement->GetTranslation());
	DuplicatedElement->SetScale(ActorElement->GetScale());
	DuplicatedElement->SetRotation(ActorElement->GetRotation());

	int32 NumChildren = ActorElement->GetChildrenCount();
	for (int32 Index = 0; Index < NumChildren; ++Index)
	{
		TSharedPtr<IDatasmithActorElement> Child = ActorElement->GetChild(Index);
		DuplicatedElement->AddChild(DuplicateActorElement(Child, DuplicateName));
	}

	int NumTags = ActorElement->GetTagsCount();
	for (int32 TagIndex = 0; TagIndex < NumTags; ++TagIndex)
	{
		DuplicatedElement->AddTag(ActorElement->GetTag(TagIndex));
	}

	return DuplicatedElement;
}

TSharedPtr<IDatasmithMetaDataElement> DuplicateMetaDataElement(const TSharedPtr<IDatasmithMetaDataElement>& SourceMetaData, const TSharedPtr<IDatasmithActorElement>& TargetActorElement)
{
	FString MetaDataName = FString::Printf(TEXT("%s%s"), TargetActorElement->GetName(), TEXT("_DATA"));
	TSharedPtr<IDatasmithMetaDataElement> DuplicatedMetaData = FDatasmithSceneFactory::CreateMetaData(*MetaDataName);
	DuplicatedMetaData->SetAssociatedElement(TargetActorElement);

	for (int32 PropertyIndex = 0, PropertyCount = SourceMetaData->GetPropertiesCount(); PropertyIndex < PropertyCount; ++PropertyIndex)
	{
		TSharedPtr<IDatasmithKeyValueProperty> SourceProperty = SourceMetaData->GetProperty(PropertyIndex);
		if (SourceProperty)
		{
			TSharedRef<IDatasmithKeyValueProperty> DuplicatedProperty = FDatasmithSceneFactory::CreateKeyValueProperty(SourceProperty->GetName());

			DuplicatedProperty->SetValue(SourceProperty->GetValue());
			DuplicatedMetaData->AddProperty(DuplicatedProperty);
		}
	}

	return DuplicatedMetaData;
}

// #ueent_wip: test with CADSDK_ENABLED undefined
class FOpenNurbsObjectWrapper
{
public:
	ON_Object* ObjectPtr;
	ON_3dmObjectAttributes Attributes;

	FOpenNurbsObjectWrapper()
		: ObjectPtr(nullptr)
	{
	}

	~FOpenNurbsObjectWrapper()
	{
		if (ObjectPtr)
		{
			delete ObjectPtr;
			ObjectPtr = nullptr;
		}
	}
};

namespace DatasmithOpenNurbsTranslatorUtils
{
	void XFormToMatrix(const ON_Xform& xform, FMatrix& matrix)
	{
		for (int iRow = 0; iRow < 4; ++iRow)
		{
			for (int iColumn = 0; iColumn < 4; ++iColumn)
			{
				matrix.M[iColumn][iRow] = xform[iRow][iColumn];
			}
		}
	}

	bool CheckForCRCErrors(ON_BinaryArchive& archive, const char* sSection, int& NumCRCErrors)
	{
		// returns true if new CRC errors are found
		bool bHasCRCErrors = false;
		int CRCCount = archive.BadCRCCount();

		if (NumCRCErrors != CRCCount)
		{
			NumCRCErrors = CRCCount;
			bHasCRCErrors = true;
		}

		return bHasCRCErrors;
	}

	FString BuildMeshName(FString& SceneName, const FOpenNurbsObjectWrapper& Object)
	{
		ON_wString uuidString;
		ON_UuidToString(Object.Attributes.m_uuid, uuidString);

		return uuidString.Array();
	}

	bool HasPackedTextureRegion(const ON_Mesh& mesh)
	{
		return (ON_IsValid(mesh.m_srf_scale[0]) && mesh.m_srf_scale[0] > 0.0
			&& ON_IsValid(mesh.m_srf_scale[1]) && mesh.m_srf_scale[1] > 0.0
			&& mesh.m_packed_tex_domain[0].IsInterval()
			&& ON_Interval::ZeroToOne.Includes(mesh.m_packed_tex_domain[0])
			&& mesh.m_packed_tex_domain[1].IsInterval()
			&& ON_Interval::ZeroToOne.Includes(mesh.m_packed_tex_domain[1])
			);
	}

	FVector2D GetMeshTexCoords(const ON_Mesh* mesh, const int vertexCount, const int texCoordIndex, bool hasPackedTexCoords)
	{
		// Ref. getMeshTexCoords
		// Use values in m_T if number of values in m_T matches number of vertices
		if (mesh->m_T.Count() == vertexCount)
		{
			return FVector2D(mesh->m_T[texCoordIndex][0], mesh->m_T[texCoordIndex][1]);
		}

		// No computed texture coordinates, try to compute based on surface parameters
		if (mesh->m_S.Count() == vertexCount)
		{
			ON_2dPoint texCoords = mesh->m_S[texCoordIndex];

			texCoords[0] = mesh->m_srf_domain[0].NormalizedParameterAt(texCoords[0]);
			texCoords[1] = mesh->m_srf_domain[1].NormalizedParameterAt(texCoords[1]);

			if (hasPackedTexCoords == true)
			{
				texCoords[0] = mesh->m_packed_tex_domain[0].ParameterAt(texCoords[0]);
				texCoords[1] = mesh->m_packed_tex_domain[1].ParameterAt(mesh->m_packed_tex_rotate ? 1.0 - texCoords[1] : texCoords[1]);

				return FVector2D(texCoords[0], texCoords[1]);
			}

			texCoords[0] = mesh->m_srf_scale[0] * texCoords[0];
			texCoords[1] = mesh->m_srf_scale[1] * texCoords[1];

			if (mesh->m_Ttag.IsSet() && !mesh->m_Ttag.IsDefaultSurfaceParameterMapping())
			{
				texCoords = mesh->m_Ttag.m_mesh_xform * texCoords;
			}

			return FVector2D(texCoords[0], texCoords[1]);
		}

		// No useful texture coordinates
		return FVector2D::ZeroVector;
	}

	struct FNode
	{
		FNode()
		{ }

		FNode(float x, float y, float z)
			: Vertex(x, y, z)
		{ }

		void SetVertex(float x, float y, float z)
		{
			Vertex = FVector3f(x, y, z);
		}

		void SetNormal(float x, float y, float z)
		{
			Normal = FVector3f(x, y, z);
		}

		void SetNormal(const ON_3fVector& InNormal)
		{
			Normal = FVector3f(InNormal[0], InNormal[1], InNormal[2]);
		}

		FVector3f Vertex;
		FVector3f Normal;
	};

	struct FFace
	{
		FFace()
		{
			Nodes[0] = nullptr;
			Nodes[1] = nullptr;
			Nodes[2] = nullptr;
		}

		FFace(const FNode& Node1, const FNode& Node2, const FNode& Node3)
		{
			Nodes[0] = &Node1;
			Nodes[1] = &Node2;
			Nodes[2] = &Node3;
		}

		const FNode* Nodes[3];
		FVector2f UVCoords[3];
	};

	bool TranslateMesh(const ON_Mesh** Meshes, int MeshCount, FMeshDescription& MeshDescription, bool& bHasNormal, double ScalingFactor, const ON_3dVector& Offset, bool bHasFaceMaterialChannel, int32* FaceMaterialChannel)
	{
		if (!Meshes || !MeshCount)
		{
			return false;
		}

		MeshDescription.Empty();

		TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = MeshDescription.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);

		// Prepared for static mesh usage ?
		if (!VertexPositions.IsValid() || !VertexInstanceNormals.IsValid() || !VertexInstanceUVs.IsValid() || !PolygonGroupImportedMaterialSlotNames.IsValid())
		{
			return false;
		}

		FPolygonGroupID PolyGroupId;
		if (bHasFaceMaterialChannel)
		{
			MeshDescription.ReserveNewPolygonGroups(MeshCount);
		}
		else
		{
			MeshDescription.ReserveNewPolygonGroups(1);
			PolyGroupId = MeshDescription.CreatePolygonGroup();
			PolygonGroupImportedMaterialSlotNames[PolyGroupId] = DatasmithMeshHelper::DefaultSlotName(0);
		}
		// At least one UV set must exist.
		VertexInstanceUVs.SetNumChannels(1);

		for (int32 MeshIndex = 0; MeshIndex < MeshCount; ++MeshIndex)
		{
			const ON_Mesh* Mesh = Meshes[MeshIndex];

			if (Mesh == nullptr || Mesh->VertexCount() == 0 || Mesh->FaceCount() == 0)
			{
				return false;
			}

			bool bHasPackedTexCoords = HasPackedTextureRegion(*Mesh);
			bool bHasUVData = Mesh->HasTextureCoordinates() || bHasPackedTexCoords;

			int VertexCount = Mesh->VertexCount();
			TArray<FNode> Nodes;
			Nodes.Reserve(VertexCount);

			for (int Index = 0; Index < VertexCount; ++Index)
			{
				ON_3fPoint p1 = Mesh->m_V[Index] + ON_3fVector(Offset);
				Nodes.Add(FNode(p1.x * ScalingFactor, p1.y * ScalingFactor, p1.z * ScalingFactor));
			}

			int faceCount = Mesh->FaceCount();
			TArray<FVector2D> UvCoords;
			TArray<FFace> Faces;
			Faces.Reserve(faceCount);

			for (int Index = 0; Index < faceCount; ++Index)
			{
				const ON_MeshFace& meshFace = Mesh->m_F[Index];

				FNode& n1 = Nodes[meshFace.vi[0]];
				FNode& n2 = Nodes[meshFace.vi[1]];
				FNode& n3 = Nodes[meshFace.vi[2]];
				Faces.Push(FFace(n1, n2, n3));

				if (Mesh->HasFaceNormals())
				{
					const ON_3fVector& NormalN1 = Mesh->m_FN[Index];

					n1.SetNormal(NormalN1);
					n2.SetNormal(NormalN1);
					n3.SetNormal(NormalN1);

					bHasNormal = true;
				}
				else if (Mesh->HasVertexNormals())
				{
					const ON_3fVector& NormalN1 = Mesh->m_N[meshFace.vi[0]];
					const ON_3fVector& NormalN2 = Mesh->m_N[meshFace.vi[1]];
					const ON_3fVector& NormalN3 = Mesh->m_N[meshFace.vi[2]];

					n1.SetNormal(NormalN1);
					n2.SetNormal(NormalN2);
					n3.SetNormal(NormalN3);

					bHasNormal = true;
				}

				if (bHasUVData)
				{
					UvCoords.Add(GetMeshTexCoords(Mesh, VertexCount, meshFace.vi[0], bHasPackedTexCoords));
					UvCoords.Add(GetMeshTexCoords(Mesh, VertexCount, meshFace.vi[1], bHasPackedTexCoords));
					UvCoords.Add(GetMeshTexCoords(Mesh, VertexCount, meshFace.vi[2], bHasPackedTexCoords));
				}

				if (meshFace.IsQuad())
				{
					FNode& n4 = Nodes[meshFace.vi[3]];
					Faces.Push(FFace(n1, n3, n4));

					if (Mesh->HasFaceNormals())
					{
						const ON_3fVector& NormalN4 = Mesh->m_FN[Index];
						n4.SetNormal(NormalN4);
					}
					else if (Mesh->HasVertexNormals())
					{
						const ON_3fVector& NormalN4 = Mesh->m_N[meshFace.vi[3]];
						n4.SetNormal(NormalN4);
					}

					if (bHasUVData)
					{
						UvCoords.Add(GetMeshTexCoords(Mesh, VertexCount, meshFace.vi[0], bHasPackedTexCoords));
						UvCoords.Add(GetMeshTexCoords(Mesh, VertexCount, meshFace.vi[2], bHasPackedTexCoords));
						UvCoords.Add(GetMeshTexCoords(Mesh, VertexCount, meshFace.vi[3], bHasPackedTexCoords));
					}
				}
			}

			if (bHasUVData)
			{
				for (int Index = 0; Index < Faces.Num(); ++Index)
				{
					FFace& Face = Faces[Index];
					for (int iTriangleNode = 0; iTriangleNode < 3; ++iTriangleNode)
					{
						Face.UVCoords[iTriangleNode] = FVector2f(UvCoords[iTriangleNode + 3 * Index][0], UvCoords[iTriangleNode + 3 * Index][1]);
					}
				}
			}

			// Fill out the MeshDescription with the processed data from the ON_Mesh
			// Ref. FDatasmithMeshUtils::ToMeshDescription

			// Reserve space for attributes
			// At this point, all the faces are triangles
			const int32 TriangleCount = Faces.Num();
			const int32 VertexInstanceCount = 3 * TriangleCount;

			int32 VertexIndexBase = MeshDescription.Vertices().Num();

			MeshDescription.ReserveNewVertices(VertexCount);

			// Do not reserve vertex instances -  UUVGenerationFlattenMapping::GetOverlappingCornersRemapping is crashing because it assumes that vertex instances has no gaps
			// but they appear when we skip triangles(degenerates) between meshes(if there are few meshes combine mesh description)
			// MeshDescription.ReserveNewVertexInstances(VertexInstanceCount);
			MeshDescription.ReserveNewEdges(VertexInstanceCount);
			MeshDescription.ReserveNewPolygons(TriangleCount);

			if (bHasFaceMaterialChannel)
			{
				checkSlow(FaceMaterialChannel);
				PolyGroupId = MeshDescription.CreatePolygonGroup();
				PolygonGroupImportedMaterialSlotNames[PolyGroupId] = DatasmithMeshHelper::DefaultSlotName(FaceMaterialChannel[MeshIndex]);
			}

			// Set vertex positions
			TMap<const FNode*, int> NodeToIndex;
			for (int Index = 0; Index < VertexCount; ++Index)
			{
				const FNode& Node = Nodes[Index];
				const FVector3f& Pos = Node.Vertex;
				NodeToIndex.Add(&Node, VertexIndexBase + Index);

				// Fill the vertex array
				FVertexID AddedVertexId = MeshDescription.CreateVertex();
				VertexPositions[AddedVertexId] = (FVector3f)FDatasmithUtils::ConvertVector(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy, Pos);
			}

			int32 VertexIndices[3];

			const int32 CornerCount = 3; // only triangles
			FVector CornerPositions[3];
			TArray<FVertexInstanceID> CornerVertexInstanceIDs;
			CornerVertexInstanceIDs.SetNum(3);
			FVertexID CornerVertexIDs[3];

			// Get per-triangle data: indices, normals and uvs
			//int WedgeIndex = 0;
			for (int FaceIndex = 0; FaceIndex < TriangleCount; ++FaceIndex)
			{
				const FFace& Face = Faces[FaceIndex];
				for (int32 CornerIndex = 0; CornerIndex < CornerCount; ++CornerIndex)
				{
					const FNode& FaceNode = *Face.Nodes[CornerIndex];
					VertexIndices[CornerIndex] = NodeToIndex[&FaceNode];

					CornerVertexIDs[CornerIndex] = FVertexID(VertexIndices[CornerIndex]);
					CornerPositions[CornerIndex] = (FVector)VertexPositions[CornerVertexIDs[CornerIndex]];
				}

				// Skip degenerated polygons
				FVector RawNormal = ((CornerPositions[1] - CornerPositions[2]) ^ (CornerPositions[0] - CornerPositions[2]));
				if (RawNormal.SizeSquared() < SMALL_NUMBER)
				{
					continue; // this will leave holes...
				}

				// Create Vertex instances and set their attributes
				for (int32 CornerIndex = 0; CornerIndex < CornerCount; ++CornerIndex)
				{
					CornerVertexInstanceIDs[CornerIndex] = MeshDescription.CreateVertexInstance(CornerVertexIDs[CornerIndex]);

					const FNode& FaceNode = *Face.Nodes[CornerIndex];

					// Set the normal
					FVector3f UENormal = FDatasmithUtils::ConvertVector(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy, FaceNode.Normal);
					UENormal = UENormal.GetSafeNormal();

					// Check to see if normal is correct. If not replace by face's normal
					if (UENormal.IsNormalized())
					{
						VertexInstanceNormals[CornerVertexInstanceIDs[CornerIndex]] = (FVector3f)UENormal;
					}
					else
					{
						// TODO: Check if this case could happen in Rhino
					}

					// Set the UV
					if (bHasUVData)
					{
						const FVector2f& UVValues = Face.UVCoords[CornerIndex];
						if (!UVValues.ContainsNaN())
						{
							// Convert UVs - bottom-left Rhino's texture origin to Unreal's top-left
							VertexInstanceUVs.Set(CornerVertexInstanceIDs[CornerIndex], 0, FVector2f(UVValues[0], 1.f - UVValues[1]));
						}
					}
				}

				const FPolygonID NewPolygonID = MeshDescription.CreatePolygon(PolyGroupId, CornerVertexInstanceIDs);
			}
		}

		// Build edge meta data
		FStaticMeshOperations::DetermineEdgeHardnessesFromVertexInstanceNormals(MeshDescription);

		return true;
	}

	bool TranslateMesh(const ON_Mesh* Mesh, FMeshDescription& MeshDescription, bool& bHasNormal, double ScalingFactor, const ON_3dVector& Offset)
	{
		return TranslateMesh(&Mesh, 1, MeshDescription, bHasNormal, ScalingFactor, Offset, false, nullptr);
	}

	void PropagateLayers(const TSharedPtr<IDatasmithActorElement>& ActorElement, const FString& LayerNames)
	{
		ActorElement->SetLayer(*LayerNames);

		int32 NumChildren = ActorElement->GetChildrenCount();
		for (int32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedPtr<IDatasmithActorElement> Child = ActorElement->GetChild(Index);
			PropagateLayers(Child, LayerNames);
		}
	}

	FMD5Hash ComputeMaterialHash(const ON_Material& Material)
	{
		FMD5 MD5;

		// Hash the material properties that are used to create the Unreal material: diffuse color, transparency, shininess and texture maps
		unsigned int ColorRef = (unsigned int)Material.Diffuse();
		MD5.Update(reinterpret_cast<const uint8*>(&ColorRef), sizeof(unsigned int));

		double Transparency = Material.Transparency();
		MD5.Update(reinterpret_cast<const uint8*>(&Transparency), sizeof(double));

		double Shininess = Material.Shine();
		MD5.Update(reinterpret_cast<const uint8*>(&Shininess), sizeof(double));

		double Reflectivity = Material.Reflectivity();
		MD5.Update(reinterpret_cast<const uint8*>(&Reflectivity), sizeof(double));

		for (int Index = 0; Index < Material.m_textures.Count(); ++Index)
		{
			const ON_Texture* Texture = Material.m_textures.At(Index);

			if (!Texture->m_bOn || (Texture->m_type != ON_Texture::TYPE::bitmap_texture && Texture->m_type != ON_Texture::TYPE::bump_texture && Texture->m_type != ON_Texture::TYPE::transparency_texture))
			{
				continue;
			}

			ON_wString FullPath = Texture->m_image_file_reference.FullPath();
			FString FilePath;
			if (FullPath.IsNotEmpty())
			{
				FilePath = FullPath.Array();
			}
			else
			{
				ON_wString RelativePath = Texture->m_image_file_reference.RelativePath();
				if (RelativePath.IsEmpty())
				{
					continue;
				}
				FilePath = RelativePath.Array();
			}

			// Hash the parameters for texture maps
			MD5.Update(reinterpret_cast<const uint8*>(*FilePath), FilePath.Len() * sizeof(TCHAR));
			MD5.Update(reinterpret_cast<const uint8*>(&Texture->m_type), sizeof(Texture->m_type));
			MD5.Update(reinterpret_cast<const uint8*>(&Texture->m_mapping_channel_id), sizeof(Texture->m_mapping_channel_id));
			MD5.Update(reinterpret_cast<const uint8*>(&Texture->m_blend_constant_A), sizeof(Texture->m_blend_constant_A));
			MD5.Update(reinterpret_cast<const uint8*>(Texture->m_uvw.m_xform), 16 * sizeof(double));
		}

		FMD5Hash Hash;
		Hash.Set(MD5);

		return Hash;
	}
}

// Hash function to use ON_UUID in TMap
inline uint32 GetTypeHash(const ON_UUID& UUID)
{
	// Data4 is 8 bytes so split it into 2 unsigned int
	unsigned int Data4_0 = *reinterpret_cast<const unsigned int*>(UUID.Data4);
	unsigned int Data4_3 = *reinterpret_cast<const unsigned int*>(UUID.Data4 + 3);
	return UUID.Data1 ^ UUID.Data2 ^ UUID.Data3 ^ Data4_0 ^ Data4_3;
}

bool operator<(const ON_UUID& uuid1, const ON_UUID& uuid2)
{
	return ON_UuidCompare(uuid1, uuid2) == -1;
}

// Hash function to use FMD5Hash in TMap
inline uint32 GetTypeHash(const FMD5Hash& Hash)
{
	uint32* HashAsInt32 = (uint32*)Hash.GetBytes();
	return HashAsInt32[0] ^ HashAsInt32[1] ^ HashAsInt32[2] ^ HashAsInt32[3];
}

class FOpenNurbsTranslatorImpl
{
public:
	FOpenNurbsTranslatorImpl(const FString& InSceneName, TSharedRef<IDatasmithScene> InScene, const FString& InCurrentPath, TSharedPtr<FTranslationCache> InTranslationCache)
		: Scene(InScene)
		, TranslationCache(InTranslationCache)
		, SceneName(InSceneName)
		, CurrentPath(InCurrentPath)
		, OpenNurbsOptionsHash(0)
		, FileVersion(0)
		, ArchiveOpenNurbsVersion(0)
		, FileLength(0)
		, NumCRCErrors(0)
	{
		if (!TranslationCache.IsValid())
		{
			TranslationCache = MakeShared<FTranslationCache>();
		}
	}

	~FOpenNurbsTranslatorImpl()
	{
		for (FOpenNurbsTranslatorImpl* ChildTranslator : ChildTranslators)
		{
			delete ChildTranslator;
		}
	}

	bool Read(ON_BinaryFile& Archive, TSharedRef<IDatasmithScene> OutScene);

	TOptional<FMeshDescription> GetMeshDescription(TSharedRef<IDatasmithMeshElement> MeshElement);

	void SetBaseOptions(const FDatasmithImportBaseOptions& InBaseOptions);
	void SetOpenNurbsOptions(const FDatasmithOpenNurbsOptions& Options);
	void SetOutputPath(const FString& Path) { OutputPath = Path; }

	void ShowMessageLog(const FString& Filename);

	bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload, const FDatasmithTessellationOptions& InTessellationOptions);

private:
	void TranslateTextureMappingTable(const ON_ObjectArray<ON_TextureMapping>& TextureMappingTable);
	void TranslateMaterialTable(const ON_ObjectArray<ON_Material>& MaterialTable);
	void TranslateLayerTable(const ON_ObjectArray<ON_Layer>& LayerTable);
	void TranslateGroupTable(const ON_ObjectArray<ON_Group>& GroupTable);
	void TranslateLightTable(const ON_ClassArray<FOpenNurbsObjectWrapper>& LightTable);
	void TranslateInstanceDefinitionTable(const TArray<ON_InstanceDefinition*>& InDefinitionTable);
	void TranslateObjectTable(const ON_ClassArray<FOpenNurbsObjectWrapper>& ObjectTable);
	void TranslateNonInstanceObject(const FOpenNurbsObjectWrapper& Object);
	bool TranslateInstance(const FOpenNurbsObjectWrapper& Object);

	bool IsValidObject(const FOpenNurbsObjectWrapper& Object);
	const ON_UUID& GetInstanceForObject(const ON_UUID& objectUUID);
	bool HasUnprocessedChildren(const ON_UUID& instanceDefUuid);

	TSharedPtr<IDatasmithActorElement> GetActorElement(const FOpenNurbsObjectWrapper& Object);
	TSharedPtr<IDatasmithMeshActorElement> GetMeshActorElement(const FOpenNurbsObjectWrapper& Object);
	TSharedPtr<IDatasmithActorElement> GetPointActorElement(const FOpenNurbsObjectWrapper& Object);
	TSharedPtr<IDatasmithMeshElement> GetMeshElement(const FOpenNurbsObjectWrapper& Object, const FString& Uuid, const FString& Label);
	TSharedPtr<IDatasmithMetaDataElement> GetObjectMetaData(const FOpenNurbsObjectWrapper& Object, const TSharedPtr<IDatasmithElement>& DatasmithElement);

	struct FMaterial
	{
		TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement;
		const ON_Material* OpenNurbsMaterialPtr = nullptr;
	};
	FMaterial GetObjectMaterial(const FOpenNurbsObjectWrapper& Object);
	ON_Material* GetOpenNurbsMaterial(int MaterialIndex);
	TSharedPtr<IDatasmithBaseMaterialElement> GetMaterial(int MaterialIndex);
	TSharedPtr<IDatasmithBaseMaterialElement> GetDefaultMaterial();

	TSharedPtr<IDatasmithActorElement> GetParentElement(const FOpenNurbsObjectWrapper& Object);
	FString GetLayerName(const TSharedPtr<IDatasmithActorElement>& LayerElement);
	void SetLayers(const TSharedPtr<IDatasmithActorElement>& ActorElement, const FOpenNurbsObjectWrapper& Object);
	void SetTags(const TSharedPtr<IDatasmithActorElement>& ActorElement, const FOpenNurbsObjectWrapper& Object);

	void SetMaterialToMeshElement(TSharedPtr<IDatasmithMeshElement> MeshElement, TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement, int32 SlotId);

	bool TranslateBRep(ON_Brep* brep, const ON_3dmObjectAttributes& Attributes, FMeshDescription& OutMesh, TSharedRef< IDatasmithMeshElement >& MeshElement, const FString& Name, bool& bHasNormal);


	bool ComputeObjectGeometryCenter(const FOpenNurbsObjectWrapper& Object, ON_3dVector& OutGeometryCenter);
	bool ComputeGeometryCenter(ON_Geometry* Geometry, ON_3dVector& OutCenter);

	// Returns vector to shift geometry to its bounding box center
	// done before tessellating or creating MeshDescription
	ON_3dVector GetGeometryOffset(TSharedRef<IDatasmithMeshElement> MeshElement)
	{
		ON_3dVector Offset = ON_3fVector::ZeroVector;
		if (ON_3dVector* OffsetPtr = MeshElementToGeometryCenter.Find(MeshElement))
		{
			Offset = *OffsetPtr;
		}
		return -Offset;
	}

	double GetScaleFactor() const
	{
		return ScaleFactor;
	}

	/**
	 * @param NewMetricUnit compare to mm
	 */
	void SetMetricUnit(const double NewMetricUnit)
	{
		if (FMath::IsNearlyZero(NewMetricUnit))
		{
			return;
		}

		// default unit is mm
		ScaleFactor = 0.1 / NewMetricUnit;
		OpenNurbsBRepConverter->SetScaleFactor(1 / NewMetricUnit);
	}

private:
	TArray<FOpenNurbsTranslatorImpl*> ChildTranslators;
	TSharedRef<IDatasmithScene> Scene;
	TSharedPtr<FTranslationCache> TranslationCache;
	FString SceneName;
	FString CurrentPath;
	FString OutputPath;
	FDatasmithOpenNurbsOptions OpenNurbsOptions;
	uint32 OpenNurbsOptionsHash;
	FDatasmithImportBaseOptions BaseOptions;

	// default unit is mm
	double ScaleFactor = 0.1;

private:
	// For OpenNurbs archive parsing
	// Ref. GP3DMReader

	// start section information
	int FileVersion;
	int ArchiveOpenNurbsVersion;
	ON_String StartSectionComments;

	// Properties include revision history, notes, information about
	// the applicaton that created the file, and an option preview image.
	ON_3dmProperties Properties;

	// Settings include tolerance, and unit system, and defaults used
	// for creating views and objects.
	ON_3dmSettings Settings;

	// Tables in an openNURBS archive
	ON_ObjectArray<ON_TextureMapping> TextureMappingTable;
	ON_ObjectArray<ON_Material> MaterialTable;
	ON_ObjectArray<ON_Layer> LayerTable;
	ON_ObjectArray<ON_Group> GroupTable;
	ON_ClassArray<FOpenNurbsObjectWrapper> LightTable;
	TArray<ON_InstanceDefinition*> InstanceDefinitionTable;
	ON_ClassArray<FOpenNurbsObjectWrapper> ObjectTable;
	ON_ClassArray<ONX_Model_UserData> UserDataTable;

	// length of archive returned by ON_BinaryArchive::Read3dmEndMark()
	size_t FileLength;

	// Number of crc errors found during archive reading.
	// If > 0, then the archive is corrupt.
	int NumCRCErrors;

	/////////////////////////////////////////////////////////////////////

	// For translated data
	TMap<ON_UUID, const ON_TextureMapping*> UUIDToTextureMapping;

	// Materials
	TMap<FMD5Hash, TSharedPtr<IDatasmithBaseMaterialElement>> HashToMaterial;
	TMap<int, TSharedPtr<IDatasmithBaseMaterialElement>> MaterialIndexToMaterial;
	TMap<int, int> MaterialIndexToMaterialTableIndex;
	TMap<ON_UUID, int> MaterialIdToIndex;
	TSet<TSharedPtr<IDatasmithBaseMaterialElement>> UsedMaterials;
	TSharedPtr<IDatasmithBaseMaterialElement> DefaultMaterial;

	// Layers
	TMap<ON_UUID, TSharedPtr<IDatasmithActorElement>> LayerUUIDToContainer;
	TMap<int, TSharedPtr<IDatasmithActorElement>> LayerIndexToContainer;
	TMap<int, int> LayerIndexToMaterialIndex;
	TMap<TSharedPtr<IDatasmithActorElement>, FString> LayerNames;
	TSet<int> HiddenLayersIndices;

	// Groups
	TArray<FString> GroupNames;

	// Instance definitions
	std::set<ON_UUID> processedUUIDs; // Used to update list of children of instance definition
	std::map<ON_UUID, TSharedPtr<IDatasmithActorElement>> uuidToInstanceContainer;
	std::map<ON_UUID, int> uuidToInstanceChildrenCount;
	std::map<ON_UUID, ON_UUID> objectUUIDToInstanceUUID;
	TMap<TSharedPtr<IDatasmithActorElement>, TSharedPtr<IDatasmithMetaDataElement>> ActorElementToMetaDataMap;
	TMap<IDatasmithMeshElement*, FOpenNurbsTranslatorImpl*> MeshElementToTranslatorMap;

	//Objects
	/** Datasmith mesh elements to OpenNurbs objects */
	TMap< IDatasmithMeshElement*, const FOpenNurbsObjectWrapper* > MeshElementToObjectMap;

	/** OpenNurbs objects to Datasmith mesh elements */
	TMap<const FOpenNurbsObjectWrapper*, TSharedPtr< IDatasmithMeshElement > > ObjectToMeshElementMap;

	TMap< TSharedPtr< IDatasmithMeshElement >, ON_3dVector > MeshElementToGeometryCenter;

	TArray<FString> MissingRenderMeshes;

	TSharedPtr<CADLibrary::ICADModelConverter> CADModelConverter;
	TSharedPtr<IOpenNurbsBRepConverter> OpenNurbsBRepConverter;

};

void FOpenNurbsTranslatorImpl::ShowMessageLog(const FString& Filename)
{
#if WITH_EDITOR
	if (MissingRenderMeshes.Num())
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		TSharedRef<IMessageLogListing> LogListing = (MessageLogModule.GetLogListing(FName(TEXT("DatasmithOpenNurbs"))));
		LogListing->SetLabel(LOCTEXT("DatasmithOpenNurbsTranslatorDescription", "DatasmithOpenNurbs"));

		LogListing->ClearMessages();

		LogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Warning,
			FText::Format(LOCTEXT("DatasmithOpenNurbsTranslator_NoMeshDataForAllMeshes", "Rhino model \"{0}\" doesn't contain mesh data for all objects. \nEither resave the 3dm file with a \"rendered view\" or change the import settings to \"Import as NURBS, Tessellate in Unreal\""), FText::FromString(Filename))
		));

		for (const FString& Name : MissingRenderMeshes)
		{
			FText ErrorMessage = FText::Format(LOCTEXT("DatasmithOpenNurbsTranslator_NoMesh", "  {0} doesn't have mesh information."), FText::FromString(Name));
			LogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error, ErrorMessage));
		}
	}
#endif
}

void FOpenNurbsTranslatorImpl::TranslateTextureMappingTable(const ON_ObjectArray<ON_TextureMapping>& InTextureMappingTable)
{
	for (int Index = 0; Index < InTextureMappingTable.Count(); ++Index)
	{
		const ON_TextureMapping& TextureMapping = *InTextureMappingTable.At(Index);
		UUIDToTextureMapping.Add(TextureMapping.Id(), &TextureMapping);
	}
}

void FOpenNurbsTranslatorImpl::TranslateMaterialTable(const ON_ObjectArray<ON_Material>& InMaterialTable)
{
	// Ref. visitMaterialTable
	// These materials are from the Materials tab and do not include materials from layers
	for (int Index = 0; Index < InMaterialTable.Count(); ++Index)
	{
		const ON_Material& OpenNurbsMaterial = *InMaterialTable.At(Index);

		FMD5Hash Hash = DatasmithOpenNurbsTranslatorUtils::ComputeMaterialHash(OpenNurbsMaterial);
		TSharedPtr<IDatasmithBaseMaterialElement>* MaterialPtr = HashToMaterial.Find(Hash);

		MaterialIndexToMaterialTableIndex.Add(OpenNurbsMaterial.Index(), Index);

		MaterialIdToIndex.Add(OpenNurbsMaterial.Id(), OpenNurbsMaterial.Index());

		if (MaterialPtr)
		{
			MaterialIndexToMaterial.Add(OpenNurbsMaterial.Index(), *MaterialPtr);
			continue;
		}

		ON_Color Diffuse = OpenNurbsMaterial.Diffuse();
		int Transparency = 255 * OpenNurbsMaterial.Transparency();

		// Note that in OpenNurbs, Alpha means Transparency, whereas it is usually an Opacity.
		// Hence the (255 - Transparency) where an opacity is expected
		FColor Color((uint8)Diffuse.Red(), (uint8)Diffuse.Green(), (uint8)Diffuse.Blue(), (uint8)(255 - Transparency));
		FLinearColor LinearColor = FLinearColor::FromSRGBColor(Color);

		FString MaterialLabel(OpenNurbsMaterial.Name().Array());
		if (MaterialLabel.IsEmpty())
		{
			MaterialLabel = TEXT("Material");
		}

		ON_wString UUIDString;
		ON_UuidToString(OpenNurbsMaterial.Id(), UUIDString);

		FString MaterialName(UUIDString.Array());
		TSharedRef<IDatasmithUEPbrMaterialElement> Material = FDatasmithSceneFactory::CreateUEPbrMaterial(*MaterialName);
		Material->SetLabel(*MaterialLabel);

		MaterialIndexToMaterial.Add(OpenNurbsMaterial.Index(), Material);
		HashToMaterial.Add(Hash, Material);

		if (OpenNurbsMaterial.m_textures.Count() != 0) // Texture
		{
			for (int TextureIndex = 0; TextureIndex < OpenNurbsMaterial.m_textures.Count(); ++TextureIndex)
			{
				const ON_Texture* Texture = OpenNurbsMaterial.m_textures.At(TextureIndex);

				if (!Texture->m_bOn || (Texture->m_type != ON_Texture::TYPE::bitmap_texture && Texture->m_type != ON_Texture::TYPE::bump_texture && Texture->m_type != ON_Texture::TYPE::transparency_texture))
				{
					continue;
				}

				FString FileName;
				FString FilePath;

				ON_wString FullPath = Texture->m_image_file_reference.FullPath();
				if (FullPath.IsNotEmpty())
				{
					FilePath = FullPath.Array();
					FileName = FPaths::GetCleanFilename(FilePath);
					if (!FPaths::FileExists(FilePath))
					{
						FilePath.Empty();
					}
				}

				// Rhino's full path did not work, check with Rhino's relative path starting from current path
				if (FilePath.IsEmpty())
				{
					ON_wString RelativePath = Texture->m_image_file_reference.RelativePath();
					if (RelativePath.IsNotEmpty())
					{
						FilePath = FPaths::Combine(CurrentPath, RelativePath.Array());
						FilePath = FPaths::ConvertRelativePathToFull(FilePath);
						if (!FPaths::FileExists(FilePath))
						{
							FilePath.Empty();
						}
					}
				}

				// Last resort, search for the file
				if (FilePath.IsEmpty())
				{
					// Search the texture in the CurrentPath and its sub-folders
					// Note that FindFiles strips the path from its result so it cannot be used directly
					TArray<FString> FileNames;
					FString SearchPath = FPaths::Combine(CurrentPath, FileName);
					IFileManager::Get().FindFiles(FileNames, *SearchPath, true, false);
					if (FileNames.Num() > 0)
					{
						FilePath = SearchPath;
					}
					else
					{
						// Search the texture in the subfolders of CurrentPath
						TArray<FString> Folders;
						SearchPath = FPaths::Combine(CurrentPath, TEXT("*"));
						IFileManager::Get().FindFiles(Folders, *SearchPath, false, true);
						for (const FString& Folder : Folders)
						{
							SearchPath = FPaths::Combine(CurrentPath, Folder, FileName);
							IFileManager::Get().FindFiles(FileNames, *SearchPath, true, false);
							if (FileNames.Num() > 0)
							{
								FilePath = SearchPath;
								break;
							}
						}
					}
				}

				if (FilePath.IsEmpty())
				{
					continue;
				}

				FString TextureName(FPaths::GetBaseFilename(FilePath));

				if (Texture->m_type == ON_Texture::TYPE::bump_texture)
				{
					TextureName += TEXT("_normal");
				}
				else if (Texture->m_type == ON_Texture::TYPE::transparency_texture)
				{
					TextureName += TEXT("_alpha");
				}

				TSharedRef<IDatasmithTextureElement> TextureElement = FDatasmithSceneFactory::CreateTexture(*TextureName);
				TextureElement->SetTextureFilter(EDatasmithTextureFilter::Default);
				TextureElement->SetRGBCurve(1.0);
				TextureElement->SetTextureAddressX(Texture->m_wrapu == ON_Texture::WRAP::clamp_wrap ? EDatasmithTextureAddress::Clamp : EDatasmithTextureAddress::Wrap);
				TextureElement->SetTextureAddressY(Texture->m_wrapv == ON_Texture::WRAP::clamp_wrap ? EDatasmithTextureAddress::Clamp : EDatasmithTextureAddress::Wrap);
				TextureElement->SetFile(*FilePath);

				Scene->AddTexture(TextureElement);

				if (Texture->m_type == ON_Texture::TYPE::bitmap_texture)
				{
					TextureElement->SetTextureMode(EDatasmithTextureMode::Diffuse);
				}
				else if (Texture->m_type == ON_Texture::TYPE::bump_texture)
				{
					TextureElement->SetTextureMode(EDatasmithTextureMode::Bump);
				}
				else if (Texture->m_type == ON_Texture::TYPE::transparency_texture)
				{
					TextureElement->SetTextureMode(EDatasmithTextureMode::Diffuse);
				}

				// Extract texture mapping info
				DatasmithMaterialsUtils::FUVEditParameters UVParameters;

				// Use cached texture coordinates(channel 0)
				UVParameters.ChannelIndex = 0;
				// Note - Texture->m_mapping_channel_id may used to get TextureMapping in order to generate texture coordinates from it.
				// Specifically, from object's attributes get MappingRef - attributes.m_rendering_attributes.m_mappings
				// Next, find channel within MappingRef's m_mapping_channels which m_mapping_channel_id is equal to Texture's
				// Then m_mapping_id of that channel will be UUID in the UUIDToTextureMapping

				// Extract the UV tiling, offset and rotation angle from the UV transform matrix
				FMatrix Matrix;
				DatasmithOpenNurbsTranslatorUtils::XFormToMatrix(Texture->m_uvw, Matrix);

				FTransform Transform(Matrix);

				// Note that the offset from m_uvw has the rotation applied to it
				FVector Translation = Transform.GetTranslation();
				FVector3f Tiling = (FVector3f)Transform.GetScale3D();	// LWC_TODO: Precision Loss
				FVector RotationAngles = Transform.GetRotation().Euler();

				UVParameters.UVTiling.X = Tiling.X;

				UVParameters.UVTiling.Y = Tiling.Y;

				if (!FMath::IsNearlyZero(Tiling.X, KINDA_SMALL_NUMBER)
					&& !FMath::IsNearlyZero(Tiling.Y, KINDA_SMALL_NUMBER))
				{
					UVParameters.UVOffset.X = Translation.X / Tiling.X;

					// Recomputing offset from Rhino to Unreal, taking into account how tiling Pivot is set in SetupUVEdit:
					// P = pivot, C - input uv, T - tiling, O - offset
					// Texture coordinate transformed like this: UV = T*(C+O-P)+P (ignoring mirror/rotation)
					// P is 0.5 (for V), so  UV = T*(C+O - 0.5) + 0.5 = T*C + T*O - T*0.5 + 0.5
					// Also, for Rhino we P` = 1 (Rhino image origin bottom-left),
					// so tiling in Rhino is defined as  UV = T`*(C` + O` - 1) + 1
					// T`*C + T`*O` - T`*1 + 1 = T*C + T*O - T*0.5 + 0.5
					// T` = T, reducing further and get:
					// O` = O + 0.5 - 0.5 / T
					// #ueent_todo: this could be simplified if we were able to pass custom Tiling Pivot
					UVParameters.UVOffset.Y = -(Translation.Y / Tiling.Y + 0.5f - 0.5f / Tiling.Y); // V-coordinate is inverted in Unreal
				}

				// Rotation angle is reversed because V-axis points down in Unreal while it points up in OpenNurbs
				UVParameters.RotationAngle = -RotationAngles.Z;

				float Weight = Texture->m_blend_constant_A;

				switch (Texture->m_type)
				{
				case ON_Texture::TYPE::bitmap_texture:
				{
					IDatasmithMaterialExpression* TextureExpression = DatasmithMaterialsUtils::CreateTextureExpression(Material, TEXT("Diffuse Map"), TextureElement->GetName(), UVParameters);
					IDatasmithMaterialExpression* Expression = DatasmithMaterialsUtils::CreateWeightedMaterialExpression(Material, TEXT("Diffuse Color"), LinearColor, TOptional<float>(), TextureExpression, Weight);
					Material->GetBaseColor().SetExpression(Expression);
				}
				break;
				case ON_Texture::TYPE::bump_texture:
				{
					IDatasmithMaterialExpression* TextureExpression = DatasmithMaterialsUtils::CreateTextureExpression(Material, TEXT("Bump Map"), TextureElement->GetName(), UVParameters);
					IDatasmithMaterialExpression* Expression = DatasmithMaterialsUtils::CreateWeightedMaterialExpression(Material, TEXT("Bump Height"), TOptional<FLinearColor>(), TOptional<float>(), TextureExpression, Weight, EDatasmithTextureMode::Bump);
					Material->GetNormal().SetExpression(Expression);
				}
				break;
				case ON_Texture::TYPE::transparency_texture:
				{
					IDatasmithMaterialExpression* TextureExpression = DatasmithMaterialsUtils::CreateTextureExpression(Material, TEXT("Opacity Map"), TextureElement->GetName(), UVParameters);
					IDatasmithMaterialExpression* Expression = DatasmithMaterialsUtils::CreateWeightedMaterialExpression(Material, TEXT("White"), FLinearColor::White, TOptional<float>(), TextureExpression, Weight);
					Material->GetOpacity().SetExpression(Expression);
					if (!FMath::IsNearlyZero(Weight, KINDA_SMALL_NUMBER))
					{
						Material->SetBlendMode(/*EBlendMode::BLEND_Translucent*/2);
					}
				}
				break;
				}
			}
		}

		// Set a diffuse color if there's nothing in the BaseColor
		if (Material->GetBaseColor().GetExpression() == nullptr)
		{
			IDatasmithMaterialExpressionColor* ColorExpression = Material->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
			ColorExpression->SetName(TEXT("Diffuse Color"));
			ColorExpression->GetColor() = LinearColor;

			Material->GetBaseColor().SetExpression(ColorExpression);
		}

		// Setup the blend mode for transparent material
		if (LinearColor.A < 1.0f)
		{
			Material->SetBlendMode(/*EBlendMode::BLEND_Translucent*/2);
			if (Material->GetOpacity().GetExpression() == nullptr)
			{
				// Transparent color
				IDatasmithMaterialExpressionScalar* Scalar = static_cast<IDatasmithMaterialExpressionScalar*>(Material->AddMaterialExpression(EDatasmithMaterialExpressionType::ConstantScalar));
				Scalar->SetName(TEXT("Opacity"));
				Scalar->GetScalar() = LinearColor.A;

				Material->GetOpacity().SetExpression(Scalar);
			}
			else
			{
				// Modulate the opacity map with the color transparency setting
				IDatasmithMaterialExpressionGeneric* Multiply = static_cast<IDatasmithMaterialExpressionGeneric*>(Material->AddMaterialExpression(EDatasmithMaterialExpressionType::Generic));
				Multiply->SetExpressionName(TEXT("Multiply"));

				IDatasmithMaterialExpressionScalar* Scalar = static_cast<IDatasmithMaterialExpressionScalar*>(Material->AddMaterialExpression(EDatasmithMaterialExpressionType::ConstantScalar));
				Scalar->SetName(TEXT("Opacity Output Level"));
				Scalar->GetScalar() = LinearColor.A;
				Scalar->ConnectExpression(*Multiply->GetInput(0));

				IDatasmithMaterialExpression* CurrentOpacityExpression = Material->GetOpacity().GetExpression();
				CurrentOpacityExpression->ConnectExpression(*Multiply->GetInput(1));

				Material->GetOpacity().SetExpression(Multiply);
			}
		}

		// Simple conversion of shininess and reflectivity to PBR roughness and metallic values; model could be improved to properly blend the values
		float Shininess = OpenNurbsMaterial.Shine() / ON_Material::MaxShine;
		if (!FMath::IsNearlyZero(Shininess))
		{
			IDatasmithMaterialExpressionScalar* Scalar = static_cast<IDatasmithMaterialExpressionScalar*>(Material->AddMaterialExpression(EDatasmithMaterialExpressionType::ConstantScalar));
			Scalar->SetName(TEXT("Roughness"));
			Scalar->GetScalar() = 1.f - Shininess;
			Material->GetRoughness().SetExpression(Scalar);
		}

		float Reflectivity = OpenNurbsMaterial.Reflectivity();
		if (!FMath::IsNearlyZero(Reflectivity))
		{
			IDatasmithMaterialExpressionScalar* Scalar = static_cast<IDatasmithMaterialExpressionScalar*>(Material->AddMaterialExpression(EDatasmithMaterialExpressionType::ConstantScalar));
			Scalar->SetName(TEXT("Metallic"));
			Scalar->GetScalar() = Reflectivity;
			Material->GetMetallic().SetExpression(Scalar);
		}
	}
}

void FOpenNurbsTranslatorImpl::TranslateLayerTable(const ON_ObjectArray<ON_Layer>& InLayerTable)
{
	// Ref. visitLayerTable
	TSet<ON_UUID> HiddenLayersUUIDs;
	for (int Index = 0; Index < InLayerTable.Count(); ++Index)
	{
		const ON_Layer& CurrentLayer = *InLayerTable.At(Index);

		TSharedPtr<IDatasmithActorElement> Parent;
		if (ON_UuidCompare(ON_nil_uuid, CurrentLayer.ParentLayerId()) != 0)
		{
			for (auto It : LayerUUIDToContainer)
			{
				if (ON_UuidCompare(CurrentLayer.ParentLayerId(), It.Key) == 0)
				{
					Parent = It.Value;
					break;
				}
			}
		}

		// Create new layer
		ON_wString UUIDString;
		ON_UuidToString(CurrentLayer.Id(), UUIDString);

		FString LayerName = UUIDString.Array();
		FString LayerLabel(CurrentLayer.Name().Length() > 0 ? CurrentLayer.Name().Array() : FString::Printf(TEXT("Layer%d"), Index));

		TSharedPtr<IDatasmithActorElement> LayerElement = FDatasmithSceneFactory::CreateActor(*LayerName);
		LayerElement->SetLabel(*LayerLabel);

		FString FullLayerName(LayerElement->GetLabel());
		if (Parent.IsValid())
		{
			Parent->AddChild(LayerElement);
			FString ParentLayerName = GetLayerName(Parent);
			FullLayerName = ParentLayerName + TEXT(".") + FullLayerName;
		}
		else
		{
			Scene->AddActor(LayerElement);
		}

		LayerIndexToContainer.Add(CurrentLayer.Index(), LayerElement);
		LayerUUIDToContainer.Add(CurrentLayer.Id(), LayerElement);
		LayerNames.Add(LayerElement, FullLayerName);

		// Note: even with visibility attribute set, a layer is only visible if its parents layers are visible
		if (!CurrentLayer.IsVisible() || (CurrentLayer.ParentIdIsNotNil() && HiddenLayersUUIDs.Contains(CurrentLayer.ParentLayerId())))
		{
			HiddenLayersIndices.Add(CurrentLayer.Index());
			HiddenLayersUUIDs.Add(CurrentLayer.Id());
		}

		// Use the layer's render material, no fallback on the display color
		if (CurrentLayer.RenderMaterialIndex() != -1)
		{
			LayerIndexToMaterialIndex.Add(CurrentLayer.Index(), CurrentLayer.RenderMaterialIndex());
		}
	}
}

void FOpenNurbsTranslatorImpl::TranslateGroupTable(const ON_ObjectArray<ON_Group>& InGroupTable)
{
	for (int Index = 0; Index < InGroupTable.Count(); ++Index)
	{
		const ON_Group& Group = *InGroupTable.At(Index);
		GroupNames.Add(Group.Name().IsEmpty() ? FString::Printf(TEXT("Group%d"), Index) : Group.Name().Array());
	}
}

void FOpenNurbsTranslatorImpl::TranslateLightTable(const ON_ClassArray<FOpenNurbsObjectWrapper>& InLightTable)
{
	// Ref. visitRenderLightsTable
	// Ref. FGPureSceneParser::ProcessLight
	for (int Index = 0; Index < InLightTable.Count(); ++Index)
	{
		const FOpenNurbsObjectWrapper& Object = *InLightTable.At(Index);
		if (!IsValidObject(Object))
		{
			continue;
		}

		const ON_Light& LightObj = *static_cast<ON_Light*>(Object.ObjectPtr);

		ON::light_style LightStyle = LightObj.Style();

		EDatasmithElementType LightType = EDatasmithElementType::PointLight;
		switch (LightStyle)
		{
		case ON::light_style::camera_directional_light:
		case ON::light_style::world_directional_light:
			LightType = EDatasmithElementType::DirectionalLight;
			break;
		case ON::light_style::camera_point_light:
		case ON::light_style::world_point_light:
			LightType = EDatasmithElementType::PointLight;
			break;
		case ON::light_style::camera_spot_light:
		case ON::light_style::world_spot_light:
			LightType = EDatasmithElementType::SpotLight;
			break;
		case ON::light_style::ambient_light: // not supported as light
			continue;
		case ON::light_style::world_linear_light:
		case ON::light_style::world_rectangular_light:
			LightType = EDatasmithElementType::AreaLight;
			break;
		}

		ON_wString UUIDString;
		ON_UuidToString(LightObj.ModelObjectId(), UUIDString);

		FString LightName(UUIDString.Array());
		TSharedPtr<IDatasmithElement> Element = FDatasmithSceneFactory::CreateElement(LightType, *LightName);

		if (!Element.IsValid() || !Element->IsA(EDatasmithElementType::Light))
		{
			continue;
		}

		TSharedPtr<IDatasmithLightActorElement> LightElement = StaticCastSharedPtr<IDatasmithLightActorElement>(Element);

		FString LightLabel(LightObj.LightName().Array());
		if (LightLabel.IsEmpty())
		{
			LightLabel = TEXT("Light");
		}

		LightElement->SetLabel(*LightLabel);
		LightElement->SetUseIes(false);
		LightElement->SetUseTemperature(false);
		LightElement->SetEnabled(LightObj.m_bOn);

		// Diffuse color (Ambient and Specular color not supported and alpha from diffuse is ignored)
		FColor Color((uint8)LightObj.Diffuse().Red(), (uint8)LightObj.Diffuse().Green(), (uint8)LightObj.Diffuse().Blue(), 255);
		LightElement->SetColor(Color.ReinterpretAsLinear());

		// Intensity (PowerWatts and ShadowIntensity not used)
		LightElement->SetIntensity(LightObj.Intensity() * 100.f);

		// Set light position
		if (LightType == EDatasmithElementType::PointLight ||
			LightType == EDatasmithElementType::DirectionalLight ||
			LightType == EDatasmithElementType::SpotLight)
		{
			FVector Location(LightObj.Location().x, LightObj.Location().y, LightObj.Location().z);
			Location *= GetScaleFactor();
			Location = FDatasmithUtils::ConvertVector(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy, Location);
			LightElement->SetTranslation(Location);
		}

		// Set light direction
		if (LightType == EDatasmithElementType::DirectionalLight ||
			LightType == EDatasmithElementType::SpotLight)
		{
			FVector Direction(LightObj.Direction().x, LightObj.Direction().y, LightObj.Direction().z);
			Direction = FDatasmithUtils::ConvertVector(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy, Direction);
			LightElement->SetRotation(FQuat::FindBetweenVectors(FVector::ForwardVector, Direction));
		}

		// AreaLight setup
		if (LightType == EDatasmithElementType::AreaLight)
		{
			TSharedRef<IDatasmithAreaLightElement> AreaLightElement = StaticCastSharedRef<IDatasmithAreaLightElement>(LightElement.ToSharedRef());

			double Length = LightObj.Length().Length() * GetScaleFactor();
			AreaLightElement->SetLength(Length);

			ON_3dVector Center = LightObj.Location() + 0.5 * LightObj.Length();
			if (LightStyle == ON::light_style::world_rectangular_light)
			{
				Center += 0.5 * LightObj.Width();
				double Width = LightObj.Width().Length() * GetScaleFactor();

				AreaLightElement->SetWidth(Width);
				AreaLightElement->SetLightShape(EDatasmithLightShape::Rectangle);
				// #ueent_todo: Determine if it should be rect instead
				AreaLightElement->SetLightType(EDatasmithAreaLightType::Spot);
			}
			else
			{
				AreaLightElement->SetWidth(0.01f * Length);
				AreaLightElement->SetLightShape(EDatasmithLightShape::Cylinder);
				AreaLightElement->SetLightType(EDatasmithAreaLightType::Point);
				// The light in Rhino doesn't have attenuation, but the attenuation radius was found by testing in Unreal to obtain a visual similar to Rhino
				AreaLightElement->SetAttenuationRadius(1800.f);
			}

			ON_3dVector LengthAxis = LightObj.Length();
			ON_3dVector WidthAxis = LightObj.Width();
			ON_3dVector InvLightAxis = ON_CrossProduct(WidthAxis, LengthAxis);
			ON_Xform Xform(Center, InvLightAxis.UnitVector(), WidthAxis.UnitVector(), LengthAxis.UnitVector());

			FMatrix Matrix;
			DatasmithOpenNurbsTranslatorUtils::XFormToMatrix(Xform, Matrix);

			FTransform Transform(Matrix);
			FTransform CorrectedTransform = FDatasmithUtils::ConvertTransform(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy, Transform);

			AreaLightElement->SetTranslation(CorrectedTransform.GetTranslation() * GetScaleFactor());
			AreaLightElement->SetScale(CorrectedTransform.GetScale3D());
			AreaLightElement->SetRotation(CorrectedTransform.GetRotation());
		}

		// Spot Light setup (SpotExponent and HotSpot not supported)
		if (LightType == EDatasmithElementType::SpotLight)
		{
			TSharedRef<IDatasmithSpotLightElement> SpotLightElement = StaticCastSharedRef<IDatasmithSpotLightElement>(LightElement.ToSharedRef());

			float InnerAngleDegree = LightObj.SpotAngleDegrees();
			SpotLightElement->SetInnerConeAngle(InnerAngleDegree);

			double InnerRadius;
			double OuterRadius;
			LightObj.GetSpotLightRadii(&InnerRadius, &OuterRadius);

			InnerRadius *= GetScaleFactor();
			OuterRadius *= GetScaleFactor();

			float OuterAngle = FMath::Atan(OuterRadius * FMath::Tan(FMath::DegreesToRadians(InnerAngleDegree)) / InnerRadius);
			SpotLightElement->SetOuterConeAngle(FMath::RadiansToDegrees(OuterAngle));
		}

		if (LightElement->IsA(EDatasmithElementType::PointLight))
		{
			TSharedRef<IDatasmithPointLightElement> PointLightElement = StaticCastSharedRef<IDatasmithPointLightElement>(LightElement.ToSharedRef());
			PointLightElement->SetIntensityUnits(EDatasmithLightUnits::Candelas);
		}

		SetLayers(LightElement, Object);

		TSharedPtr<IDatasmithActorElement> Parent = GetParentElement(Object);
		if (Parent.IsValid())
		{
			Parent->AddChild(LightElement);
		}
		else
		{
			Scene->AddActor(LightElement);
		}

		TSharedPtr<IDatasmithMetaDataElement> MetaData = GetObjectMetaData(Object, LightElement);
		if (MetaData.IsValid())
		{
			Scene->AddMetaData(MetaData);
		}
	}
}

void FOpenNurbsTranslatorImpl::TranslateInstanceDefinitionTable(const TArray<ON_InstanceDefinition*>& InDefinitionTable)
{
	// Ref. visitInstanceDefinitionTable

	static int instanceDefCount = 0;

	for (const ON_InstanceDefinition* pInstanceDef : InDefinitionTable)
	{
		++instanceDefCount;

		const ON_InstanceDefinition& instanceDef = *pInstanceDef;
		const ON_UUID& instanceDefUuid = instanceDef.Id();

		FString InstanceDefName;
		if (instanceDef.Name().Length() > 0)
		{
			InstanceDefName = instanceDef.Name().Array();
		}
		else
		{
			InstanceDefName = FString::Printf(TEXT("InstanceDef%d"), instanceDefCount);
		}

		TSharedPtr<IDatasmithActorElement> InstanceElement = FDatasmithSceneFactory::CreateActor(*InstanceDefName);

		ON_wString uuidString;
		ON_UuidToString(instanceDefUuid, uuidString);

		if (instanceDef.LinkedFileReference().IsSet())
		{
			// Find the absolute path to the referenced file
			FString FileName;

			// Check if full path provided by Rhino works
			ON_wString FullPath = instanceDef.LinkedFileReference().FullPath();
			if (FullPath.IsNotEmpty())
			{
				FileName = FullPath.Array();
				if (!FPaths::FileExists(FileName))
				{
					FileName.Empty();
				}
			}

			// Rhino's full path did not work, check with relative path from current file's path
			if (FileName.IsEmpty())
			{
				ON_wString RelativePath = instanceDef.LinkedFileReference().RelativePath();
				if (RelativePath.IsNotEmpty())
				{
					FileName = FPaths::Combine(CurrentPath, RelativePath.Array());
					FileName = FPaths::ConvertRelativePathToFull(FileName);
					if (!FPaths::FileExists(FileName))
					{
						FileName.Empty();
					}
				}
			}

			uuidToInstanceChildrenCount[instanceDefUuid] = 0;

			if (!FileName.IsEmpty())
			{
				TSharedPtr<IDatasmithActorElement> CachedElement = TranslationCache->GetElementForLinkedFileReference(FileName);
				if (CachedElement.IsValid())
				{
					uuidToInstanceChildrenCount[instanceDefUuid] = CachedElement->GetChildrenCount();
					uuidToInstanceContainer[instanceDefUuid] = CachedElement;
					continue;
				}
				else
				{

					FILE* FileHandle = ON::OpenFile(*FileName, L"rb");
					if (!FileHandle)
					{
						continue;
					}

					FString ChildSceneName = FPaths::GetBaseFilename(FileName);
					TSharedRef< IDatasmithScene > ChildScene = FDatasmithSceneFactory::CreateScene(*ChildSceneName);

					FOpenNurbsTranslatorImpl* LinkedFileTranslator = new FOpenNurbsTranslatorImpl(ChildSceneName, ChildScene, FPaths::GetPath(FileName), TranslationCache);
					if (!LinkedFileTranslator)
					{
						continue;
					}

					ChildTranslators.Add(LinkedFileTranslator);

					LinkedFileTranslator->SetOpenNurbsOptions(OpenNurbsOptions);

					ON_BinaryFile Archive(ON::archive_mode::read3dm, FileHandle);

					bool bResult = LinkedFileTranslator->Read(Archive, ChildScene);

					ON::CloseFile(FileHandle);

					if (bResult)
					{
						// Propagate data from child to parent translator for use by the "root" translator
						MeshElementToTranslatorMap.Append(LinkedFileTranslator->MeshElementToTranslatorMap);
						MeshElementToObjectMap.Append(LinkedFileTranslator->MeshElementToObjectMap);
						MeshElementToGeometryCenter.Append(LinkedFileTranslator->MeshElementToGeometryCenter);

						// Merge ChildScene with ParentScene
						int32 NumActors = ChildScene->GetActorsCount();
						for (int32 Index = 0; Index < NumActors; ++Index)
						{
							InstanceElement->AddChild(ChildScene->GetActor(Index));
						}

						for (int32 Index = 0; Index < ChildScene->GetMeshesCount(); ++Index)
						{
							Scene->AddMesh(ChildScene->GetMesh(Index));
						}

						for (int32 Index = 0; Index < ChildScene->GetMaterialsCount(); ++Index)
						{
							// Note that the child scene could include a duplicated material (as defined by the hash), but assigned through different material elements
							Scene->AddMaterial(ChildScene->GetMaterial(Index));
						}

						for (int32 Index = 0; Index < ChildScene->GetTexturesCount(); ++Index)
						{
							Scene->AddTexture(ChildScene->GetTexture(Index));
						}

						uuidToInstanceChildrenCount[instanceDefUuid] = NumActors;
						uuidToInstanceContainer[instanceDefUuid] = InstanceElement;

						TranslationCache->AddElementForLinkedFileReference(FileName, InstanceElement);
					}
				}
			}

			continue;
		}

		const ON_SimpleArray<ON_UUID>& componentsUUIDs = instanceDef.InstanceGeometryIdList();

		uuidToInstanceChildrenCount[instanceDefUuid] = componentsUUIDs.Count();
		uuidToInstanceContainer[instanceDefUuid] = InstanceElement;

		// Relate all components of instance definition to itself
		for (int i = 0; i < componentsUUIDs.Count(); ++i)
		{
			objectUUIDToInstanceUUID[componentsUUIDs[i]] = instanceDefUuid;
		}
	}
}

void FOpenNurbsTranslatorImpl::TranslateObjectTable(const ON_ClassArray<FOpenNurbsObjectWrapper>& InObjectTable)
{
	// Ref. visitObjectTable
	// Process all objects which are not instances
	std::deque<int> instanceRefs;
	int NumObjects = InObjectTable.Count();
	for (int Index = 0; Index < NumObjects; ++Index)
	{
		const FOpenNurbsObjectWrapper& Object = *InObjectTable.At(Index);
		if (!ON_InstanceRef::Cast(Object.ObjectPtr))
		{
			TranslateNonInstanceObject(Object);
		}
		else
		{
			processedUUIDs.insert(Object.Attributes.m_uuid); // mark all block def as processed (only unprocessed objects are relevant)
			instanceRefs.push_back(Index);
		}
	}

	// Update children count of instance definition based on what has been actually processed
	for (auto& entry : objectUUIDToInstanceUUID)
	{
		if (processedUUIDs.find(entry.first) == processedUUIDs.end())
		{
			int& childrenCount = uuidToInstanceChildrenCount[entry.second];
			childrenCount--;
		}
	}

	// Process all instances
	int SuccessiveFailureCount = 0; // safety check to prevent infinite loop when all remaining instanceRefs are not loadable
	while (instanceRefs.size() > SuccessiveFailureCount)
	{
		int index = instanceRefs.front();
		instanceRefs.pop_front();

		if (!TranslateInstance(*InObjectTable.At(index)))
		{
			++SuccessiveFailureCount;
			instanceRefs.push_back(index);
		}
		else
		{
			SuccessiveFailureCount = 0;
		}
	}
}

void FOpenNurbsTranslatorImpl::SetLayers(const TSharedPtr<IDatasmithActorElement>& ActorElement, const FOpenNurbsObjectWrapper& Object)
{
	// Convert group names to tags on the actor
	FString Layers;
	ON_SimpleArray<int> GroupList;
	Object.Attributes.GetGroupList(GroupList);
	if (GroupList.Count() > 0)
	{
		for (int Index = 0; Index < GroupList.Count(); ++Index)
		{
			ActorElement->AddTag(*FString::Printf(TEXT("Rhino.GroupName: %s"), *GroupNames[*GroupList.At(Index)]));
		}
	}

	// Append the layer name of the parent container, if any
	TSharedPtr<IDatasmithActorElement> Parent = GetParentElement(Object);
	if (Parent.IsValid())
	{
		FString ParentLayer = GetLayerName(Parent);

		if (!Layers.IsEmpty())
		{
			Layers += TEXT(",") + ParentLayer;
		}
		else
		{
			Layers = ParentLayer;
		}
	}

	DatasmithOpenNurbsTranslatorUtils::PropagateLayers(ActorElement, *Layers);
}

void FOpenNurbsTranslatorImpl::SetTags(const TSharedPtr<IDatasmithActorElement>& ActorElement, const FOpenNurbsObjectWrapper& Object)
{
	ON_wString UUIDString;
	ON_UuidToString(Object.Attributes.m_uuid, UUIDString);
	FString UUID(UUIDString.Array());

	ON::object_type objType = Object.ObjectPtr->ObjectType();
	const TCHAR* StrObjectType;
	switch (objType)
	{
	case ON::instance_definition:
		StrObjectType = TEXT("block definition");
		break;
	case ON::instance_reference:
		StrObjectType = TEXT("block instance");
		break;
	case ON::point_object:
		StrObjectType = TEXT("point");
		break;
	case ON::curve_object:
		StrObjectType = TEXT("curve");
		break;
	case ON::surface_object:
		StrObjectType = TEXT("surface");
		break;
	case ON::brep_object:
		StrObjectType = TEXT("brep");
		break;
	case ON::mesh_object:
		StrObjectType = TEXT("mesh");
		break;
	case ON::text_dot:
		StrObjectType = TEXT("textdot");
		break;
	case ON::subd_object:
		StrObjectType = TEXT("subd");
		break;
	case ON::loop_object:
		StrObjectType = TEXT("loop");
		break;
	case ON::cage_object:
		StrObjectType = TEXT("cage");
		break;
	case ON::clipplane_object:
		StrObjectType = TEXT("clip plane");
		break;
	case ON::extrusion_object:
		StrObjectType = TEXT("extrusion");
		break;
	default:
		StrObjectType = TEXT("unknown");
		break;
	}

	ActorElement->AddTag(*FString::Printf(TEXT("Rhino.ID: %s"), *UUID));
	ActorElement->AddTag(*FString::Printf(TEXT("Rhino.Entity.Type: %s"), StrObjectType));
}

void FOpenNurbsTranslatorImpl::TranslateNonInstanceObject(const FOpenNurbsObjectWrapper& Object)
{
	// Ref. visitNonInstanceObject
	if (!IsValidObject(Object) || Object.ObjectPtr->IsKindOf(&ON_InstanceRef::m_ON_InstanceRef_class_rtti))
	{
		return;
	}

	// Get UUID of possible instance definition referring to this object
	ON_UUID instanceUuid = GetInstanceForObject(Object.Attributes.m_uuid);

	TSharedPtr<IDatasmithActorElement> PartElement = GetActorElement(Object);
	if (!PartElement.IsValid())
	{
		return;
	}

	if (ON_UuidIsNotNil(instanceUuid))
	{
		TSharedPtr<IDatasmithActorElement> ContainerElement = uuidToInstanceContainer[instanceUuid];
		if (ContainerElement.IsValid())
		{
			ContainerElement->AddChild(PartElement);

			TSharedPtr<IDatasmithMetaDataElement> MetaData = GetObjectMetaData(Object, PartElement);
			if (MetaData.IsValid())
			{
				ActorElementToMetaDataMap.FindOrAdd(PartElement) = MetaData;
			}
		}

		SetTags(PartElement, Object);
	}
	else
	{
		TSharedPtr<IDatasmithActorElement> Parent = GetParentElement(Object);

		if (Parent.IsValid())
		{
			Parent->AddChild(PartElement);
		}
		else
		{
			Scene->AddActor(PartElement);
		}

		TSharedPtr<IDatasmithMetaDataElement> MetaData = GetObjectMetaData(Object, PartElement);
		if (MetaData.IsValid())
		{
			Scene->AddMetaData(MetaData);
		}

		SetLayers(PartElement, Object);
		SetTags(PartElement, Object);
	}

	// Register UUID of fully processed object
	processedUUIDs.insert(Object.Attributes.m_uuid);
}

bool FOpenNurbsTranslatorImpl::IsValidObject(const FOpenNurbsObjectWrapper& Object)
{
	if (Object.ObjectPtr == nullptr)
	{
		return false;
	}

	if (!Object.Attributes.IsVisible() || Object.Attributes.Mode() == ON::hidden_object || HiddenLayersIndices.Contains(Object.Attributes.m_layer_index))
	{
		// Object skipped because it's not visible
		return false;
	}

	if (Object.ObjectPtr->IsKindOf(&ON_Mesh::m_ON_Mesh_class_rtti) ||
		Object.ObjectPtr->IsKindOf(&ON_Brep::m_ON_Brep_class_rtti) ||
		Object.ObjectPtr->IsKindOf(&ON_PlaneSurface::m_ON_PlaneSurface_class_rtti) ||
		Object.ObjectPtr->IsKindOf(&ON_InstanceRef::m_ON_InstanceRef_class_rtti) ||
		Object.ObjectPtr->IsKindOf(&ON_Point::m_ON_Point_class_rtti) ||
		//object.ObjectPtr->IsKindOf(&ON_LineCurve::m_ON_LineCurve_class_rtti) ||
		Object.ObjectPtr->IsKindOf(&ON_Extrusion::m_ON_Extrusion_class_rtti) ||
		Object.ObjectPtr->IsKindOf(&ON_Hatch::m_ON_Hatch_class_rtti) ||
		Object.ObjectPtr->IsKindOf(&ON_Light::m_ON_Light_class_rtti))
	{
		return true;
	}

	// Support for object type not implemented

	return false;
}

const ON_UUID& FOpenNurbsTranslatorImpl::GetInstanceForObject(const ON_UUID& objectUUID)
{
	std::map<ON_UUID, ON_UUID>::iterator ite = objectUUIDToInstanceUUID.find(objectUUID);
	if (ite == objectUUIDToInstanceUUID.end())
		return ON_nil_uuid;
	return ite->second;
}

/**
* Check if this object has "unprocessed" children (blockref/sub-objects are not loaded)
* Previous load is necessary before copy because we do not keep instances
*/
bool FOpenNurbsTranslatorImpl::HasUnprocessedChildren(const ON_UUID& instanceDefUuid)
{
	int childrenCount = uuidToInstanceChildrenCount[instanceDefUuid];
	if (childrenCount == 0)
	{
		return false;
	}

	TSharedPtr<IDatasmithActorElement> InstanceDefinition = uuidToInstanceContainer[instanceDefUuid];
	childrenCount -= InstanceDefinition->GetChildrenCount();

	return childrenCount > 0 ? true : false;
}

// Returns false if visit must be delayed and true if creation was successful
bool FOpenNurbsTranslatorImpl::TranslateInstance(const FOpenNurbsObjectWrapper& Object)
{
	// Ref. visitInstance
	ON_InstanceRef* instanceRef = ON_InstanceRef::Cast(Object.ObjectPtr);
	if (instanceRef == nullptr)
	{
		return true;
	}

	ON_UUID instanceDefUuid = instanceRef->m_instance_definition_uuid;
	std::map<ON_UUID, TSharedPtr<IDatasmithActorElement>>::iterator ite = uuidToInstanceContainer.find(instanceDefUuid);
	if (ite == uuidToInstanceContainer.end())
	{
		return true;
	}

	if (uuidToInstanceChildrenCount[instanceDefUuid] <= 0)
	{
		return true;
	}

	if (HasUnprocessedChildren(instanceDefUuid))
	{
		return false;
	}

	TSharedPtr<IDatasmithActorElement> InstanceDefinition = ite->second;

	const ON_UUID& instanceRefUuid = Object.Attributes.m_uuid;
	ON_UUID instanceUuid = GetInstanceForObject(instanceRefUuid);

	// container name
	FString ContainerName;
	if (Object.Attributes.Name().Length() > 0)
	{
		ContainerName = Object.Attributes.Name().Array();
	}
	else
	{
		ContainerName = InstanceDefinition->GetName();
	}

	TSharedPtr<IDatasmithActorElement> ContainerElement = FDatasmithSceneFactory::CreateActor(*ContainerName);
	ContainerElement->SetLabel(*ContainerName);

	// Instance world transform
	FMatrix Matrix;
	DatasmithOpenNurbsTranslatorUtils::XFormToMatrix(instanceRef->m_xform, Matrix);

	// Ref. FDatasmithCADImporter::SetWorldTransform
	FTransform Transform(Matrix);
	FTransform CorrectedTransform = FDatasmithUtils::ConvertTransform(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy, Transform);

	ContainerElement->SetTranslation(CorrectedTransform.GetTranslation() * GetScaleFactor());
	ContainerElement->SetScale(CorrectedTransform.GetScale3D());
	ContainerElement->SetRotation(CorrectedTransform.GetRotation());
	bool bIsPartOfInstanceDefinition = false;

	// If instance of an instance, parent to parent instance definition
	if (ON_UuidIsNotNil(instanceUuid) == true)
	{
		bIsPartOfInstanceDefinition = true;
		TSharedPtr<IDatasmithActorElement> InstanceContainer = uuidToInstanceContainer[instanceUuid];
		InstanceContainer->AddChild(ContainerElement);

		TSharedPtr<IDatasmithMetaDataElement> MetaData = GetObjectMetaData(Object, ContainerElement);
		if (MetaData.IsValid())
		{
			ActorElementToMetaDataMap.FindOrAdd(ContainerElement) = MetaData;
		}
	}
	else
	{
		TSharedPtr<IDatasmithActorElement> Parent = GetParentElement(Object);
		if (Parent.IsValid())
		{
			Parent->AddChild(ContainerElement);
		}
		else
		{
			Scene->AddActor(ContainerElement);
		}

		TSharedPtr<IDatasmithMetaDataElement> MetaData = GetObjectMetaData(Object, ContainerElement);
		if (MetaData.IsValid())
		{
			Scene->AddMetaData(MetaData);
		}
	}

	// Update UUID attribute
	ON_wString uuidString;
	ON_UuidToString(instanceRefUuid, uuidString);
	ContainerElement->SetName(uuidString.Array());

	// Copy the children of the instance definition
	// Clone the elements from the instance definition to the container element (recursive)
	int32 NumChildren = InstanceDefinition->GetChildrenCount();
	for (int32 Index = 0; Index < NumChildren; ++Index)
	{
		TSharedPtr<IDatasmithActorElement> Child = InstanceDefinition->GetChild(Index);

		TSharedPtr<IDatasmithActorElement> DuplicatedChild = DuplicateActorElement(Child, ContainerElement->GetName());
		ContainerElement->AddChild(DuplicatedChild, EDatasmithActorAttachmentRule::KeepRelativeTransform);

		if (TSharedPtr<IDatasmithMetaDataElement>* SourceMetaData = ActorElementToMetaDataMap.Find(Child))
		{
			TSharedPtr<IDatasmithMetaDataElement> DuplicatedMetaData = DuplicateMetaDataElement(*SourceMetaData, DuplicatedChild);

			if (bIsPartOfInstanceDefinition)
			{
				ActorElementToMetaDataMap.FindOrAdd(DuplicatedChild) = DuplicatedMetaData;
			}
			else
			{
				Scene->AddMetaData(DuplicatedMetaData);
			}
		}
	}

	SetLayers(ContainerElement, Object);
	SetTags(ContainerElement, Object);

	// TODO: Apply material override
	return true;
}

TSharedPtr<IDatasmithMetaDataElement> FOpenNurbsTranslatorImpl::GetObjectMetaData(const FOpenNurbsObjectWrapper& Object, const TSharedPtr<IDatasmithElement>& DatasmithElement)
{
	ON_ClassArray<ON_wString> UserStringKeys;
	if (Object.Attributes.GetUserStringKeys(UserStringKeys))
	{
		FString MetaDataName = FString::Printf(TEXT("%s%s"), DatasmithElement->GetName(), TEXT("_DATA"));
		TSharedPtr<IDatasmithMetaDataElement> MetaData = FDatasmithSceneFactory::CreateMetaData(*MetaDataName);
		MetaData->SetAssociatedElement(DatasmithElement);

		for (int32 UserTextIndex = 0; UserTextIndex < UserStringKeys.Count(); ++UserTextIndex)
		{
			ON_wString& KeyString = *UserStringKeys.At(UserTextIndex);
			ON_wString Value;
			if (Object.Attributes.GetUserString(KeyString.Array(), Value))
			{
				TSharedRef<IDatasmithKeyValueProperty> MetaDataProperty = FDatasmithSceneFactory::CreateKeyValueProperty(KeyString.Array());
				MetaDataProperty->SetValue(Value.Array());
				MetaDataProperty->SetPropertyType(EDatasmithKeyValuePropertyType::String);

				MetaData->AddProperty(MetaDataProperty);
			}
		}

		if (MetaData->GetPropertiesCount() > 0)
		{
			return  MetaData;
		}
	}

	return TSharedPtr<IDatasmithMetaDataElement>();
}

TSharedPtr<IDatasmithActorElement> FOpenNurbsTranslatorImpl::GetActorElement(const FOpenNurbsObjectWrapper& Object)
{
	TSharedPtr<IDatasmithActorElement> ActorElement;

	if (Object.ObjectPtr->ObjectType() == ON::object_type::point_object)
	{
		ActorElement = GetPointActorElement(Object);
	}
	else
	{
		ActorElement = GetMeshActorElement(Object);
	}

	return ActorElement;
}

TSharedPtr<IDatasmithMeshActorElement> FOpenNurbsTranslatorImpl::GetMeshActorElement(const FOpenNurbsObjectWrapper& Object)
{
	// Ref. createScenePart
	// Ref. FDatasmithCADImporter::ProcessPart

	// TODO: Get UUID
	FString Uuid;

	ON_wString uuidString;
	ON_UuidToString(Object.Attributes.m_uuid, uuidString);

	FString ActorName = Uuid + uuidString.Array();
	FString ActorLabel;
	if (Object.Attributes.m_name.Length() > 0)
	{
		ActorLabel = Object.Attributes.m_name.Array();
	}
	else
	{
		const ON_ClassId* classId = Object.ObjectPtr->ClassId();
		ActorLabel = classId->ClassName();
	}

	if (ActorName.IsEmpty())
	{
		ActorName = Uuid + ActorLabel;
	}

	// Get the associated Mesh element
	TSharedPtr<IDatasmithMeshActorElement> ActorElement;

	TSharedPtr<IDatasmithMeshElement> MeshElement = GetMeshElement(Object, Uuid, ActorLabel);
	if (!MeshElement.IsValid())
	{
		return ActorElement;
	}

	ActorElement = FDatasmithSceneFactory::CreateMeshActor(*ActorName);
	if (!ActorElement.IsValid())
	{
		return ActorElement;
	}

	ActorElement->SetLabel(*ActorLabel);
	ActorElement->SetStaticMeshPathName(MeshElement->GetName());

	ON_3dVector GeometryCenter;
	if (ComputeObjectGeometryCenter(Object, GeometryCenter))
	{
		MeshElementToGeometryCenter.Add(MeshElement.ToSharedRef(), GeometryCenter);
		FVector ActorOffset(GeometryCenter.x, GeometryCenter.y, GeometryCenter.z);
		ActorOffset = FDatasmithUtils::ConvertVector(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy, ActorOffset) * GetScaleFactor();
		ActorElement->SetTranslation(ActorOffset);
	}

	// TODO: TBD if need to set Material Override

	return ActorElement;
}

TSharedPtr<IDatasmithActorElement> FOpenNurbsTranslatorImpl::GetPointActorElement(const FOpenNurbsObjectWrapper& Object)
{
	TSharedPtr<IDatasmithActorElement> ActorElement;

	ON_Point* pointObj = ON_Point::Cast(Object.ObjectPtr);
	if (pointObj == nullptr)
	{
		return ActorElement;
	}

	ON_wString uuidString;
	ON_UuidToString(Object.Attributes.m_uuid, uuidString);

	FString ActorName = uuidString.Array();
	FString ActorLabel;
	if (Object.Attributes.m_name.Length() > 0)
	{
		ActorLabel = Object.Attributes.m_name.Array();
	}
	else
	{
		const ON_ClassId* classId = Object.ObjectPtr->ClassId();
		ActorLabel = classId->ClassName();
	}

	if (ActorName.IsEmpty())
	{
		ActorName = ActorLabel;
	}

	ActorElement = FDatasmithSceneFactory::CreateActor(*ActorName);
	if (!ActorElement.IsValid())
	{
		return ActorElement;
	}

	FVector Location(pointObj->point.x, pointObj->point.y, pointObj->point.z);
	Location *= GetScaleFactor();
	Location = FDatasmithUtils::ConvertVector(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy, Location);

	ActorElement->SetTranslation(Location);
	ActorElement->SetLabel(*ActorLabel);

	return ActorElement;
}

void FOpenNurbsTranslatorImpl::SetMaterialToMeshElement(TSharedPtr<IDatasmithMeshElement> MeshElement, TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement, int32 SlotId)
{
	if (MaterialElement.IsValid())
	{
		MeshElement->SetMaterial(MaterialElement->GetName(), SlotId);

		// And add it to the Datasmith Scene as needed
		if (!UsedMaterials.Contains(MaterialElement))
		{
			UsedMaterials.Add(MaterialElement);
			Scene->AddMaterial(MaterialElement);
		}
	}
}

TSharedPtr<IDatasmithMeshElement> FOpenNurbsTranslatorImpl::GetMeshElement(const FOpenNurbsObjectWrapper& Object, const FString& Uuid, const FString& Label)
{
	// Ref. FDatasmithCADImporter::FindOrAddMeshElement
	// Look if geometry has not been already processed, return it if found
	TSharedPtr<IDatasmithMeshElement>* MeshElementPtr = ObjectToMeshElementMap.Find(&Object);
	if (MeshElementPtr != nullptr && MeshElementPtr->IsValid())
	{
		return *MeshElementPtr;
	}

	// Not processed yet, build new Datasmith mesh element
	FString MeshName = Uuid + DatasmithOpenNurbsTranslatorUtils::BuildMeshName(SceneName, Object);
	TSharedPtr<IDatasmithMeshElement> MeshElement = FDatasmithSceneFactory::CreateMesh(*MeshName);

	MeshElement->SetLabel(*Label);
	MeshElement->SetLightmapSourceUV(-1);

	if (BaseOptions.bIncludeMaterial)
	{
		FMaterial Material = GetObjectMaterial(Object);

		SetMaterialToMeshElement(MeshElement, Material.MaterialElement, 0);

		// Material slots(OpenNurbs 'channels')
		// https://developer.rhino3d.com/guides/opennurbs/reading-per-face-render-materials/
		if (Material.OpenNurbsMaterialPtr)
		{
			const ON_Material& OpenNurbsMaterial = *Material.OpenNurbsMaterialPtr;
			for (int MaterialChannel = 0; MaterialChannel < OpenNurbsMaterial.m_material_channel.Count(); ++MaterialChannel)
			{
				const ON_UuidIndex& MaterialIdx = OpenNurbsMaterial.m_material_channel[MaterialChannel];
				if (int* IndexPtr = MaterialIdToIndex.Find(MaterialIdx.m_id))
				{
					// In OpenNurbs, m_material_channel is indexed by "m_face_material_channel-1"(where m_face_material_channel==0 means 'parent' material)
					// We put "channel material" into slot "channel+1" so 'parent' material can go into slot 0 and can index slots by unmodified m_face_material_channel
					SetMaterialToMeshElement(MeshElement, GetMaterial(*IndexPtr), MaterialChannel + 1);
				}
			}
		}
	}

	Scene->AddMesh(MeshElement);

	// Update all tables for future referencing
	ObjectToMeshElementMap.Add(&Object, MeshElement);
	MeshElementToObjectMap.Add(MeshElement.Get(), &Object);
	MeshElementToTranslatorMap.Add(MeshElement.Get(), this);

	uint8 IncludeMaterial = static_cast<uint8>(BaseOptions.bIncludeMaterial);

	FMD5 MD5;
	MD5.Update(&IncludeMaterial, sizeof(IncludeMaterial));

	// Use the object's CRC as the mesh element hash
	uint32 CRC = Object.ObjectPtr->DataCRC(0);
	if (ON_Brep::Cast(Object.ObjectPtr))
	{
		CRC ^= OpenNurbsOptionsHash;
	}
	MD5.Update(reinterpret_cast<const uint8*>(&CRC), sizeof CRC);

	FMD5Hash Hash;
	Hash.Set(MD5);
	MeshElement->SetFileHash(Hash);

	return MeshElement;
}

FOpenNurbsTranslatorImpl::FMaterial FOpenNurbsTranslatorImpl::GetObjectMaterial(const FOpenNurbsObjectWrapper& Object)
{
	// Ref: getObjectMaterialID(const ONX_Model_Object& object)
	ON::object_material_source MaterialSource = Object.Attributes.MaterialSource();
	if (MaterialSource == ON::material_from_parent && Object.Attributes.Mode() != ON::object_mode::idef_object)
	{
		MaterialSource = ON::material_from_layer;
	}

	switch (MaterialSource)
	{
	case ON::material_from_object:
	{
		if (Object.Attributes.m_material_index != -1)
		{
			return { GetMaterial(Object.Attributes.m_material_index) , GetOpenNurbsMaterial(Object.Attributes.m_material_index) };
		}
		break;
	}
	case ON::material_from_layer:
	{
		if (int* MaterialIndexPtr = LayerIndexToMaterialIndex.Find(Object.Attributes.m_layer_index))
		{
			return { GetMaterial(*MaterialIndexPtr), GetOpenNurbsMaterial(*MaterialIndexPtr) };
		}
		break;
	}
	}

	return { GetDefaultMaterial() };
}

ON_Material* FOpenNurbsTranslatorImpl::GetOpenNurbsMaterial(int MaterialIndex)
{
	int* MaterialTableIndexPtr = MaterialIndexToMaterialTableIndex.Find(MaterialIndex);

	if (!MaterialTableIndexPtr)
	{
		return nullptr;
	}

	return MaterialTable.At(*MaterialTableIndexPtr);
}

TSharedPtr<IDatasmithBaseMaterialElement> FOpenNurbsTranslatorImpl::GetMaterial(int MaterialIndex)
{
	// This is populated when translating the material table
	TSharedPtr<IDatasmithBaseMaterialElement>* Material = MaterialIndexToMaterial.Find(MaterialIndex);
	if (Material)
	{
		return *Material;
	}
	return GetDefaultMaterial();
}

TSharedPtr<IDatasmithBaseMaterialElement> FOpenNurbsTranslatorImpl::GetDefaultMaterial()
{
	if (DefaultMaterial.IsValid())
	{
		return DefaultMaterial;
	}

	// Generate a default material that mimics the white plaster in Rhino
	TSharedRef<IDatasmithUEPbrMaterialElement> Material = FDatasmithSceneFactory::CreateUEPbrMaterial(TEXT("ON_DefaultMaterial"));
	Material->SetLabel(TEXT("Default"));

	FColor Color(250, 250, 250, 255);
	FLinearColor LinearColor = FLinearColor::FromSRGBColor(Color);

	IDatasmithMaterialExpressionColor* ColorExpression = Material->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
	ColorExpression->SetName(TEXT("Diffuse Color"));
	ColorExpression->GetColor() = LinearColor;

	Material->GetBaseColor().SetExpression(ColorExpression);

	DefaultMaterial = Material;

	return DefaultMaterial;
}

TSharedPtr<IDatasmithActorElement> FOpenNurbsTranslatorImpl::GetParentElement(const FOpenNurbsObjectWrapper& Object)
{
	TSharedPtr<IDatasmithActorElement>* Parent = LayerIndexToContainer.Find(Object.Attributes.m_layer_index);
	if (Parent)
	{
		return *Parent;
	}
	return nullptr;
}

FString FOpenNurbsTranslatorImpl::GetLayerName(const TSharedPtr<IDatasmithActorElement>& LayerElement)
{
	FString* LayerName = LayerNames.Find(LayerElement);
	if (LayerName)
	{
		return *LayerName;
	}

	return LayerElement->GetLabel();
}

//////////////////////////////////////////////////////////////////////////

bool FOpenNurbsTranslatorImpl::Read(ON_BinaryFile& Archive, TSharedRef<IDatasmithScene> OutScene)
{
	bool bResult = false;
	NumCRCErrors = 0;

	// Step 1: REQUIRED - Read Start Section
	if (!Archive.Read3dmStartSection(&FileVersion, StartSectionComments))
	{
		return false;
	}
	else if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, nullptr, NumCRCErrors))
	{
		bResult = false;
	}

	// Step 2: REQUIRED - Read properties table
	if (!Archive.Read3dmProperties(Properties))
	{
		return false;
	}
	else if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "properties section", NumCRCErrors))
	{
		bResult = false;
	}

	// Version of OpenNurbs used to write the file. Only available after Read3dmProperties
	ArchiveOpenNurbsVersion = Archive.ArchiveOpenNURBSVersion();

	// Set ProductName, ProductVersion in DatasmithScene
	// application_name is something like "Rhino 5.0"
	// while application_details contains build type and version
	OutScene->SetProductName(Properties.m_Application.m_application_name.Array());
	OutScene->SetProductVersion(Properties.m_Application.m_application_details.Array());

	// Step 3: REQUIRED - Read settings table
	if (!Archive.Read3dmSettings(Settings))
	{
		return false;
	}
	else if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "settings section", NumCRCErrors))
	{
		bResult = false;
	}

	const double MetricUnit = Settings.m_ModelUnitsAndTolerances.Scale(ON::LengthUnitSystem::Millimeters);
	SetMetricUnit(MetricUnit);

	// Step 4: REQUIRED - Read bitmap table (it can be empty)
	int Count = 0;
	int ReturnCode = 0;

	if (Archive.BeginRead3dmBitmapTable())
	{
		ON_SimpleArray<ON_Bitmap*> BitmapTable;

		// At the moment no bitmaps are embedded so this table is empty
		ON_Bitmap* pBitmap = nullptr;
		for (Count = 0; true; ++Count)
		{
			pBitmap = nullptr;
			ReturnCode = Archive.Read3dmBitmap(&pBitmap);
			if (ReturnCode == 0)
			{
				break; // end of bitmap table
			}
			if (ReturnCode < 0)
			{
				bResult = false;
			}
		}

		// EndRead3dmBitmapTable() must be called when BeginRead3dmBitmapTable() returns true
		if (!Archive.EndRead3dmBitmapTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "bitmap table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	// Nothing to do for now

	// Step 5: REQUIRED - Read texture mapping table (it can be empty)
	if (Archive.BeginRead3dmTextureMappingTable())
	{
		ON_TextureMapping* pTextureMapping = nullptr;
		for (Count = 0; true; ++Count)
		{
			ReturnCode = Archive.Read3dmTextureMapping(&pTextureMapping);
			if (ReturnCode == 0)
			{
				break; // end of texture_mapping table
			}
			if (ReturnCode < 0)
			{
				continue;
			}
			ON_UserDataHolder ud;
			ud.MoveUserDataFrom(*pTextureMapping);
			TextureMappingTable.Append(*pTextureMapping);
			pTextureMapping->SetIndex(Count);
			ud.MoveUserDataTo(*TextureMappingTable.Last(), false);
			delete pTextureMapping;
			pTextureMapping = nullptr;
		}

		// EndRead3dmTextureMappingTable() must be called when BeginRead3dmTextureMappingTable() returns true.
		if (!Archive.EndRead3dmTextureMappingTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "render texture_mapping table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	TranslateTextureMappingTable(TextureMappingTable);

	// Step 6: REQUIRED - Write/Read render material table (it can be empty)
	if (Archive.BeginRead3dmMaterialTable())
	{
		ON_Material* pMaterial = nullptr;
		for (Count = 0; true; ++Count)
		{
			ReturnCode = Archive.Read3dmMaterial(&pMaterial);
			if (ReturnCode == 0)
			{
				break; // end of material table
			}
			if (ReturnCode < 0)
			{
				pMaterial = new ON_Material; // use default
				pMaterial->SetIndex(Count);
			}
			ON_UserDataHolder ud;
			ud.MoveUserDataFrom(*pMaterial);
			MaterialTable.Append(*pMaterial);
			ud.MoveUserDataTo(*MaterialTable.Last(), false);
			delete pMaterial;
			pMaterial = nullptr;
		}

		// EndRead3dmMaterialTable() must be called when BeginRead3dmMaterialTable() returns true.
		if (!Archive.EndRead3dmMaterialTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "render material table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	// Translate OpenNurbs materials to Datasmith Material elements
	TranslateMaterialTable(MaterialTable);

	// Step 7: REQUIRED - Read linetype table (it can be empty)
	if (Archive.BeginRead3dmLinetypeTable())
	{
		ON_ObjectArray<ON_Linetype> LineTypeTable;

		ON_Linetype* pLinetype = nullptr;
		for (Count = 0; true; ++Count)
		{
			ReturnCode = Archive.Read3dmLinetype(&pLinetype);
			if (ReturnCode == 0)
			{
				break; // end of linetype table
			}
			if (ReturnCode < 0)
			{
				pLinetype = new ON_Linetype; // use default
				pLinetype->SetIndex(Count);
			}
			ON_UserDataHolder ud;
			ud.MoveUserDataFrom(*pLinetype);
			LineTypeTable.Append(*pLinetype);
			ud.MoveUserDataTo(*LineTypeTable.Last(), false);
			delete pLinetype;
			pLinetype = NULL;
		}

		// EndRead3dmLinetypeTable() must be called when BeginRead3dmLinetypeTable() returns true.
		if (!Archive.EndRead3dmLinetypeTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "render linetype table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	// Nothing to do for now

	// Step 8: REQUIRED - Read layer table (it can be empty)
	if (Archive.BeginRead3dmLayerTable())
	{
		ON_Layer* pLayer = nullptr;
		for (Count = 0; true; ++Count)
		{
			pLayer = nullptr;
			ReturnCode = Archive.Read3dmLayer(&pLayer);
			if (ReturnCode == 0)
			{
				break; // end of layer table
			}
			if (ReturnCode < 0)
			{
				pLayer = new ON_Layer; // use default
				pLayer->SetIndex(Count);
			}
			ON_UserDataHolder ud;
			ud.MoveUserDataFrom(*pLayer);
			LayerTable.Append(*pLayer);
			ud.MoveUserDataTo(*LayerTable.Last(), false);
			delete pLayer;
			pLayer = nullptr;
		}

		// EndRead3dmLayerTable() must be called when BeginRead3dmLayerTable() returns true.
		if (!Archive.EndRead3dmLayerTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "layer table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	// Translate layer data to Datasmith Actor elements
	TranslateLayerTable(LayerTable);

	// Step 9: REQUIRED - Read group table (it can be empty)
	if (Archive.BeginRead3dmGroupTable())
	{
		ON_Group* pGroup = nullptr;
		for (Count = 0; true; ++Count)
		{
			ReturnCode = Archive.Read3dmGroup(&pGroup);
			if (ReturnCode == 0)
			{
				break; // end of group table
			}
			if (ReturnCode < 0)
			{
				pGroup = new ON_Group; // use default
				pGroup->SetIndex(-1);
			}
			ON_UserDataHolder ud;
			ud.MoveUserDataFrom(*pGroup);
			GroupTable.Append(*pGroup);
			ud.MoveUserDataTo(*GroupTable.Last(), false);
			delete pGroup;
			pGroup = nullptr;
		}

		// EndRead3dmGroupTable() must be called when BeginRead3dmGroupTable() returns true.
		if (!Archive.EndRead3dmGroupTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "group table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	TranslateGroupTable(GroupTable);

	// Step 10: REQUIRED - Read font table (it can be empty)
	// Actually deprecated, no processing required

	// Step 11: REQUIRED - Read dimstyle table (it can be empty)
	if (Archive.BeginRead3dmDimStyleTable())
	{
		ON_ObjectArray<ON_DimStyle> DimStyleTable;

		ON_DimStyle* pDimStyle = nullptr;
		for (Count = 0; true; ++Count)
		{
			ReturnCode = Archive.Read3dmDimStyle(&pDimStyle);
			if (ReturnCode == 0)
			{
				break; // end of dimstyle table
			}
			if (ReturnCode < 0)
			{
				pDimStyle = new ON_DimStyle; // use default
				pDimStyle->SetIndex(Count);
			}
			ON_UserDataHolder ud;
			ud.MoveUserDataFrom(*pDimStyle);
			DimStyleTable.Append(*pDimStyle);
			ud.MoveUserDataTo(*DimStyleTable.Last(), false);
			delete pDimStyle;
			pDimStyle = nullptr;
		}

		// If BeginRead3dmDimStyleTable() returns true,
		// then you MUST call EndRead3dmDimStyleTable().
		if (!Archive.EndRead3dmDimStyleTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "dimstyle table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	// Nothing to do for now

	// Step 12: REQUIRED - Write/Read render light table (it can be empty)
	if (Archive.BeginRead3dmLightTable())
	{
		ON_Light* pLight = nullptr;
		ON_3dmObjectAttributes object_attributes;
		for (Count = 0; true; ++Count)
		{
			object_attributes.Default();
			ReturnCode = Archive.Read3dmLight(&pLight, &object_attributes);
			if (ReturnCode == 0)
			{
				break; // end of light table
			}
			if (ReturnCode < 0)
			{
				continue;
			}

			FOpenNurbsObjectWrapper& lightObject = LightTable.AppendNew();
			lightObject.ObjectPtr = pLight;
			lightObject.Attributes = object_attributes;
		}

		// EndRead3dmLightTable() must be called when BeginRead3dmLightTable() returns true.
		if (!Archive.EndRead3dmLightTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "render light table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	TranslateLightTable(LightTable);

	// Free up memory. This should already be handled by FOpenNurbsObjectWrapper!
	for (int ite = 0; ite < LightTable.Count(); ++ite)
	{
		FOpenNurbsObjectWrapper& object = *LightTable.At(ite);

		ON_Light* pLight = static_cast<ON_Light*>(object.ObjectPtr);
		delete pLight;

		object.ObjectPtr = nullptr;
	}

	LightTable.Empty();

	// Step 13: REQUIRED - Read hatch pattern table (it can be empty)
	if (Archive.BeginRead3dmHatchPatternTable())
	{
		ON_ObjectArray<ON_HatchPattern> HatchPatternTable;

		ON_HatchPattern* pHatchPattern = nullptr;
		for (Count = 0; true; ++Count)
		{
			ReturnCode = Archive.Read3dmHatchPattern(&pHatchPattern);
			if (ReturnCode == 0)
			{
				break; // end of hatchpattern table
			}
			if (ReturnCode < 0)
			{
				pHatchPattern = new ON_HatchPattern; // use default
				pHatchPattern->SetIndex(Count);
			}
			ON_UserDataHolder ud;
			ud.MoveUserDataFrom(*pHatchPattern);
			HatchPatternTable.Append(*pHatchPattern);
			ud.MoveUserDataTo(*HatchPatternTable.Last(), false);
			delete pHatchPattern;
			pHatchPattern = nullptr;
		}

		// EndRead3dmHatchPatternTable() must be called when BeginRead3dmHatchPatternTable() returns true.
		if (!Archive.EndRead3dmHatchPatternTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "hatchpattern table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	// Nothing to do

	// Step 14: REQUIRED - Read instance definition table (it can be empty)
	if (Archive.BeginRead3dmInstanceDefinitionTable())
	{
		ON_InstanceDefinition* pIDef = nullptr;
		for (Count = 0; true; ++Count)
		{
			ReturnCode = Archive.Read3dmInstanceDefinition(&pIDef);
			if (ReturnCode == 0)
			{
				break; // end of instance definition table
			}
			if (ReturnCode < 0)
			{
				continue;
			}
			InstanceDefinitionTable.Add(pIDef);
		}

		// EndRead3dmInstanceDefinitionTable() must be called when BeginRead3dmInstanceDefinitionTable() returns true.
		if (!Archive.EndRead3dmInstanceDefinitionTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "instance definition table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	TranslateInstanceDefinitionTable(InstanceDefinitionTable);

	// Step 15: REQUIRED - Read geometry and annotation table (it can be empty)
	if (Archive.BeginRead3dmObjectTable())
	{
		// optional filter made by setting ON::object_type bits
		// For example, if you just wanted to just read points and meshes, you would use
		// object_filter = ON::point_object | ON::mesh_object;
		int object_filter = 0;

		for (Count = 0; true; ++Count)
		{
			ON_Object* pObject = nullptr;
			ON_3dmObjectAttributes attributes;
			ReturnCode = Archive.Read3dmObject(&pObject, &attributes, object_filter);

			if (ReturnCode == 0)
			{
				break; // end of object table
			}
			if (ReturnCode < 0)
			{
				continue;
			}
			if (pObject)
			{
				FOpenNurbsObjectWrapper& mo = ObjectTable.AppendNew();
				mo.ObjectPtr = pObject;
				mo.Attributes = attributes;
			}
		}

		// EndRead3dmObjectTable() must be called when BeginRead3dmObjectTable() returns true.
		if (!Archive.EndRead3dmObjectTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "object table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	TranslateObjectTable(ObjectTable);

	// Clean up all instance definitions, not required anymore
	for (ON_InstanceDefinition* pInstanceDef : InstanceDefinitionTable)
	{
		delete pInstanceDef;
	}

	// Step 16: REQUIRED - Read history record table (it can be empty)
	if (Archive.BeginRead3dmHistoryRecordTable())
	{
		ON_SimpleArray<ON_HistoryRecord*> HistoryRecordTable;
		for (Count = 0; true; ++Count)
		{
			ON_HistoryRecord* pHistoryRecord = nullptr;
			ReturnCode = Archive.Read3dmHistoryRecord(pHistoryRecord);
			if (ReturnCode == 0)
			{
				break; // end of history record table
			}
			if (ReturnCode < 0)
			{
				continue;
			}

			if (pHistoryRecord)
			{
				HistoryRecordTable.Append(pHistoryRecord);
			}
		}

		// EndRead3dmHistoryRecordTable() must be called when BeginRead3dmHistoryRecordTable() returns true.
		if (!Archive.EndRead3dmHistoryRecordTable())
		{
			return false;
		}
		if (DatasmithOpenNurbsTranslatorUtils::CheckForCRCErrors(Archive, "history record table", NumCRCErrors))
		{
			bResult = false;
		}
	}
	else
	{
		bResult = false;
	}

	// Nothing to do

	// Step 17: OPTIONAL - Read 0 or more user tables as anonymous goo to be interpreted by plug-ins
	for (Count = 0; true; ++Count)
	{
		if (Archive.Archive3dmVersion() <= 1)
		{
			// no user tables in version 1 archives.
			break;
		}

		{
			ON__UINT32 tcode = 0;
			ON__INT64 big_value = 0;
			if (!Archive.PeekAt3dmBigChunkType(&tcode, &big_value))
			{
				break;
			}
			if (TCODE_USER_TABLE != tcode)
			{
				break;
			}
		}
		ON_UUID plugin_id;
		bool bGoo = false;
		int usertable_3dm_version = 0;
		unsigned int usertable_opennurbs_version = 0;
		if (!Archive.BeginRead3dmUserTable(plugin_id, &bGoo, &usertable_3dm_version, &usertable_opennurbs_version))
		{
			// attempt to skip bogus user table
			const ON__UINT64 pos0 = Archive.CurrentPosition();
			ON__UINT32 tcode = 0;
			ON__INT64 big_value = 0;
			if (!Archive.BeginRead3dmBigChunk(&tcode, &big_value))
			{
				break;
			}
			if (!Archive.EndRead3dmChunk())
			{
				break;
			}
			const ON__UINT64 pos1 = Archive.CurrentPosition();
			if (pos1 <= pos0)
			{
				break;
			}
			if (TCODE_USER_TABLE != tcode)
			{
				break;
			}

			continue; // skip this bogus user table
		}

		ONX_Model_UserData& ud = UserDataTable.AppendNew();
		ud.m_uuid = plugin_id;
		ud.m_usertable_3dm_version = usertable_3dm_version;
		ud.m_usertable_opennurbs_version = usertable_opennurbs_version;

		if (!Archive.Read3dmAnonymousUserTable(usertable_3dm_version, usertable_opennurbs_version, ud.m_goo))
		{
			break;
		}

		// If BeginRead3dmObjectTable() returns true,
		// EndRead3dmUserTable() must be called when BeginRead3dmObjectTable() returns true.
		if (!Archive.EndRead3dmUserTable())
		{
			break;
		}
	}

	// Clean up the scene by removing unused layer actors
	TArray<TSharedPtr<IDatasmithActorElement>> ActorsToRemove;
	int32 NumActors = Scene->GetActorsCount();
	for (int32 Index = 0; Index < NumActors; ++Index)
	{
		const TSharedPtr<IDatasmithActorElement>& ActorElement = Scene->GetActor(Index);
		if (ActorElement->GetChildrenCount() == 0)
		{
			ActorsToRemove.Add(ActorElement);
		}
	}

	for (const TSharedPtr<IDatasmithActorElement>& ActorElement : ActorsToRemove)
	{
		Scene->RemoveActor(ActorElement, EDatasmithActorRemovalRule::RemoveChildren);
	}

	return true;
}

bool FOpenNurbsTranslatorImpl::TranslateBRep(ON_Brep* Brep, const ON_3dmObjectAttributes& Attributes, FMeshDescription& OutMesh, TSharedRef< IDatasmithMeshElement >& MeshElement, const FString& Name, bool& bHasNormal)
{
	if (Brep == nullptr)
	{
		return false;
	}

	ON_3dVector Offset = GetGeometryOffset(MeshElement);

	// No tessellation if CAD library is not present...
	if (OpenNurbsOptions.Geometry == EDatasmithOpenNurbsBrepTessellatedSource::UseUnrealNurbsTessellation)
	{
		// Ref. visitBRep
		CADModelConverter->SetImportParameters(OpenNurbsOptions.ChordTolerance, OpenNurbsOptions.MaxEdgeLength, OpenNurbsOptions.NormalTolerance, (CADLibrary::EStitchingTechnique)OpenNurbsOptions.StitchingTechnique);

		CADModelConverter->InitializeProcess();

		OpenNurbsBRepConverter->AddBRep(*Brep, Offset);
		CADModelConverter->RepairTopology();

		CADModelConverter->SaveModel(*OutputPath, MeshElement);

		CADLibrary::FMeshParameters MeshParameters;
		bool bRet = CADModelConverter->Tessellate(MeshParameters, OutMesh);
		if (!bRet)
		{
			FString StaticMeshLable = MeshElement->GetLabel();
			UE_LOGF(LogDatasmithOpenNurbsTranslator, Warning, "Failed to generate the mesh of \"%ls\" (%ls) StaticMesh.", *StaticMeshLable, *Name);
		}
		return bRet;
	}
	else
	{
		// .. Trying to load the mesh tessellated by Rhino
		ON_Mesh BRepMesh;

		ON_SimpleArray<const ON_Mesh*> RenderMeshes;
		Brep->GetMesh(ON::mesh_type::render_mesh, RenderMeshes);

		// Aborting because there is no mesh associated with the BRep
		if (RenderMeshes.Count() == 0)
		{
			MissingRenderMeshes.Add(MeshElement->GetLabel());
			return false;
		}

		TArray<int32> FaceMaterialChannel;
		bool bHasFaceMaterialChannel = false;
		// Use BRep's face material channel for each of the render meshes
		// Assuming that meshes correspond to brep faces
		if (ensure(RenderMeshes.Count() == Brep->m_F.Count()))
		{
			FaceMaterialChannel.Reserve(Brep->m_F.Count());
			for (int FaceIndex = 0; FaceIndex < Brep->m_F.Count(); ++FaceIndex)
			{
				const ON_BrepFace& Face = Brep->m_F[FaceIndex];
				int Channel = Face.m_face_material_channel;

				if (Channel != 0 && (Channel < MeshElement->GetMaterialSlotCount()))
				{
					bHasFaceMaterialChannel = true;
				}

				FaceMaterialChannel.Add(Channel);
			}
		}
		if (bHasFaceMaterialChannel)
		{
			if (!DatasmithOpenNurbsTranslatorUtils::TranslateMesh(RenderMeshes, RenderMeshes.Count(), OutMesh, bHasNormal, GetScaleFactor(), Offset, bHasFaceMaterialChannel, FaceMaterialChannel.GetData()))
			{
				return false;
			}
		}
		else
		{
			ON_Mesh Mesh;
			Mesh.Append(RenderMeshes.Count(), RenderMeshes);

			if (!DatasmithOpenNurbsTranslatorUtils::TranslateMesh(&Mesh, OutMesh, bHasNormal, GetScaleFactor(), Offset))
			{
				return false;
			}
		}


		return true;
	}
}

bool FOpenNurbsTranslatorImpl::ComputeGeometryCenter(ON_Geometry* Geometry, ON_3dVector& OutCenter)
{
	if (Geometry == nullptr)
	{
		return false;
	}

	double BoxMin[3];
	double BoxMax[3];
	if (!Geometry->GetBBox(BoxMin, BoxMax))
	{
		return false;
	}

	OutCenter = (ON_3dVector(BoxMin[0], BoxMin[1], BoxMin[2]) + ON_3dVector(BoxMax[0], BoxMax[1], BoxMax[2])) * 0.5f;

	return true;
}

bool FOpenNurbsTranslatorImpl::ComputeObjectGeometryCenter(const FOpenNurbsObjectWrapper& Object, ON_3dVector& OutGeometryCenter)
{
	bool bIsValid = false;
	ON_3dVector GeometryCenter;
	GeometryCenter.Zero();
	if (Object.ObjectPtr->IsKindOf(&ON_Mesh::m_ON_Mesh_class_rtti))
	{
		bIsValid = ComputeGeometryCenter(ON_Mesh::Cast(Object.ObjectPtr), GeometryCenter);
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_Brep::m_ON_Brep_class_rtti))
	{
		bIsValid = ComputeGeometryCenter(ON_Brep::Cast(Object.ObjectPtr), GeometryCenter);
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_Extrusion::m_ON_Extrusion_class_rtti))
	{
		const ON_Extrusion* extrusion = ON_Extrusion::Cast(Object.ObjectPtr);
		ON_Brep brep;
		if (extrusion != nullptr && extrusion->BrepForm(&brep) != nullptr)
		{
			bIsValid = ComputeGeometryCenter(&brep, GeometryCenter);
		}
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_Hatch::m_ON_Hatch_class_rtti))
	{
		const ON_Hatch* hatch = ON_Hatch::Cast(Object.ObjectPtr);
		ON_Brep brep;
		if (hatch != nullptr && hatch->BrepForm(&brep) != nullptr)
		{
			bIsValid = ComputeGeometryCenter(&brep, GeometryCenter);
		}
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_PlaneSurface::m_ON_PlaneSurface_class_rtti))
	{
		//
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_LineCurve::m_ON_LineCurve_class_rtti))
	{
		// Not supported
	}

	if (bIsValid)
	{
		OutGeometryCenter = GeometryCenter;
		return true;
	}
	return false;
}

TOptional<FMeshDescription> FOpenNurbsTranslatorImpl::GetMeshDescription(TSharedRef<IDatasmithMeshElement> MeshElement)
{
	// Ref. visitNonInstanceObject (mesh collection part)
	const FOpenNurbsObjectWrapper** ObjectPtr = MeshElementToObjectMap.Find(&MeshElement.Get());
	FOpenNurbsTranslatorImpl* SelectedTranslator = this;
	if (ObjectPtr == nullptr)
	{
		return TOptional< FMeshDescription >();
	}
	else
	{
		FOpenNurbsTranslatorImpl** ChildTranslator = MeshElementToTranslatorMap.Find(&MeshElement.Get());
		if (ChildTranslator != nullptr)
		{
			SelectedTranslator = *ChildTranslator;
		}
	}

	const FOpenNurbsObjectWrapper& Object = **ObjectPtr;

	ON_wString UUIDString;
	ON_UuidToString(Object.Attributes.m_uuid, UUIDString);
	FString UUID(UUIDString.Array());

	FMeshDescription MeshDescription;
	DatasmithMeshHelper::PrepareAttributeForStaticMesh(MeshDescription);

	bool bHasNormal = false;
	bool bIsValid = false;
	if (Object.ObjectPtr->IsKindOf(&ON_Mesh::m_ON_Mesh_class_rtti))
	{
		ON_3dVector Offset = GetGeometryOffset(MeshElement);
		bIsValid = DatasmithOpenNurbsTranslatorUtils::TranslateMesh(ON_Mesh::Cast(Object.ObjectPtr), MeshDescription, bHasNormal, SelectedTranslator->GetScaleFactor(), Offset);
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_Brep::m_ON_Brep_class_rtti))
	{
		bIsValid = SelectedTranslator->TranslateBRep(ON_Brep::Cast(Object.ObjectPtr), Object.Attributes, MeshDescription, MeshElement, UUID, bHasNormal);
	}
	else if (const ON_Extrusion* extrusion = ON_Extrusion::Cast(Object.ObjectPtr))
	{
		if (OpenNurbsOptions.Geometry == EDatasmithOpenNurbsBrepTessellatedSource::UseRenderMeshes)
		{
			ON_3dVector Offset = GetGeometryOffset(MeshElement);
			if (const ON_Mesh* Mesh = extrusion->m_mesh_cache.Mesh(ON::mesh_type::render_mesh))
			{
				if (DatasmithOpenNurbsTranslatorUtils::TranslateMesh(Mesh, MeshDescription, bHasNormal, GetScaleFactor(), Offset))
				{
					return MeshDescription;
				}
			}
			else
			{
				MissingRenderMeshes.Add(MeshElement->GetLabel());
			}
		}

		ON_Brep brep;
		if (extrusion->BrepForm(&brep) != nullptr)
		{
			bIsValid = SelectedTranslator->TranslateBRep(&brep, Object.Attributes, MeshDescription, MeshElement, UUID, bHasNormal);
		}
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_Hatch::m_ON_Hatch_class_rtti))
	{
		const ON_Hatch* hatch = ON_Hatch::Cast(Object.ObjectPtr);
		ON_Brep brep;
		if (hatch != nullptr && hatch->BrepForm(&brep) != nullptr)
		{
			bIsValid = SelectedTranslator->TranslateBRep(&brep, Object.Attributes, MeshDescription, MeshElement, UUID, bHasNormal);
		}
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_PlaneSurface::m_ON_PlaneSurface_class_rtti))
	{
		const ON_PlaneSurface* planeSurface = ON_PlaneSurface::Cast(Object.ObjectPtr);
		ON_Interval xInterval = planeSurface->Extents(0);
		ON_Interval yInterval = planeSurface->Extents(1);

		// TODO: Create simple plane
		// 3 --- 2
		// |     |
		// |     |
		// 0 --- 1

		ON_3dVector vec0 = planeSurface->m_plane.origin + planeSurface->m_plane.xaxis * xInterval.m_t[0] + planeSurface->m_plane.yaxis * yInterval.m_t[0];
		ON_3dVector vec1 = planeSurface->m_plane.origin + planeSurface->m_plane.xaxis * xInterval.m_t[1] + planeSurface->m_plane.yaxis * yInterval.m_t[0];
		ON_3dVector vec2 = planeSurface->m_plane.origin + planeSurface->m_plane.xaxis * xInterval.m_t[1] + planeSurface->m_plane.yaxis * yInterval.m_t[1];
		ON_3dVector vec3 = planeSurface->m_plane.origin + planeSurface->m_plane.xaxis * xInterval.m_t[0] + planeSurface->m_plane.yaxis * yInterval.m_t[1];
	}
	else if (Object.ObjectPtr->IsKindOf(&ON_LineCurve::m_ON_LineCurve_class_rtti))
	{
		// Not supported
	}

	return bIsValid ? MoveTemp(MeshDescription) : TOptional< FMeshDescription >();
}

void FOpenNurbsTranslatorImpl::SetBaseOptions(const FDatasmithImportBaseOptions& InBaseOptions)
{
	BaseOptions = InBaseOptions;
}

void FOpenNurbsTranslatorImpl::SetOpenNurbsOptions(const FDatasmithOpenNurbsOptions& Options)
{
	OpenNurbsOptions = Options;
	OpenNurbsOptionsHash = OpenNurbsOptions.GetHash();

	const CADLibrary::FImportParameters ImportParameters(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded_FBXLegacy);

	if (CADLibrary::FImportParameters::bGDisableCADKernelTessellation)
	{
		TSharedRef<FOpenNurbsBRepToTechSoftConverter> OpenNurbsBRepToTechSoftConverter = MakeShared<FOpenNurbsBRepToTechSoftConverter>(ImportParameters);
		CADModelConverter = OpenNurbsBRepToTechSoftConverter;
		OpenNurbsBRepConverter = OpenNurbsBRepToTechSoftConverter;
	}
	else
	{
		TSharedRef<FOpenNurbsBRepToCADKernelConverter> OpenNurbsBRepToCADKernelConverter = MakeShared<FOpenNurbsBRepToCADKernelConverter>(ImportParameters, Options);
		CADModelConverter = OpenNurbsBRepToCADKernelConverter;
		OpenNurbsBRepConverter = OpenNurbsBRepToCADKernelConverter;
	}

	for (FOpenNurbsTranslatorImpl* ChildTranslator : ChildTranslators)
	{
		ChildTranslator->SetOpenNurbsOptions(OpenNurbsOptions);
	}
}

//////////////////////////////////////////////////////////////////////////
// UDatasmithOpenNurbsTranslator
//////////////////////////////////////////////////////////////////////////

void FDatasmithOpenNurbsTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
{
#if WITH_EDITOR
	if (GIsEditor && !GEditor->PlayWorld && !GIsPlayInEditorWorld)
	{
		OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("3dm"), TEXT("Rhino file format") });
		return;
	}
#endif

	OutCapabilities.bIsEnabled = false;
}

bool FDatasmithOpenNurbsTranslator::LoadScene(TSharedRef<IDatasmithScene> OutScene)
{
	const FString& Filename = GetSource().GetSourceFile();
	FILE* FileHandle = ON::OpenFile(*Filename, L"rb");
	if (!FileHandle)
	{
		return false;
	}

	{
		FString TesselationLibrary;
		if (OpenNurbsOptions.Geometry == EDatasmithOpenNurbsBrepTessellatedSource::UseRenderMeshes)
		{
			TesselationLibrary = TEXT("Rhino");
		}
		else if (CADLibrary::FImportParameters::bGDisableCADKernelTessellation)
		{
			TesselationLibrary = TEXT("TechSoft");
		}
		else
		{
			TesselationLibrary = TEXT("CADKernel");
		}

		UE_LOGF(LogDatasmithOpenNurbsTranslator, Display, "CAD translation [%ls].", *Filename);
		UE_LOGF(LogDatasmithOpenNurbsTranslator, Display, " - Parsing Library:      %ls", TEXT("OpenNurbs"));
		UE_LOGF(LogDatasmithOpenNurbsTranslator, Display, " - Tessellation Library: %ls", *TesselationLibrary);
	}

	Translator = MakeShared<FOpenNurbsTranslatorImpl>(GetSource().GetSceneName(), OutScene, FPaths::GetPath(Filename), nullptr);
	if (!Translator)
	{
		return false;
	}

	FString OutputPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FDatasmithOpenNurbsTranslatorModule::Get().GetTempDir(), TEXT("Cache"), GetSource().GetSceneName()));
	IFileManager::Get().MakeDirectory(*OutputPath, true);
	Translator->SetOutputPath(OutputPath);
	Translator->SetOpenNurbsOptions(OpenNurbsOptions);
	Translator->SetBaseOptions(BaseOptions);

	ON_BinaryFile Archive(ON::archive_mode::read3dm, FileHandle);

	bool bResult = Translator->Read(Archive, OutScene);

	ON::CloseFile(FileHandle);

	return bResult;
}

void FDatasmithOpenNurbsTranslator::UnloadScene()
{
	if (Translator)
	{
		Translator->ShowMessageLog(GetSource().GetSourceFile());
		Translator.Reset();
	}
}

bool FDatasmithOpenNurbsTranslator::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	if (Translator)
	{
		return Translator->LoadStaticMesh(MeshElement, OutMeshPayload, GetCommonTessellationOptions());
	}
	return false;
}

bool FOpenNurbsTranslatorImpl::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload, const FDatasmithTessellationOptions& InTessellationOptions)
{
	if (TOptional<FMeshDescription> Mesh = GetMeshDescription(MeshElement))
	{
		CADLibrary::FMeshParameters MeshParameters;
		OutMeshPayload.LodMeshes.Add(MoveTemp(Mesh.GetValue()));
		CADModelConverter->AddSurfaceDataForMesh(MeshElement->GetFile(), MeshParameters, InTessellationOptions, OutMeshPayload);
	}
	return OutMeshPayload.LodMeshes.Num() > 0;
}

void FDatasmithOpenNurbsTranslator::SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	for (const TObjectPtr<UDatasmithOptionsBase>& Option : Options)
	{
		if (UDatasmithImportOptions* DatasmithOptions = Cast<UDatasmithImportOptions>(Option))
		{
			BaseOptions = DatasmithOptions->BaseOptions;
		}
		else if (UDatasmithOpenNurbsImportOptions* OpenNurbsOptionsObj = Cast<UDatasmithOpenNurbsImportOptions>(Option))
		{
			OpenNurbsOptions = OpenNurbsOptionsObj->Options;
			OpenNurbsOptionsObj->SaveConfig(CPF_Config);
		}
	}

	if (Translator)
	{
		Translator->SetOpenNurbsOptions(OpenNurbsOptions);
		Translator->SetBaseOptions(BaseOptions);
	}
}

void FDatasmithOpenNurbsTranslator::GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	TObjectPtr<UDatasmithOpenNurbsImportOptions> OpenNurbsOptionsPtr = Datasmith::MakeOptionsObjectPtr<UDatasmithOpenNurbsImportOptions>();
	if (ICADInterfacesModule::GetAvailability() == ECADInterfaceAvailability::Available)
	{
		OpenNurbsOptionsPtr->Options.Geometry = EDatasmithOpenNurbsBrepTessellatedSource::UseUnrealNurbsTessellation;
	}
	else
	{
		// #ue_opennurbs: Disable Geometry property in UI
	}

	Options.Add(OpenNurbsOptionsPtr);
}

#undef LOCTEXT_NAMESPACE // "DatasmithOpenNurbsTranslator"

#endif // USE_OPENNURBS
