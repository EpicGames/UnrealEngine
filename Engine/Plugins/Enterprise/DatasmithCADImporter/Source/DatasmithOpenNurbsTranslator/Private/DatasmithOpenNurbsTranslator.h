// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithTranslator.h"
#include "DatasmithOpenNurbsImportOptions.h"
#include "ParametricSurfaceTranslator.h"

#include "CoreMinimal.h"

class FOpenNurbsTranslatorImpl;

class FDatasmithOpenNurbsTranslator : public FParametricSurfaceTranslator
{
public:
	virtual FName GetFName() const override { return "DatasmithOpenNurbsTranslator"; }

#ifndef USE_OPENNURBS
	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override { OutCapabilities.bIsEnabled = false; }
#else // USE_OPENNURBS

	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;

	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;

	virtual void UnloadScene() override;

	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) override;

	virtual void SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;

	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;

private:
	TSharedPtr<FOpenNurbsTranslatorImpl> Translator;

	// Temporarily store this here for UE-81278 so that we can trigger the recreation of
	// static meshes if we're reimporting with new materials that haven't been assigned yet
	FDatasmithImportBaseOptions BaseOptions;

	FDatasmithOpenNurbsOptions OpenNurbsOptions;
#endif // USE_OPENNURBS
};

