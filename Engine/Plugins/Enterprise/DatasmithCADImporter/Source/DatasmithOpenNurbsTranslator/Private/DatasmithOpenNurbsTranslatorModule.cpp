// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithOpenNurbsTranslatorModule.h"

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "DatasmithOpenNurbsTranslator.h"

IMPLEMENT_MODULE(FDatasmithOpenNurbsTranslatorModule, DatasmithOpenNurbsTranslator);

void FDatasmithOpenNurbsTranslatorModule::StartupModule()
{
	// Create temporary directory which will be used by CoreTech to store tessellation data
	TempDir = FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("OpenNurbsImportTemp"));
	IFileManager::Get().MakeDirectory(*TempDir);

	Datasmith::RegisterTranslator<FDatasmithOpenNurbsTranslator>();
}

void FDatasmithOpenNurbsTranslatorModule::ShutdownModule()
{
	Datasmith::UnregisterTranslator<FDatasmithOpenNurbsTranslator>();
}

FString FDatasmithOpenNurbsTranslatorModule::GetTempDir() const
{
	return TempDir;
}
