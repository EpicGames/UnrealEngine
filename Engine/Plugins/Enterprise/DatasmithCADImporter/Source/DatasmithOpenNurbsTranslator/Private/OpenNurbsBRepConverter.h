// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathUtility.h"

class ON_Brep;
class ON_3dVector;

class IOpenNurbsBRepConverter
{
public:
	virtual bool AddBRep(ON_Brep& Brep, const ON_3dVector& Offset) = 0;
	void SetScaleFactor(double NewScaleFactor)
	{
		ensure(!FMath::IsNearlyZero(NewScaleFactor));
		ScaleFactor = NewScaleFactor;
	}

protected:
	double ScaleFactor = 1; // mm to mm 
};

