// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenNurbsBRepToCADKernelConverter.h"

#ifdef USE_OPENNURBS

#pragma warning(push)
#pragma warning(disable:4265)
#pragma warning(disable:4005) // TEXT macro redefinition
#include "opennurbs.h"
#pragma warning(pop)

#include "CADKernelTools.h"

#include "Core/Session.h"
#include "Geo/Curves/Curve.h"
#include "Geo/Curves/NURBSCurveData.h"
#include "Geo/Surfaces/Surface.h"
#include "Geo/Surfaces/NurbsSurfaceData.h"

#include "Math/Boundary.h"
#include "Math/Point.h"
#include "Mesh/Structure/ModelMesh.h"

#include "Topo/Body.h"
#include "Topo/Model.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalLoop.h"


namespace OpenNurbsBRepConverterTool
{
	enum EAxis { U, V };

	void FillPerAxisInfo(EAxis Axis, ON_NurbsSurface& OpenNurbsSurface, UE::CADKernel::FNurbsSurfaceHomogeneousData& OutNurbsInfo)
	{
		int32& Degree = Axis == EAxis::U ? OutNurbsInfo.UDegree : OutNurbsInfo.VDegree;
		Degree = OpenNurbsSurface.Order(Axis) - 1;

		int32& CtrlVertCount = Axis == EAxis::U ? OutNurbsInfo.PoleUCount : OutNurbsInfo.PoleVCount; // number of control points
		CtrlVertCount = OpenNurbsSurface.CVCount(Axis);

		uint32 KnotSize = Degree + CtrlVertCount + 1;

		// detect cases not handled by CADKernel, that is knot vectors with multiplicity < order on either end
		if (OpenNurbsSurface.KnotMultiplicity(Axis, 0) < Degree || OpenNurbsSurface.KnotMultiplicity(Axis, KnotSize - 3) < Degree)
		{
			OpenNurbsSurface.IncreaseDegree(Axis, OpenNurbsSurface.Degree(Axis) + 1);
			Degree = OpenNurbsSurface.Order(Axis) - 1;
			KnotSize = Degree + CtrlVertCount + 1;
		}

		TArray<double>& Knots = Axis == EAxis::U ? OutNurbsInfo.UNodalVector : OutNurbsInfo.VNodalVector; // t values with superfluous values
		Knots.Reserve(KnotSize);
		Knots.Add(OpenNurbsSurface.SuperfluousKnot(Axis, 0));
		uint32 KnotCount = OpenNurbsSurface.KnotCount(Axis);
		for (uint32 i = 0; i < KnotCount; ++i)
		{
			Knots.Add(OpenNurbsSurface.Knot(Axis, i));
		}
		Knots.Add(OpenNurbsSurface.SuperfluousKnot(Axis, 1));
	}
}

TSharedPtr<UE::CADKernel::FSurface> FOpenNurbsBRepToCADKernelConverter::AddSurface(ON_NurbsSurface& OpenNurbsSurface)
{
	UE::CADKernel::FNurbsSurfaceHomogeneousData NurbsData;
	OpenNurbsBRepConverterTool::FillPerAxisInfo(OpenNurbsBRepConverterTool::U, OpenNurbsSurface, NurbsData);
	OpenNurbsBRepConverterTool::FillPerAxisInfo(OpenNurbsBRepConverterTool::V, OpenNurbsSurface, NurbsData);

	{
		int32 ControlVertexDimension = OpenNurbsSurface.CVSize();
		NurbsData.HomogeneousPoles.SetNumUninitialized(NurbsData.PoleUCount * NurbsData.PoleVCount * ControlVertexDimension);
		double* ControlPoints = NurbsData.HomogeneousPoles.GetData();
		NurbsData.bIsRational = OpenNurbsSurface.IsRational();
		ON::point_style PointStyle = OpenNurbsSurface.IsRational() ? ON::point_style::euclidean_rational : ON::point_style::not_rational;
		for (int32 UIndex = 0; UIndex < NurbsData.PoleUCount; ++UIndex)
		{
			for (int32 VIndex = 0; VIndex < NurbsData.PoleVCount; ++VIndex, ControlPoints += ControlVertexDimension)
			{
				OpenNurbsSurface.GetCV(UIndex, VIndex, PointStyle, ControlPoints);
			}
		}
	}

	// Scale ControlPoints into mm
	{
		int32 Offset = NurbsData.bIsRational ? 4 : 3;
		double* ControlPoints = NurbsData.HomogeneousPoles.GetData();
		for (int32 Index = 0; Index < NurbsData.HomogeneousPoles.Num(); Index += Offset)
		{
			ControlPoints[Index + 0] *= ScaleFactor;
			ControlPoints[Index + 1] *= ScaleFactor;
			ControlPoints[Index + 2] *= ScaleFactor;
		}
	}

	return UE::CADKernel::FSurface::MakeNurbsSurface(GeometricTolerance, NurbsData);
}

TSharedPtr<UE::CADKernel::FTopologicalLoop> FOpenNurbsBRepToCADKernelConverter::AddLoop(const ON_BrepLoop& OpenNurbsLoop, TSharedPtr<UE::CADKernel::FSurface> & CarrierSurface, const bool bIsExternal)
{
	using namespace UE::CADKernel;

	if (!OpenNurbsLoop.IsValid())
	{
		return TSharedPtr<FTopologicalLoop>();
	}

	ON_BrepLoop::TYPE LoopType = OpenNurbsLoop.m_type;
	bool bIsOuter = (LoopType == ON_BrepLoop::TYPE::outer);

	int32 EdgeCount = OpenNurbsLoop.TrimCount();
	TArray<TSharedPtr<FTopologicalEdge>> Edges;
	TArray<UE::CADKernel::EOrientation> Directions;

	Edges.Reserve(EdgeCount);
	Directions.Reserve(EdgeCount);

	for (int32 Index = 0; Index < EdgeCount; ++Index)
	{
		ON_BrepTrim& Trim = *OpenNurbsLoop.Trim(Index);

		TSharedPtr<FTopologicalEdge> Edge = AddEdge(Trim, CarrierSurface);
		if (Edge.IsValid())
		{
			Edges.Add(Edge);
			Directions.Emplace(UE::CADKernel::EOrientation::Front);
		}
	}

	if (Edges.Num() == 0)
	{
		return TSharedPtr<FTopologicalLoop>();
	}

	return FTopologicalLoop::Make(Edges, Directions, bIsExternal, GeometricTolerance);
}

void FOpenNurbsBRepToCADKernelConverter::LinkEdgesLoop(const ON_BrepLoop& OpenNurbsLoop, UE::CADKernel::FTopologicalLoop& Loop)
{
	int32 EdgeCount = OpenNurbsLoop.TrimCount();
	for (int32 Index = 0; Index < EdgeCount; ++Index)
	{
		ON_BrepTrim& OpenNurbsTrim = *OpenNurbsLoop.Trim(Index);
		ON_BrepEdge* OpenNurbsEdge = OpenNurbsTrim.Edge();
		if (OpenNurbsEdge == nullptr)
		{
			continue;
		}

		TSharedPtr<UE::CADKernel::FTopologicalEdge>* Edge = OpenNurbsTrimId2CADKernelEdge.Find(OpenNurbsTrim.m_trim_index);
		if (!Edge || !Edge->IsValid() || (*Edge)->IsDeleted() || (*Edge)->IsDegenerated())
		{
			continue;
		}

		for (int32 Endex = 0; Endex < OpenNurbsEdge->m_ti.Count(); ++Endex)
		{
			int32 LinkedEdgeId = OpenNurbsEdge->m_ti[Endex];
			if (LinkedEdgeId == OpenNurbsTrim.m_trim_index)
			{
				continue;
			}

			TSharedPtr<UE::CADKernel::FTopologicalEdge>* TwinEdge = OpenNurbsTrimId2CADKernelEdge.Find(LinkedEdgeId);
			if (TwinEdge != nullptr && TwinEdge->IsValid() && !(*TwinEdge)->IsDeleted() && !(*TwinEdge)->IsDegenerated())
			{
				(*Edge)->LinkIfCoincident(**TwinEdge, EdgeLengthTolerance, SquareTolerance);
				break;
			}
		}
	}
}

TSharedPtr<UE::CADKernel::FTopologicalEdge> FOpenNurbsBRepToCADKernelConverter::AddEdge(const ON_BrepTrim& OpenNurbsTrim, TSharedPtr<UE::CADKernel::FSurface>& CarrierSurface)
{
	using namespace UE::CADKernel;

	ON_BrepEdge* OpenNurbsEdge = OpenNurbsTrim.Edge();
	if (OpenNurbsEdge == nullptr)
	{
		return TSharedPtr<FTopologicalEdge>();
	}

	ON_NurbsCurve OpenNurbsCurve;
	int32 NurbFormSuccess = OpenNurbsTrim.GetNurbForm(OpenNurbsCurve); // 0:No ok 1:Ok 2:OkBut
	if (NurbFormSuccess == 0)
	{
		return TSharedPtr<FTopologicalEdge>();
	}

	FNurbsCurveData NurbsCurveData;

	NurbsCurveData.Dimension = 2;
	NurbsCurveData.Degree = OpenNurbsCurve.Order() - 1;

	int32 KnotCount = OpenNurbsCurve.KnotCount();
	int32 ControlPointCount = OpenNurbsCurve.CVCount();

	NurbsCurveData.NodalVector.Reserve(KnotCount + 2);
	NurbsCurveData.NodalVector.Emplace(OpenNurbsCurve.SuperfluousKnot(0));
	for (int32 Index = 0; Index < KnotCount; ++Index)
	{
		NurbsCurveData.NodalVector.Emplace(OpenNurbsCurve.Knot(Index));
	}
	NurbsCurveData.NodalVector.Emplace(OpenNurbsCurve.SuperfluousKnot(1));

	NurbsCurveData.bIsRational = OpenNurbsCurve.IsRational();

	NurbsCurveData.Poles.SetNumUninitialized(ControlPointCount);

	double* ControlPoints = (double*) NurbsCurveData.Poles.GetData();
	for (int32 Index = 0; Index < ControlPointCount; ++Index, ControlPoints += 3)
	{
		OpenNurbsCurve.GetCV(Index, OpenNurbsCurve.IsRational() ? ON::point_style::euclidean_rational : ON::point_style::not_rational, ControlPoints);
	}

	if(NurbsCurveData.bIsRational)
	{
		NurbsCurveData.Weights.SetNumUninitialized(ControlPointCount);
		for (int32 Index = 0; Index < ControlPointCount; ++Index)
		{
			NurbsCurveData.Weights[Index] = NurbsCurveData.Poles[Index].Z;
		}
	}

	for (int32 Index = 0; Index < ControlPointCount; ++Index)
	{
		NurbsCurveData.Poles[Index].Z = 0;
	}

	TSharedPtr<FCurve> Nurbs = FCurve::MakeNurbsCurve(NurbsCurveData);

	TSharedRef<FRestrictionCurve> RestrictionCurve = FEntity::MakeShared<FRestrictionCurve>(CarrierSurface.ToSharedRef(), Nurbs.ToSharedRef());

	ON_Interval dom = OpenNurbsCurve.Domain();
	FLinearBoundary Boundary(dom.m_t[0], dom.m_t[1]);
	TSharedPtr<FTopologicalEdge> Edge = FTopologicalEdge::Make(RestrictionCurve, Boundary);
	if (!Edge.IsValid())
	{
		return TSharedPtr<FTopologicalEdge>();
	}


	// Only Edge with twin need to be in the map used in LinkEdgesLoop 
	if(OpenNurbsEdge->m_ti.Count() > 1)
	{
		OpenNurbsTrimId2CADKernelEdge.Add(OpenNurbsTrim.m_trim_index, Edge);
	}

	return Edge;
}

TSharedPtr<UE::CADKernel::FTopologicalFace> FOpenNurbsBRepToCADKernelConverter::AddFace(const ON_BrepFace& OpenNurbsFace)
{
	using namespace UE::CADKernel;

	ON_NurbsSurface OpenNurbsSurface;
	OpenNurbsFace.NurbsSurface(&OpenNurbsSurface);

	TSharedPtr<FSurface> Surface = AddSurface(OpenNurbsSurface);
	if (!Surface.IsValid())
	{
		return TSharedPtr<FTopologicalFace>();
	}

	TSharedRef<FTopologicalFace> Face = FEntity::MakeShared<FTopologicalFace>(Surface);

	const ON_BrepLoop* OuterLoop = OpenNurbsFace.OuterLoop();
	if (OuterLoop == nullptr)
	{
		Face->ApplyNaturalLoops();
		return Face;
	}

	bool bIsExternal = true;
	int32 LoopCount = OpenNurbsFace.LoopCount();
	for (int32 LoopIndex = 0; LoopIndex < LoopCount; ++LoopIndex)
	{
		const ON_BrepLoop& OpenNurbsLoop = *OpenNurbsFace.Loop(LoopIndex);
		TSharedPtr<FTopologicalLoop> Loop = AddLoop(OpenNurbsLoop, Surface, bIsExternal);
		if(Loop.IsValid())
		{
			LinkEdgesLoop(OpenNurbsLoop, *Loop);
			Face->AddLoop(Loop);
			bIsExternal = false;
		}
	}

	if (Face->GetLoops().Num() == 0)
	{
		Face->SetAsDegenerated();
		Face->Delete();

		return TSharedPtr<FTopologicalFace>();
	}

	return Face;
}

bool FOpenNurbsBRepToCADKernelConverter::AddBRep(ON_Brep& BRep, const ON_3dVector& Offset)
{
	using namespace UE::CADKernel;

	OpenNurbsTrimId2CADKernelEdge.Empty();

	TSharedRef<FBody> Body = FEntity::MakeShared<FBody>();
	TSharedRef<FShell> Shell = FEntity::MakeShared<FShell>();
	Body->AddShell(Shell);

	BRep.Translate(Offset);

	BRep.FlipReversedSurfaces();

	// Create faces
	int32 FaceCount = BRep.m_F.Count();
	for (int32 index = 0; index < FaceCount; index++)
	{
		const ON_BrepFace& OpenNurbsFace = BRep.m_F[index];
		TSharedPtr<FTopologicalFace> Face = AddFace(OpenNurbsFace);
		if (Face.IsValid())
		{
			Shell->Add(Face.ToSharedRef(), UE::CADKernel::EOrientation::Front);
		}
	}

	BRep.Translate(-Offset);

	CADKernelSession.GetModel().Add(Body);

	return true;
}

#endif // defined(USE_OPENNURBS)
