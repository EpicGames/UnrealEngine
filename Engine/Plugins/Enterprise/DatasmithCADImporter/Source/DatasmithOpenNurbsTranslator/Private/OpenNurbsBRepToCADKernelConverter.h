// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#ifdef USE_OPENNURBS

#include "CADModelToCADKernelConverterBase.h"
#include "OpenNurbsBRepConverter.h"

#include "Core/Session.h"
#include "Core/Types.h"

class ON_BoundingBox;
class ON_Brep;
class ON_BrepFace;
class ON_BrepLoop;
class ON_BrepTrim;
class ON_NurbsSurface;
class ON_3dVector;

namespace UE::CADKernel
{
class FShell;
class FSurface;
class FTopologicalEdge;
class FTopologicalFace;
class FTopologicalLoop;
}

class FOpenNurbsBRepToCADKernelConverter : public FCADModelToCADKernelConverterBase, public IOpenNurbsBRepConverter
{
public:

	FOpenNurbsBRepToCADKernelConverter(const CADLibrary::FImportParameters& InImportParameters, const FDatasmithTessellationOptions& Options)
		: FCADModelToCADKernelConverterBase(InImportParameters)
	{
		SetTolerances(Options.GetGeometricTolerance(true), Options.GetStitchingTolerance(true));
	}

	virtual ~FOpenNurbsBRepToCADKernelConverter() = default;

	/**
	 * Set BRep to tessellate, offsetting it prior to tessellation(used to set mesh pivot at the center of the surface bounding box)
	 *
	 * @param  Brep	a BRep to tessellate
	 * @param  Offset translate BRep by this value before tessellating
	 */
	bool AddBRep(ON_Brep& Brep, const ON_3dVector& Offset);

private:
	TSharedPtr<UE::CADKernel::FTopologicalFace> AddFace(const ON_BrepFace& OpenNurbsFace);
	TSharedPtr<UE::CADKernel::FSurface> AddSurface(ON_NurbsSurface& Surface);

	TSharedPtr<UE::CADKernel::FTopologicalLoop> AddLoop(const ON_BrepLoop& OpenNurbsLoop, TSharedPtr<UE::CADKernel::FSurface>& CarrierSurface, const bool bIsExternal);

	/**
	 * Build face's links with its neighbor have to be done after the loop is finalize.
	 * This is to avoid to link an edge with another and then to delete it...
	 */
	void LinkEdgesLoop(const ON_BrepLoop& OpenNurbsLoop, UE::CADKernel::FTopologicalLoop& Loop);

	TSharedPtr<UE::CADKernel::FTopologicalEdge> AddEdge(const ON_BrepTrim& OpenNurbsTrim, TSharedPtr<UE::CADKernel::FSurface>& CarrierSurface);

protected:

	TMap<int32, TSharedPtr<UE::CADKernel::FTopologicalEdge>>  OpenNurbsTrimId2CADKernelEdge;

};

#endif // USE_OPENNURBS