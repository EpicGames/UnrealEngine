// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenNurbsBRepToTechSoftConverter.h"

#ifdef USE_OPENNURBS

#pragma warning(push)
#pragma warning(disable:4265)
#pragma warning(disable:4005) // TEXT macro redefinition
#include "opennurbs.h"
#pragma warning(pop)

#include "TechSoftInterface.h"
#include "TechSoftUtils.h"
#include "TUniqueTechSoftObj.h"

#ifdef USE_TECHSOFT_SDK
namespace OpenNurbsUtils
{
enum EAxis { U, V };

struct FOpenNurbsSurfaceInfo
{
	FOpenNurbsSurfaceInfo(const ON_NurbsSurface& InOpenNurbsSurface, A3DSurfNurbsData& InData, const double InScale)
		: OpenNurbsSurface(InOpenNurbsSurface)
		, Data(InData)
		, Scale(InScale)
	{
	}

	const ON_NurbsSurface& OpenNurbsSurface;
	A3DSurfNurbsData& Data;
	const double Scale;

	TArray<A3DDouble> UNodalVector;
	TArray<A3DDouble> VNodalVector;

	TArray<A3DDouble> Weights;
	TArray<A3DVector3dData> ControlPoints;

	template<typename PointType>
	void SetA3DVector(const PointType& OpenNurbsPoint, A3DVector3dData& OutTSPoint)
	{
		OutTSPoint.m_dX = OpenNurbsPoint.x * Scale;
		OutTSPoint.m_dY = OpenNurbsPoint.y * Scale;
		OutTSPoint.m_dZ = OpenNurbsPoint.z * Scale;
	};

	template<typename PointType>
	void SetA3DVectorAndWeight(const PointType& OpenNurbsPoint, A3DVector3dData& OutTSPoint, A3DDouble& OutWeight)
	{
		SetA3DVector(OpenNurbsPoint, OutTSPoint);
		OutWeight = OpenNurbsPoint.w;
	};

	void FillPerAxisInfo(EAxis Axis)
	{
		A3DUns32& Degree = Axis == EAxis::U ? Data.m_uiUDegree : Data.m_uiVDegree;
		Degree = OpenNurbsSurface.Order(Axis) - 1;

		A3DUns32& ControlPointCount = Axis == EAxis::U ? Data.m_uiUCtrlSize : Data.m_uiVCtrlSize;
		ControlPointCount = OpenNurbsSurface.CVCount(Axis);

		A3DUns32& KnotSize = Axis == EAxis::U ? Data.m_uiUKnotSize : Data.m_uiVKnotSize;
		KnotSize = Degree + ControlPointCount + 1;

		TArray<A3DDouble>& NodalVector = Axis == EAxis::U ? UNodalVector : VNodalVector;
		NodalVector.Reserve(KnotSize);
		NodalVector.Add(OpenNurbsSurface.Knot(Axis, 0));
		uint32 KnotCount = OpenNurbsSurface.KnotCount(Axis);
		for (uint32 i = 0; i < KnotCount; ++i)
		{
			NodalVector.Add(OpenNurbsSurface.Knot(Axis, i));
		}
		double LastValue = NodalVector[KnotCount];
		NodalVector.Add(LastValue);

		A3DDouble*& UKnots = Axis == EAxis::U ? Data.m_pdUKnots : Data.m_pdVKnots;
		UKnots = NodalVector.GetData();
	}

	void BuildControlPoints()
	{
		bool bIsRational = OpenNurbsSurface.IsRational();

		ControlPoints.SetNum(Data.m_uiUCtrlSize * Data.m_uiVCtrlSize);
		A3DVector3dData* ControlPointPtr = ControlPoints.GetData();
		Data.m_pCtrlPts = ControlPointPtr;

		if(bIsRational)
		{
			Weights.SetNum(Data.m_uiUCtrlSize * Data.m_uiVCtrlSize);
			A3DDouble* WeightPtr = Weights.GetData();
			Data.m_pdWeights = WeightPtr;

			ON_4dPoint OpenNurbsPoint;
			for (A3DUns32 UIndex = 0; UIndex < Data.m_uiUCtrlSize; ++UIndex)
			{
				for (A3DUns32 VIndex = 0; VIndex < Data.m_uiVCtrlSize; ++VIndex, ++ControlPointPtr, ++WeightPtr)
				{
					OpenNurbsSurface.GetCV(UIndex, VIndex, ON::point_style::euclidean_rational, OpenNurbsPoint);
					SetA3DVectorAndWeight(OpenNurbsPoint, *ControlPointPtr, *WeightPtr);
				}
			}
		}
		else
		{
			ON_3dPoint OpenNurbsPoint;
			for (A3DUns32 UIndex = 0; UIndex < Data.m_uiUCtrlSize; ++UIndex)
			{
				for (A3DUns32 VIndex = 0; VIndex < Data.m_uiVCtrlSize; ++VIndex, ++ControlPointPtr)
				{
					OpenNurbsSurface.GetCV(UIndex, VIndex, ON::point_style::not_rational, OpenNurbsPoint);
					SetA3DVector(OpenNurbsPoint, *ControlPointPtr);
				}
			}
		}
	}

	void Populate()
	{
		Data.m_eKnotType = A3DEKnotType::kA3DKnotTypeUnspecified;
		Data.m_eSurfaceForm = A3DEBSplineSurfaceForm::kA3DBSplineSurfaceFormUnspecified;

		FillPerAxisInfo(U);
		FillPerAxisInfo(V);

		BuildControlPoints();
	}
};

} // NS

A3DSurfBase* FOpenNurbsBRepToTechSoftConverter::CreateSurface(const ON_NurbsSurface& OpenNurbsSurface)
{
	CADLibrary::TUniqueTSObj<A3DSurfNurbsData> NurbsSurfaceData;

	OpenNurbsUtils::FOpenNurbsSurfaceInfo NurbsInfo(OpenNurbsSurface, *NurbsSurfaceData, ScaleFactor);
	NurbsInfo.Populate();

	return CADLibrary::TechSoftInterface::CreateSurfaceNurbs(*NurbsSurfaceData);
}

A3DCrvBase* FOpenNurbsBRepToTechSoftConverter::CreateCurve(const ON_NurbsCurve& OpenNurbsCurve)
{
	CADLibrary::TUniqueTSObj<A3DCrvNurbsData> NurbsCurveData;

	NurbsCurveData->m_bIs2D = true;
	NurbsCurveData->m_bRational = OpenNurbsCurve.IsRational();
	NurbsCurveData->m_uiDegree = OpenNurbsCurve.Order() - 1;


	NurbsCurveData->m_eKnotType = kA3DKnotTypeUnspecified;
	NurbsCurveData->m_eCurveForm = kA3DBSplineCurveFormUnspecified;

	int32 KnotCount = OpenNurbsCurve.KnotCount();
	TArray<A3DDouble> NodalVector;
	NodalVector.Reserve(KnotCount + 2);
	NodalVector.Emplace(OpenNurbsCurve.Knot(0));
	for (int Index = 0; Index < KnotCount; ++Index)
	{
		NodalVector.Emplace(OpenNurbsCurve.Knot(Index));
	}
	double LastValue = NodalVector[KnotCount];
	NodalVector.Emplace(LastValue);

	TArray<A3DVector3dData> ControlPointArray;
	TArray<A3DDouble> WeightArray;

	int32 ControlPointCount = OpenNurbsCurve.CVCount();

	ControlPointArray.SetNumUninitialized(ControlPointCount);
	A3DVector3dData* ControlPoints = ControlPointArray.GetData();
	ON_3dPoint OpenNurbsPoint;
	if (OpenNurbsCurve.IsRational())
	{
		WeightArray.SetNumUninitialized(ControlPointCount);
		A3DDouble* Weights = WeightArray.GetData();
		for (int32 Index = 0; Index < ControlPointCount; ++Index, ++ControlPoints, ++Weights)
		{
			OpenNurbsCurve.GetCV(Index, ON::point_style::euclidean_rational, OpenNurbsPoint);
			ControlPoints->m_dX = OpenNurbsPoint.x;
			ControlPoints->m_dY = OpenNurbsPoint.y;
			ControlPoints->m_dZ = 0;
			*Weights = OpenNurbsPoint.z;
		}
	}
	else
	{
		for (int32 Index = 0; Index < ControlPointCount; ++Index, ++ControlPoints)
		{
			OpenNurbsCurve.GetCV(Index, ON::point_style::not_rational, OpenNurbsPoint);
			ControlPoints->m_dX = OpenNurbsPoint.x;
			ControlPoints->m_dY = OpenNurbsPoint.y;
			ControlPoints->m_dZ = 0;
		}
	}

	NurbsCurveData->m_pCtrlPts = ControlPointArray.GetData();
	NurbsCurveData->m_uiCtrlSize = ControlPointArray.Num();

	NurbsCurveData->m_pdWeights = NurbsCurveData->m_bRational ? WeightArray.GetData() : nullptr;
	NurbsCurveData->m_uiWeightSize = NurbsCurveData->m_bRational ? WeightArray.Num() : 0;

	NurbsCurveData->m_pdKnots = NodalVector.GetData();
	NurbsCurveData->m_uiKnotSize = NodalVector.Num();
	
	return CADLibrary::TechSoftInterface::CreateCurveNurbs(*NurbsCurveData);
}

A3DTopoLoop* FOpenNurbsBRepToTechSoftConverter::CreateTopoLoop(const ON_BrepLoop& OpenNurbsLoop)
{
	if (!OpenNurbsLoop.IsValid())
	{
		return nullptr;
	}

	int32 EdgeCount = OpenNurbsLoop.TrimCount();
	TArray<A3DTopoCoEdge*> CoEdges;
	CoEdges.Reserve(EdgeCount);

	for (int32 Index = 0; Index < EdgeCount; ++Index)
	{
		ON_BrepTrim& Trim = *OpenNurbsLoop.Trim(Index);

		A3DTopoCoEdge* CoEdge = CreateTopoCoEdge(Trim);
		if (CoEdge != nullptr)
		{
			CoEdges.Add(CoEdge);
		}
	}

	if (CoEdges.Num() == 0)
	{
		return nullptr;
	}

	CADLibrary::TUniqueTSObj<A3DTopoLoopData> LoopData;

	LoopData->m_ppCoEdges = CoEdges.GetData();
	LoopData->m_uiCoEdgeSize = CoEdges.Num();
	LoopData->m_ucOrientationWithSurface = 1;

	return CADLibrary::TechSoftInterface::CreateTopoLoop(*LoopData);
}

void FOpenNurbsBRepToTechSoftConverter::LinkEdgesLoop(const ON_BrepLoop& OpenNurbsLoop)
{
	int32 EdgeCount = OpenNurbsLoop.TrimCount();
	for (int32 Index = 0; Index < EdgeCount; ++Index)
	{
		ON_BrepTrim& OpenNurbsTrim = *OpenNurbsLoop.Trim(Index);
		ON_BrepEdge* OpenNurbsEdge = OpenNurbsTrim.Edge();
		if (OpenNurbsEdge == nullptr)
		{
			continue;
		}

		A3DTopoCoEdge** CoEdge = OpenNurbsTrimId2TechSoftCoEdge.Find(OpenNurbsTrim.m_trim_index);
		if (CoEdge == nullptr)
		{
			continue;
		}

		for (int32 Endex = 0; Endex < OpenNurbsEdge->m_ti.Count(); ++Endex)
		{
			int32 LinkedEdgeId = OpenNurbsEdge->m_ti[Endex];
			if (LinkedEdgeId == OpenNurbsTrim.m_trim_index)
			{
				continue;
			}

			A3DTopoCoEdge** TwinCoEdge = OpenNurbsTrimId2TechSoftCoEdge.Find(LinkedEdgeId);
			if (TwinCoEdge != nullptr)
			{
				CADLibrary::TechSoftInterface::LinkCoEdges(*CoEdge, *TwinCoEdge);
				break;
			}
		}
	}
}

A3DTopoCoEdge* FOpenNurbsBRepToTechSoftConverter::CreateTopoCoEdge(ON_BrepTrim& OpenNurbsTrim)
{
	ON_NurbsCurve OpenNurbsCurve;
	int NurbFormSuccess = OpenNurbsTrim.GetNurbForm(OpenNurbsCurve); // 0:Nok 1:Ok 2:OkBut
	if (NurbFormSuccess == 0)
	{
		return nullptr;
	}

	A3DCrvBase* NurbsCurvePtr = CreateCurve(OpenNurbsCurve);
	if (NurbsCurvePtr == nullptr)
	{
		return nullptr;
	}

	ON_Interval dom = OpenNurbsCurve.Domain();
	if (!FMath::IsNearlyEqual(OpenNurbsCurve.SuperfluousKnot(0), dom.m_t[0]) ||
		!FMath::IsNearlyEqual(OpenNurbsCurve.SuperfluousKnot(1), dom.m_t[1]))
	{
		NurbsCurvePtr = CADLibrary::TechSoftUtils::CreateTrimNurbsCurve(NurbsCurvePtr, dom.m_t[0], dom.m_t[1], true);
		if (NurbsCurvePtr == nullptr)
		{
			return nullptr;
		}
	}

	A3DTopoEdge* EdgePtr = CADLibrary::TechSoftUtils::CreateTopoEdge();
	if (EdgePtr == nullptr)
	{
		return nullptr;
	}

	CADLibrary::TUniqueTSObj<A3DTopoCoEdgeData> CoEdgeData;
	CoEdgeData->m_pUVCurve = NurbsCurvePtr;
	CoEdgeData->m_pEdge = EdgePtr;
	CoEdgeData->m_ucOrientationWithLoop = OpenNurbsTrim.m_bRev3d ? 0 : 1;
	CoEdgeData->m_ucOrientationUVWithLoop = 1;

	A3DTopoCoEdge* CoEdgePtr = CADLibrary::TechSoftInterface::CreateTopoCoEdge(*CoEdgeData);

	// Only Edge with twin need to be in the map used in LinkEdgesLoop 
	ON_BrepEdge* OpenNurbsEdge = OpenNurbsTrim.Edge();
	if (CoEdgePtr != nullptr && OpenNurbsEdge != nullptr && OpenNurbsEdge->m_ti.Count() > 1)
	{
		OpenNurbsTrimId2TechSoftCoEdge.Add(OpenNurbsTrim.m_trim_index, CoEdgePtr);
	}
	return CoEdgePtr;
}

A3DTopoFace* FOpenNurbsBRepToTechSoftConverter::CreateTopoFace(const ON_BrepFace& OpenNurbsFace)
{
	ON_NurbsSurface OpenNurbsSurface;
	OpenNurbsFace.NurbsSurface(&OpenNurbsSurface);

	A3DSurfBase* CarrierSurface = CreateSurface(OpenNurbsSurface);
	if (CarrierSurface == nullptr)
	{
		return nullptr;
	}

	const ON_BrepLoop* OuterLoop = OpenNurbsFace.OuterLoop();
	if (OuterLoop == nullptr)
	{
		return CADLibrary::TechSoftUtils::CreateTopoFaceWithNaturalLoop(CarrierSurface);
	}

	int32 LoopCount = OpenNurbsFace.LoopCount();

	TArray<A3DTopoLoop*> Loops;
	Loops.Reserve(LoopCount);

	int32 OuterLoopIndex = 0;
	for (int32 LoopIndex = 0; LoopIndex < LoopCount; ++LoopIndex)
	{
		const ON_BrepLoop& OpenNurbsLoop = *OpenNurbsFace.Loop(LoopIndex);
		ON_BrepLoop::TYPE LoopType = OpenNurbsLoop.m_type;
		if (LoopType == ON_BrepLoop::TYPE::outer)
		{
			OuterLoopIndex = Loops.Num();
		}

		A3DTopoLoop* Loop = CreateTopoLoop(OpenNurbsLoop);
		if (Loop != nullptr)
		{
			Loops.Add(Loop);
			LinkEdgesLoop(OpenNurbsLoop);
		}
	}

	if (Loops.Num() == 0)
	{
		return nullptr;
	}

	CADLibrary::TUniqueTSObj<A3DTopoFaceData> Face;
	Face->m_pSurface = CarrierSurface;
	Face->m_bHasTrimDomain = false;
	Face->m_ppLoops = Loops.GetData();
	Face->m_uiLoopSize = Loops.Num();
	Face->m_uiOuterLoopIndex = OuterLoopIndex;
	Face->m_dTolerance = 0.01; //mm

	A3DTopoFace* FacePtr = CADLibrary::TechSoftInterface::CreateTopoFace(*Face);
	
	// Color with color name = 0. c.f. CADLibrary::BuildColorName 
	constexpr FColor DefaultColor(0, 0, 0, 0);
	CADLibrary::TechSoftUtils::SetEntityGraphicsColor(FacePtr, DefaultColor);

	return FacePtr;
}
#endif

bool FOpenNurbsBRepToTechSoftConverter::AddBRep(ON_Brep& BRep, const ON_3dVector& Offset)
{
#ifdef USE_TECHSOFT_SDK
	OpenNurbsTrimId2TechSoftCoEdge.Empty();

	BRep.Translate(Offset);
	BRep.FlipReversedSurfaces();

	// Create faces
	int32 FaceCount = BRep.m_F.Count();
	TArray<A3DTopoFace*> TSFaces;
	TArray<A3DUns8> FaceOrientations;

	TSFaces.Reserve(FaceCount);
	FaceOrientations.Reserve(FaceCount);

	for (int32 index = 0; index < FaceCount; index++)
	{
		const ON_BrepFace& OpenNurbsFace = BRep.m_F[index];
		A3DTopoFace* TSFace = CreateTopoFace(OpenNurbsFace);
		if (TSFace != nullptr)
		{
			TSFaces.Add(TSFace);
			FaceOrientations.Add(1);
		}
	}

	BRep.Translate(-Offset);

	if (TSFaces.IsEmpty())
	{
		return false;
	}

	A3DTopoShell* TopoShellPtr = nullptr;
	{
		CADLibrary::TUniqueTSObj<A3DTopoShellData> TopoShellData;
		TopoShellData->m_bClosed = false;
		TopoShellData->m_ppFaces = TSFaces.GetData();
		TopoShellData->m_uiFaceSize = TSFaces.Num();
		TopoShellData->m_pucOrientationWithShell = FaceOrientations.GetData();

		TopoShellPtr = CADLibrary::TechSoftInterface::CreateTopoShell(*TopoShellData);

		if (TopoShellPtr == nullptr)
		{
			return false;
		}
	}

	A3DRiRepresentationItem* RiRepresentationItem = CADLibrary::TechSoftUtils::CreateRIBRep(TopoShellPtr);

	if (RiRepresentationItem != nullptr)
	{
		RiRepresentationItems.Add(RiRepresentationItem);
		return true;
	}
#endif
	return false;
}

#endif // defined(USE_OPENNURBS)
