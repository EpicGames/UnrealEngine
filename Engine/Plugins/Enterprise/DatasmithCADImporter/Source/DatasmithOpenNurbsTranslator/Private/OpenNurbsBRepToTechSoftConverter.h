// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifdef USE_OPENNURBS

#include "OpenNurbsBRepConverter.h"
#include "CADModelToTechSoftConverterBase.h"

#ifdef USE_TECHSOFT_SDK
#include "TechSoftInterface.h"
#endif


class ON_BoundingBox;
class ON_Brep;
class ON_BrepFace;
class ON_BrepLoop;
class ON_BrepTrim;
class ON_NurbsCurve;
class ON_NurbsSurface;
class ON_3dVector;

class FOpenNurbsBRepToTechSoftConverter : public FCADModelToTechSoftConverterBase, public IOpenNurbsBRepConverter
{
public:

	/**
	 * Make sure CT is initialized, and a main object is ready.
	 * Handle input file unit and an output unit
	 * @param InOwner
	 */
	explicit FOpenNurbsBRepToTechSoftConverter(const CADLibrary::FImportParameters& ImportParameters)
		: FCADModelToTechSoftConverterBase(ImportParameters)
	{
	}

	virtual ~FOpenNurbsBRepToTechSoftConverter() = default;

	/**
	 * Set BRep to tessellate, offsetting it prior to tessellation(used to set mesh pivot at the center of the surface bounding box)
	 *
	 * @param  Brep	a BRep to tessellate
	 * @param  Offset translate Brep by this value before tessellating 
	 */
	virtual bool AddBRep(ON_Brep& Brep, const ON_3dVector& Offset) override;

private:
#ifdef USE_TECHSOFT_SDK

	TMap<int32, A3DTopoCoEdge*> OpenNurbsTrimId2TechSoftCoEdge;

	A3DTopoCoEdge* CreateTopoCoEdge(ON_BrepTrim& Trim);
	A3DTopoFace* CreateTopoFace(const ON_BrepFace& OpenNurbsFace);
	A3DTopoLoop* CreateTopoLoop(const ON_BrepLoop& OpenNurbsLoop);
	A3DSurfBase* CreateSurface(const ON_NurbsSurface& OpenNurbsSurface);
	A3DCrvBase* CreateCurve(const ON_NurbsCurve& OpenNurbsCurve);

	void LinkEdgesLoop(const ON_BrepLoop& OpenNurbsLoop);
#endif
};

#endif // USE_OPENNURBS