// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CADInterfacesModule.h"
#include "DatasmithImportOptions.h"
#include "DatasmithAssetImportData.h"

#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "UObject/ObjectMacros.h"

#include "DatasmithOpenNurbsImportOptions.generated.h"


UENUM()
enum class EDatasmithOpenNurbsBrepTessellatedSource : uint8
{
	/** Tessellate all Breps on import */
	UseUnrealNurbsTessellation UMETA(DisplayName = "Import as NURBS, Tessellate in Unreal"),

	/** Use render meshes stored in the scene file */
	UseRenderMeshes UMETA(DisplayName = "Import Rhino Meshes and UVs"),
};


USTRUCT(BlueprintType)
struct FDatasmithOpenNurbsOptions : public FDatasmithTessellationOptions
{
	GENERATED_BODY()

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Geometry & Tessellation Options")
	EDatasmithOpenNurbsBrepTessellatedSource Geometry = EDatasmithOpenNurbsBrepTessellatedSource::UseRenderMeshes;

	uint32 GetHash() const
	{
		return HashCombine(FDatasmithTessellationOptions::GetHash(), GetTypeHash(Geometry));
	}
};


UCLASS(BlueprintType, config = EditorPerProjectUserSettings)
class DATASMITHOPENNURBSTRANSLATOR_API UDatasmithOpenNurbsImportOptions : public UDatasmithOptionsBase
{
	GENERATED_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Geometry & Tessellation Options", meta = (ShowOnlyInnerProperties))
	FDatasmithOpenNurbsOptions Options;

#if WITH_EDITOR
	bool CanEditChange(const FProperty* InProperty) const override;
#endif //WITH_EDITOR

};

