// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define DATASMITHOPENNURBSTRANSLATOR_MODULE_NAME TEXT("DatasmithOpenNurbsTranslator")

/**
 * Datasmith Translator for .3dm files.
 */
class FDatasmithOpenNurbsTranslatorModule : public IModuleInterface
{
public:

    static FDatasmithOpenNurbsTranslatorModule& Get()
    {
        return FModuleManager::LoadModuleChecked< FDatasmithOpenNurbsTranslatorModule >(DATASMITHOPENNURBSTRANSLATOR_MODULE_NAME);
    }

    static bool IsAvailable()
    {
        return FModuleManager::Get().IsModuleLoaded(DATASMITHOPENNURBSTRANSLATOR_MODULE_NAME);
    }

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	FString GetTempDir() const;

private:
	FString TempDir;
};
