// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithPLMXMLTranslator : ModuleRules
	{
		public DatasmithPLMXMLTranslator(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CADInterfaces",
					"CADKernelSurface",
					"CADLibrary",
					"CADTools",
					"DatasmithCADTranslator", // for DatasmithMeshBuilder
					"DatasmithContent",
					"DatasmithCore",
					"DatasmithDispatcher",
					"DatasmithTranslator",
					"Engine",
					"MeshDescription",
					"ParametricSurface",
					"XmlParser",
				}
			);


			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
	 					"MessageLog",
	 					"UnrealEd",
					}
				);
			}
		}
	}
}
