// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithPlmXmlImporter.h"

#include "DatasmithImportOptions.h"
#include "DatasmithPayload.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"

#include "HAL/FileManager.h" // IFileManager
#include "Misc/FileHelper.h"
#include "Math/Transform.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include "XmlParser.h"


#include "CADData.h"
#include "CADToolsModule.h"
#include "CADKernelSurfaceExtension.h"
#include "DatasmithAdditionalData.h"
#include "DatasmithDispatcher.h"
#include "DatasmithMeshBuilder.h"
#include "DatasmithSceneGraphBuilder.h"
#include "DatasmithUtils.h"
#include "ParametricSurfaceTranslator.h"



DEFINE_LOG_CATEGORY_STATIC(LogDatasmithPlmXmlImport, Log, All);

class IDatasmithScene;
class IDatasmithMeshElement;
class FDatasmithSceneSource;
struct FDatasmithMeshElementPayload;

#define LOCTEXT_NAMESPACE "DatasmithPlmXmlImporter"

namespace PlmXml
{

	// for extra typesafety - so that id ref not mixed up with uri ref
	struct FIdRef
	{
		FString Value;

		explicit FIdRef(const FString& InValue): Value(InValue){}

		explicit operator bool() const
		{
			return !Value.IsEmpty();
		}

		bool operator == (const FIdRef& Other) const
		{
			return Value == Other.Value;
		}
	};

	// represents plmxml instance graph node with actor created for it
	struct FImportedInstanceTreeNode
	{
		TSharedPtr<IDatasmithActorElement> DatasmithActorElement;
		TMap<FString, TSharedPtr<FImportedInstanceTreeNode> > ChildInstances;
	};

	struct FImportedProductInstance
	{
		FString InstanceId;
		TSharedRef<IDatasmithActorElement> ActorElement;
	};

	struct FImportedProductRevision
	{
		FString Id;
		FString Label;
		TSharedRef<IDatasmithActorElement> ActorElement;
	};

	struct FImportedMesh
	{
		static const int32 INVALID_MESH_ID = -1;

		int32 ImportedMeshId = INVALID_MESH_ID;

		bool IsValid()
		{
			return ImportedMeshId != INVALID_MESH_ID;
		}
	};

	struct FImportedRepresentationInstance
	{
		FTransform Transform;
		TSharedPtr<IDatasmithActorElement> Parent;
		FString InstanceDatasmithName;
		FImportedMesh ImportedRepresentation;
		bool bOverrideWorldTransform = false;
	};

	struct FImportedExternalFile
	{
		TSharedPtr<IDatasmithActorElement> Parent;
		FString InstanceDatasmithName;
		FImportedMesh ImportedRepresentation;
	};

	FString GetAttributeId(const FXmlNode* Node)
	{
		return Node->GetAttribute(TEXT("id"));// xsd:ID
	}

	FString GetLabel(const FXmlNode* Node)
	{
		FString Name = Node->GetAttribute(TEXT("name"));
		return FDatasmithUtils::SanitizeObjectName(Name.IsEmpty() ? GetAttributeId(Node) : Name);
	}

	FString GetReferenceIdFromUri(FString Uri)
	{
		if (Uri.IsEmpty())
		{
			return FString();
		}

		if (!ensure(Uri[0] == TCHAR('#'))) // haven't seen any examples of URI not referencing anything besides local identifier with '#'
		{
			return FString();
		}
		Uri.RemoveAt(0, 1);
		return Uri;
	}

	// plm:uriReferenceListType
	TArray<FIdRef> GetAttributeUriReferenceList(const FXmlNode* Node, const FString& Name)
	{
		TArray<FString> InstanceUris;
		Node->GetAttribute(Name).ParseIntoArray(InstanceUris, TEXT(" "), false);

		TArray<FIdRef> InstanceIds;
		for(FString& InstanceUri: InstanceUris)
		{
			FString Id = GetReferenceIdFromUri(InstanceUri);
			if(!ensure(!Id.IsEmpty())) // we don't expect empty instanceRef here
			{
				continue;
			}
			InstanceIds.Emplace(Id);
		}
		return InstanceIds;
	}

	// Return id referenced by an 'Uri' attribute. 
	// plm:anyURIType
	FIdRef GetUriReferenceId(const FXmlNode* Node, const FString& Name)
	{
		return FIdRef(GetReferenceIdFromUri(Node->GetAttribute(Name)));
	}

	// xsd:IDREFS
	TArray<FIdRef> GetAttributeIDREFS(const FXmlNode* Node, const FString Name)
	{
		FString IdRefsText = Node->GetAttribute(*Name);
		if (IdRefsText.IsEmpty())
		{
			return TArray<FIdRef>();
		}

		TArray<FString> IdRefs;
		IdRefsText.ParseIntoArray(IdRefs, TEXT(" "), false);

		TArray<FIdRef> Result;
		Result.Reserve(IdRefs.Num());
		for(FString IdRef: IdRefs)
		{
			Result.Add(FIdRef(IdRef));
		}
		
		return Result;
	}

	FString CreateDatasmithName(FString ParentName, const FString& Name) {
		return FMD5::HashAnsiString(*(ParentName + Name));
	}

	struct FProductInstance
	{
		const FXmlNode* Node = nullptr;

		operator bool() const
		{
			return Node != nullptr;
		}
	};

	struct FProductRevisionView
	{
		const FXmlNode* Node = nullptr;
		
		operator bool() const
		{
			return Node != nullptr;
		}
	};

	struct FRepresentation
	{
		const FXmlNode* Node = nullptr;

		operator bool() const
		{
			return Node != nullptr;
		}
	};

	void SetActorWorldTransform(const TSharedRef<IDatasmithActorElement>& ActorElement, const FTransform& Transform)
	{
		ActorElement->SetTranslation(Transform.GetTranslation()); // TODO: convert units
		ActorElement->SetRotation(Transform.GetRotation());
		ActorElement->SetScale(Transform.GetScale3D());
	}

	class FPlmXmlMeshLoaderWithDatasmithDispatcher
	{
	public:
		TSharedRef<IDatasmithScene> DatasmithScene;
		FDatasmithTessellationOptions& TessellationOptions;
		
		TUniquePtr<DatasmithDispatcher::FDatasmithDispatcher> DatasmithDispatcher;

		TUniquePtr <FDatasmithSceneGraphBuilder> SceneGraphBuilder;
		TUniquePtr<FDatasmithMeshBuilder> MeshBuilderPtr;

		CADLibrary::FImportParameters ImportParameters;
		TMap<uint32, FString> CADFileToUEGeomMap;
		TMap<uint32, FString> CADFileToUEFileMap;

		TArray<FString> FilePaths;

		FString CacheDir;

		FPlmXmlMeshLoaderWithDatasmithDispatcher(TSharedRef<IDatasmithScene> InDatasmithScene, FDatasmithTessellationOptions& InTessellationOptions)
			: DatasmithScene(InDatasmithScene)
			, TessellationOptions(InTessellationOptions)
		{
			FCADToolsModule& CADToolsModule = FCADToolsModule::Get();
			
			CacheDir = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("DatasmithCADCache"), *FString::FromInt(CADToolsModule.GetCacheVersion())));
			IFileManager::Get().MakeDirectory(*CacheDir);

			// Setup of import parameters for DatasmithDispatcher copied from FDatasmithCADTranslator's setup
			ImportParameters.SetTesselationParameters(TessellationOptions.ChordTolerance, TessellationOptions.MaxEdgeLength, TessellationOptions.NormalTolerance, (CADLibrary::EStitchingTechnique) TessellationOptions.StitchingTechnique);

			DatasmithDispatcher = MakeUnique<DatasmithDispatcher::FDatasmithDispatcher>(ImportParameters, CacheDir, CADFileToUEFileMap, CADFileToUEGeomMap);
		}

		// Adds geom file to load and returns Id to use in InstantiateMesh later(after all is loaded)
		int32 AddMeshToLoad(const FString& FullPath)
		{
			using namespace CADLibrary;
			const EMesher Mesher = FImportParameters::bGDisableCADKernelTessellation ? EMesher::TechSoft : EMesher::CADKernel;

			FFileDescriptor FileDescription(*FullPath);
			DatasmithDispatcher->AddTask(FileDescription, Mesher);
			return FilePaths.Add(FullPath);
		}

		// Creates an instance of the mesh (identified by Id returned from AddMeshToLoad)
		TSharedPtr<IDatasmithActorElement> InstantiateMesh(FString DatasmithName, int32 Id)
		{
			TSharedRef<IDatasmithActorElement> ActorElement = FDatasmithSceneFactory::CreateActor(*DatasmithName);
			ActorElement->SetLabel(TEXT("Invalid")); // Label is set in FillAnchorActor
			
			if(FilePaths.IsValidIndex(Id))
			{
				// Make sure file was loaded
				SceneGraphBuilder->FillAnchorActor(ActorElement, FilePaths[Id]);
			}

			return ActorElement;
		}

		void Process(const FDatasmithSceneSource& Source)
		{
			DatasmithDispatcher->Process(CADLibrary::GMaxImportThreads != 1);
			SceneGraphBuilder = MakeUnique<FDatasmithSceneGraphBuilder>(CADFileToUEFileMap, CacheDir, DatasmithScene, Source, ImportParameters);
			SceneGraphBuilder->LoadSceneGraphDescriptionFiles();
			MeshBuilderPtr = MakeUnique<FDatasmithMeshBuilder>(CADFileToUEGeomMap, CacheDir, ImportParameters);
		}

		void UnloadScene()
		{
			MeshBuilderPtr.Reset();
			SceneGraphBuilder.Reset();
		}

		bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement>& MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
		{
			CADLibrary::FMeshParameters MeshParameters;

			if (TOptional<FMeshDescription> Mesh = MeshBuilderPtr->GetMeshDescription(MeshElement, MeshParameters))
			{
				OutMeshPayload.LodMeshes.Add(MoveTemp(Mesh.GetValue()));

				if (CADLibrary::FImportParameters::bGDisableCADKernelTessellation)
				{
					ParametricSurfaceUtils::AddSurfaceData(MeshElement->GetFile(), ImportParameters, MeshParameters, TessellationOptions, OutMeshPayload);
				}
				else
				{
					CADKernelSurface::AddSurfaceDataForMesh(MeshElement->GetFile(), ImportParameters, MeshParameters, TessellationOptions, OutMeshPayload);
				}

				return true;
			}
			return false;
		}

		void SetTessellationOptions(const UDatasmithCommonTessellationOptions& TessellationOptions);
	};

	struct FParsedPlmXml
	{
		TMap<FString, const FXmlNode*> ExternalFileNodes;
		TMap<FString, const FXmlNode*> DataSetNodes;
		TMap<FString, const FXmlNode*> ProductRevisionNodes;
		TArray<const FXmlNode*> ProductViewNodes;
	};

	struct FParsedProductDef
	{
		TMap<FString, const FXmlNode*> ProductInstanceNodes;
		TMap<FString, const FXmlNode*> ProductRevisionViewNodes;
	};

	struct FImportContext
	{
		FString MainPlmXmlFilePath;
		const FDatasmithSceneSource& Source;
		
		TSharedRef<IDatasmithScene> DatasmithScene;
		TSharedRef<IDatasmithLevelVariantSetsElement> LVS;

		FPlmXmlMeshLoader& MeshLoader;

		FParsedPlmXml ParsedPlmXml;
		
		TArray<FIdRef> InstanceGraphRootRefs;
		TArray<FImportedRepresentationInstance> RepresentationInstances;
		TSharedRef<FImportedInstanceTreeNode> ImportedInstanceTreeRootNode;
		TMap<FString, TSharedPtr<FImportedInstanceTreeNode> > ImportedInstanceTreeNodes;
		TArray<FImportedProductInstance> ProductInstances;
		TSharedPtr<IDatasmithActorElement> RootActorElementForProductRevisions;

		TMap<FString, FImportedMesh> RepresentationsById;


		FImportContext(FString InRootFilePath,
		               FPlmXmlMeshLoader& InMeshLoader,
		               TSharedRef<IDatasmithScene> InDatasmithScene,
		               const FDatasmithSceneSource& InSource
		)
			: MainPlmXmlFilePath(InRootFilePath)
			, Source(InSource)
			, DatasmithScene(InDatasmithScene)
			, LVS(FDatasmithSceneFactory::CreateLevelVariantSets(TEXT("LevelVariantSets")))
			, MeshLoader(InMeshLoader)
			, ImportedInstanceTreeRootNode(MakeShared<PlmXml::FImportedInstanceTreeNode>())
		{
			
		}
	};

	struct FProductDefImportContext
	{
		FImportContext& MainImportContext;
		
		const FParsedProductDef& ProductDef;

		FProductDefImportContext(FImportContext& InMainImportContext, const FParsedProductDef& InProductDef)
			: MainImportContext(InMainImportContext)
			, ProductDef(InProductDef)
		{
		}

		bool HasProductInstance(FIdRef InstanceRef)
		{
			return ProductDef.ProductInstanceNodes.Contains(InstanceRef.Value);
		}
		
		FProductInstance GetProductInstance(FIdRef InstanceRef)
		{
			return FProductInstance{ ProductDef.ProductInstanceNodes[InstanceRef.Value] };
		}
		
		bool HasProductRevisionView(FIdRef InstanceRef)
		{
			return ProductDef.ProductRevisionViewNodes.Contains(InstanceRef.Value);
		}

		FProductRevisionView GetProductRevisionView(FIdRef InstanceRef)
		{
			return FProductRevisionView{ ProductDef.ProductRevisionViewNodes[InstanceRef.Value] };
		}
	};

	class FTraverseContext
	{
	public:
		FTraverseContext(FProductDefImportContext& InImportContext, int32 Level)
			: ImportContext(InImportContext)
			, Parent(nullptr)
			, CurrentDatasmithName()
			, Level(0)
		{
		}

		// Make child context
		FTraverseContext(FTraverseContext& InParent, FString InName)
			: ImportContext(InParent.ImportContext)
			, Parent(&InParent)
			, CurrentDatasmithName(CreateDatasmithName(InParent.CurrentDatasmithName, InName))
			, Level(InParent.Level)
			, Transform(InParent.Transform)
		{
		}

		virtual ~FTraverseContext()
		{}

		virtual TSharedPtr<IDatasmithActorElement> GetActorElement() = 0;
		virtual void AddChildActorElement(TSharedRef<IDatasmithActorElement> ActorElement) = 0;
		virtual void AddChildProductInstance(FString Id, TSharedRef<FImportedInstanceTreeNode> ImportedInstanceTreeNode) = 0;

		FProductDefImportContext& ImportContext;

		// Traverse hierarchy data
		FTraverseContext* Parent = nullptr;
		FString CurrentDatasmithName; // Used to identify nodes in PLMXLM hierarchy and as unique Name for Datasmith
		int32 Level;
		FTransform Transform; // TODO: PRV doesn't have Transform, so, probably this needs to be moved to ProductInstance and made as a method in base context
	};

	class FTraverseRootContext : public FTraverseContext
	{
		using FTraverseContext::FTraverseContext;
	public:

		virtual TSharedPtr<IDatasmithActorElement> GetActorElement()
		{
			return TSharedPtr<IDatasmithActorElement>(); // mullptr indicates root
		}

		void AddChildActorElement(TSharedRef<IDatasmithActorElement> ActorElement) override
		{
			ImportContext.MainImportContext.DatasmithScene->AddActor(ActorElement);
		}
		
		void AddChildProductInstance(FString Id, TSharedRef<FImportedInstanceTreeNode> InImportedInstanceTreeNode) override
		{
			ImportContext.MainImportContext.ImportedInstanceTreeRootNode->ChildInstances.Add(Id, InImportedInstanceTreeNode);

			ImportContext.MainImportContext.ImportedInstanceTreeNodes.Add(Id, InImportedInstanceTreeNode);
		}
	};

	class FTraverseProductRevisionViewContext : public FTraverseContext
	{
		using FTraverseContext::FTraverseContext;
	public:
		virtual TSharedPtr<IDatasmithActorElement> GetActorElement() override
		{
			return ActorElement ? ActorElement : Parent->GetActorElement();
		}

		virtual void AddChildActorElement(TSharedRef<IDatasmithActorElement> InActorElement) override
		{
			Parent->AddChildActorElement(InActorElement);
		}
		
		void AddChildProductInstance(FString Id, TSharedRef<FImportedInstanceTreeNode> InImportedInstanceTreeNode) override
		{
			Parent->AddChildProductInstance(Id, InImportedInstanceTreeNode);
		}

		void SetActorElement(const TSharedRef<IDatasmithActorElement>& InActorElement)
		{
			ActorElement = InActorElement;
			Parent->AddChildActorElement(InActorElement);
		}

		TSharedPtr<IDatasmithActorElement> ActorElement;
		FProductRevisionView ProductRevisionView;
	};

	class FTraverseProductInstanceContext : public FTraverseContext
	{
		using FTraverseContext::FTraverseContext;

	public:
		
		virtual TSharedPtr<IDatasmithActorElement> GetActorElement() override
		{
			return ActorElement;
		}

		virtual void AddChildActorElement(TSharedRef<IDatasmithActorElement> ChildActorElement) override
		{
			ActorElement->AddChild(ChildActorElement);
		}

		void AddChildProductInstance(FString Id, TSharedRef<FImportedInstanceTreeNode> InImportedInstanceTreeNode) override
		{
			ImportedInstanceTreeNode->ChildInstances.Add(Id, InImportedInstanceTreeNode);
		}

		void SetActorElement(FString InstanceId, const TSharedRef<IDatasmithActorElement>& InActorElement)
		{
			ImportedInstanceTreeNode = MakeShared<FImportedInstanceTreeNode>();
			ImportedInstanceTreeNode->DatasmithActorElement = InActorElement;
			ImportContext.MainImportContext.ProductInstances.Add(FImportedProductInstance{ InstanceId, InActorElement });
			
			// ImportBinding uses Tag[0] ('original name') to group parts used in variants
			InActorElement->AddTag(*InstanceId);
			
			ActorElement = InActorElement;

			Parent->AddChildActorElement(InActorElement);
			Parent->AddChildProductInstance(InstanceId, ImportedInstanceTreeNode.ToSharedRef());
		}

		FProductInstance ProductInstance;
		TSharedPtr<IDatasmithActorElement> ActorElement;
		TSharedPtr<FImportedInstanceTreeNode> ImportedInstanceTreeNode;
	};

	class FTraverseRepresentationContext : public FTraverseContext
	{
		using FTraverseContext::FTraverseContext;

	public:
		
		virtual TSharedPtr<IDatasmithActorElement> GetActorElement() override
		{
			ensure(false);
			return TSharedPtr<IDatasmithActorElement>();
		}

		virtual void AddChildActorElement(TSharedRef<IDatasmithActorElement> ChildActorElement) override
		{
			ensure(false);
		}

		void AddChildProductInstance(FString Id, TSharedRef<FImportedInstanceTreeNode> InImportedInstanceTreeNode) override
		{
		}


		FRepresentation Representation;
	};

	FTraverseProductInstanceContext MakeTraverseProductInstanceContext(FTraverseContext& ParentContext, FProductInstance ProductInstance)
	{
		if (!ProductInstance)
		{
			return FTraverseProductInstanceContext(ParentContext, FString());
		}
		
		FTraverseProductInstanceContext Result(ParentContext, GetAttributeId(ProductInstance.Node));
		Result.ProductInstance = ProductInstance;
		return Result;
	}

	FTraverseProductRevisionViewContext MakeTraverseContext(FTraverseContext& ParentContext, FTransform Transform, FProductRevisionView PRV)
	{
		FTraverseProductRevisionViewContext Result(ParentContext, GetAttributeId(PRV.Node));
		Result.Transform = Transform * Result.Transform;
		Result.ProductRevisionView = PRV;
		return Result;
	}

	FTraverseRepresentationContext MakeTraverseContext(FTraverseContext& ParentContext, FRepresentation Representation)
	{
		FTraverseRepresentationContext Result(ParentContext, GetAttributeId(Representation.Node));
		Result.Representation = Representation;
		return Result;
	}

	void TraverseProductRevisionView(FTraverseProductRevisionViewContext&& Context);
	void TraverseProductInstance(FTraverseProductInstanceContext&& Context);

	FImportedMesh GetOrImportRepresentation(FImportContext& ImportContext, FString ParentId, const FRepresentation& Representation)
	{
		const FXmlNode* RepresentationNode = Representation.Node;

		FString RepresentationId = GetAttributeId(RepresentationNode);
		FString RepresentationFormat = RepresentationNode->GetAttribute(TEXT("format")); // "JT", "CATPart", "PLMXML", ...anything

		ensure(!RepresentationId.IsEmpty());
		
		// Combine parent id(in case representation itself has no id), id and format into representation unique key
		FString RepresentationHashId = FMD5::HashAnsiString(*(ParentId + RepresentationId + RepresentationFormat));

		if (FImportedMesh* ImportedRepresentationPtr = ImportContext.RepresentationsById.Find(RepresentationHashId))
		{
			return *ImportedRepresentationPtr;
		}

		FImportedMesh ImportedRepresentation;
		// Queue mesh loading for any Representation format(unsupported won't be loaded) 
		{
			FString LocationText = RepresentationNode->GetAttribute(TEXT("location")); // plm:anyURIType, so potentially can be anything but we assume relative file path for now

			// TODO:  Possible(not included in samples yet) - <EntityMaterial>, <Material>, <Transform>, <CompoundRep>
			if (LocationText.IsEmpty())
			{
				ImportContext.RepresentationsById.Add(RepresentationHashId, ImportedRepresentation);
				return ImportedRepresentation;
			}
			
			FString FullPath = FPaths::IsRelative(LocationText) ? FPaths::ConvertRelativePathToFull(FPaths::GetPath(ImportContext.MainPlmXmlFilePath) / LocationText) : LocationText;
			ImportedRepresentation.ImportedMeshId = ImportContext.MeshLoader.AddMeshToLoad(FullPath);
		}

		ImportContext.RepresentationsById.Add(RepresentationHashId, ImportedRepresentation);
		return ImportedRepresentation;
	}

	void ParseUserDataToDatasmithMetadata(const FXmlNode* Node, TSharedRef<IDatasmithActorElement> ActorElement, TSharedRef<IDatasmithScene> DatasmithScene)
	{
		TSharedPtr<IDatasmithMetaDataElement> Metadata;
		for (const FXmlNode* ChildNode : Node->GetChildrenNodes())
		{
			if (TEXT("UserData") == ChildNode->GetTag())
			{
				const FXmlNode* UserDataNode = ChildNode;
				if (!Metadata.IsValid())
				{
					Metadata = DatasmithScene->GetMetaData(ActorElement);
					if (!Metadata.IsValid())
					{
						Metadata = FDatasmithSceneFactory::CreateMetaData(ActorElement->GetName());
						Metadata->SetAssociatedElement(ActorElement);
						DatasmithScene->AddMetaData(Metadata);
					}
				}

				for (const FXmlNode* UserDataChildNode : UserDataNode->GetChildrenNodes())
				{
					if (TEXT("UserValue") == UserDataChildNode->GetTag())
					{
						const FXmlNode* UserValueNode = UserDataChildNode;

						FString UserValueTitle = UserDataChildNode->GetAttribute("title");
						FString UserValueType = UserDataChildNode->GetAttribute("type");

						FString Key = UserValueTitle;
						FString Value = UserDataChildNode->GetAttribute("value"); // "value is ignored if the type attribute is list, enum or reference"

						// "int", "ints", "real", "reals", "boolean", "booleans", "string", "reference", "enum", "list", "dateTime"

						// "list"

						if (UserValueType.IsEmpty() // defaults to 'string'
							|| TEXT("string") == UserValueType
							|| TEXT("none") == UserValueType
							|| TEXT("dateTime") == UserValueType // dateTime as simple string
							|| TEXT("reference") == UserValueType // just record reference value
							|| TEXT("enum") == UserValueType // "value" contains this enum's value, UserList lists all possible enum values
							|| TEXT("ints") == UserValueType // "value" contains a list of integers
							|| TEXT("reals") == UserValueType // "value" contains a list of doubles
							|| TEXT("booleans") == UserValueType // "value" contains a list of bools
							)
						{
							TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
							MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::String);
							MetadataPropertyPtr->SetValue(*Value);
							Metadata->AddProperty(MetadataPropertyPtr);
						}
						else if (TEXT("real") == UserValueType)
						{
							TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
							MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
							MetadataPropertyPtr->SetValue(*Value);
							Metadata->AddProperty(MetadataPropertyPtr);
						}
						else if (TEXT("boolean") == UserValueType)
						{
							TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
							MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Bool);
							MetadataPropertyPtr->SetValue(*Value);
							Metadata->AddProperty(MetadataPropertyPtr);
						}
						else if (TEXT("int") == UserValueType)
						{
							TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
							MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Float); // NOTE : "int" translated as Float
							MetadataPropertyPtr->SetValue(*Value);
							Metadata->AddProperty(MetadataPropertyPtr);
						}
					}
				}
			}
		}
	}

	void ParseRepresentation(FImportContext& ImportContext, const FXmlNode* Node, const TSharedPtr<IDatasmithActorElement>& ParentElement, FString Name)
	{
		// TODO: probably need to check if Representation is of format 'JT', not another type and handle those other types separately(or just ignore)
		if (TEXT("Representation") == Node->GetTag())
		{
			FRepresentation Representation = { Node };

			FImportedMesh ImportedRepresentation = GetOrImportRepresentation(ImportContext, Name, Representation);

			if (ImportedRepresentation.IsValid())
			{
				// Record instance info to instantiate later(after finishing InstanceGraph traversal and geometry loading)

				FImportedRepresentationInstance RepresentationInstance;
				RepresentationInstance.ImportedRepresentation = ImportedRepresentation;

				RepresentationInstance.bOverrideWorldTransform = false;
				
				RepresentationInstance.Parent = ParentElement;
				RepresentationInstance.InstanceDatasmithName = CreateDatasmithName(Name, TEXT("RepresentationInstance_") + FString::FromInt(ImportedRepresentation.ImportedMeshId));

				ImportContext.RepresentationInstances.Add(RepresentationInstance);
			}
		}
	}
	
	void TraverseProductRevisionView(FTraverseProductRevisionViewContext&& Context)
	{
		FProductRevisionView ProductRevisionView = Context.ProductRevisionView;

		if (!Context.GetActorElement()) // in case no ActorElement was created by parent(this happens when PRV is among rootRefs)
		{
			FString InstanceId = GetAttributeId(ProductRevisionView.Node);

			FString DatasmithName = CreateDatasmithName(Context.CurrentDatasmithName, InstanceId);

			TSharedRef<IDatasmithActorElement> ActorElement = FDatasmithSceneFactory::CreateActor(*DatasmithName);
			Context.SetActorElement(ActorElement);
		}
		TSharedRef<IDatasmithActorElement> ActorElement = Context.GetActorElement().ToSharedRef();
		// Label actor after ProductRevisionView
		FString DatasmithLabel = GetLabel(ProductRevisionView.Node);
		ActorElement->SetLabel(*DatasmithLabel);

		// type: The type of PRV. This is one of assembly, minimal, wire, solid, sheet, general. Optional, though.
		FString ProductRevisionViewType = ProductRevisionView.Node->GetAttribute(TEXT("type"));
		FString ProductRevisionViewId = PlmXml::GetAttributeId(ProductRevisionView.Node);

		if (true) // TEXT("Solid") == ProductRevisionViewType) ??? looks like testing for 'solid' leaves only single JT node in our samples
		{
			// Consider every Representation child node to represent visible geometry
			for (const FXmlNode* ProductRevisionViewChildNode : ProductRevisionView.Node->GetChildrenNodes())
			{
				FTransform LocalTransform;
				LocalTransform.SetIdentity(); 
				ParseRepresentation(Context.ImportContext.MainImportContext, ProductRevisionViewChildNode, Context.GetActorElement(), Context.CurrentDatasmithName);
			}
		}

		ParseUserDataToDatasmithMetadata(ProductRevisionView.Node, ActorElement, Context.ImportContext.MainImportContext.DatasmithScene);

		TArray<FIdRef> InstanceRefs = GetAttributeIDREFS(ProductRevisionView.Node, TEXT("instanceRefs")); // xsd:IDREFS, unlike in ProductInstance, ProductRevisionView uses IDREFS, not Uri

		for (const FIdRef& InstanceRef : InstanceRefs)
		{
			if (Context.ImportContext.HasProductInstance(InstanceRef))
			{
				TraverseProductInstance(MakeTraverseProductInstanceContext(Context, Context.ImportContext.GetProductInstance(InstanceRef)));
			}
			else if (Context.ImportContext.HasProductRevisionView(InstanceRef)) // in case InstanceRef is not a product instance
			{
				FTransform RootTransform;
				RootTransform.SetIdentity();
				TraverseProductRevisionView(MakeTraverseContext(Context, RootTransform, Context.ImportContext.GetProductRevisionView(InstanceRef)));
			}
		}
	}

	FTransform ParseTransform(const FXmlNode* TransformNode)
	{
		FTransform Transform;
		Transform.SetIdentity();
		
		TArray<FString> Tokens;
		if (ensure(16 == TransformNode->GetContent().ParseIntoArray(Tokens, TEXT(" "))))
		{
			float M[16];
			for (int32 I = 0; I < 16; ++I)
			{
				M[I] = FCString::Atof(*(Tokens[I]));
			}

			// 				- r is a rotation component
			// 				- p is a perspective component
			// 				- t is a translation component
			// 				- s is the inverse scale
			// 					Thus, the content of a default <Transform> is ordered as follows :
			// 				r00 r10 r20 p0 r01 r11 r21 p1 r02 r12 r22 p2 t0 t1 t2 s

			FMatrix Matrix = FMatrix::Identity;
			float InverseScaling = M[15];
			float ScalingScalar = FMath::Abs(InverseScaling) < KINDA_SMALL_NUMBER ? FMath::Sign(InverseScaling) : 1.f / InverseScaling;
			FVector Axis0 = FVector(M[0], M[1], M[2]);
			FVector Axis1 = FVector(M[4], M[5], M[6]);
			FVector Axis2 = FVector(M[8], M[9], M[10]);

			FVector Scaling(ScalingScalar, ScalingScalar, ScalingScalar);

			// In case Axis basis is not LeftHanded(i.e. can't be represented as rotation in Unreal left-handed coordinate system)
			// Make it left-handed by flipping one axis and negating scaling for that axis
			if (FVector::DotProduct(FVector::CrossProduct(Axis0, Axis1), Axis2) < 0.0f)
			{
				Axis2 = -Axis2;
				Scaling[2] = -Scaling[2];
			}

			FVector Origin = FVector(M[12], M[13], M[14]) * 100.f; //TODO: what units we have here?
			Matrix.SetAxes(&Axis0, &Axis1, &Axis2, &Origin);
			Transform.SetFromMatrix(Matrix);
			Transform.SetScale3D(Scaling);

			Transform = FDatasmithUtils::ConvertTransform(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded, Transform);
		}

		return Transform;
	}

	void TraverseProductInstance(FTraverseProductInstanceContext&& ProductInstanceContext)
	{
		FProductInstance ProductInstance = ProductInstanceContext.ProductInstance;
		if (!ProductInstance)
		{
			return;
		}

		FString InstanceId = GetAttributeId(ProductInstance.Node);
		
		FString DatasmithName = CreateDatasmithName(ProductInstanceContext.CurrentDatasmithName, InstanceId);
		FString DatasmithLabel = GetLabel(ProductInstance.Node);
		
		TSharedRef<IDatasmithActorElement> ActorElement = FDatasmithSceneFactory::CreateActor(*DatasmithName);
		ActorElement->SetLabel(*DatasmithLabel);
		ProductInstanceContext.SetActorElement(InstanceId, ActorElement);

		const FXmlNode* TransformNode = ProductInstance.Node->FindChildNode(TEXT("Transform"));

		FTransform Transform;
		if (TransformNode)
		{
			Transform = ParseTransform(TransformNode);
		}
		else
		{
			Transform.SetIdentity();
		}

		FTransform ParentWorldTransform = ProductInstanceContext.Transform;
		FTransform ActorWorldTransform;
		FTransform::Multiply(&ActorWorldTransform, &Transform, &ParentWorldTransform);
		PlmXml::SetActorWorldTransform(ActorElement, ActorWorldTransform);

		FIdRef PartRefId = GetUriReferenceId(ProductInstance.Node, TEXT("partRef")); // partRef can be <Product>, <ProductRevision>, or<ProductRevisionView>.

		if (!PartRefId)
		{
			return;
		}

		ParseUserDataToDatasmithMetadata(ProductInstance.Node, ActorElement, ProductInstanceContext.ImportContext.MainImportContext.DatasmithScene);

		if (ProductInstanceContext.ImportContext.HasProductRevisionView(PartRefId))
		{
			TraverseProductRevisionView(MakeTraverseContext(ProductInstanceContext, Transform, ProductInstanceContext.ImportContext.GetProductRevisionView(PartRefId)));
		}
		else
		{
			UE_LOGF(LogDatasmithPlmXmlImport, Warning, "ProductInstance with id='%ls' has partRef that doesn't refer to an existing ProductRevisionView", *InstanceId)
		}
	}

}  // namespace PlmXml

FDatasmithPlmXmlImporter::FDatasmithPlmXmlImporter(TSharedRef<IDatasmithScene> OutScene)
	: DatasmithScene(OutScene)
{
}

FDatasmithPlmXmlImporter::~FDatasmithPlmXmlImporter()
{
}

namespace PlmXml
{


	FImportedProductRevision ParseProductRevision(FImportContext& ImportContext, TSharedRef<IDatasmithActorElement> RootActorElement, const FXmlNode* ProductRevisionNode, FString ParentUUID)
	{
		PlmXml::FParsedPlmXml& ParsedPlmXml = ImportContext.ParsedPlmXml;
		FString ProductRevisionUUID = GetAttributeId(ProductRevisionNode);
		FString ProductRevisionLabel = GetLabel(ProductRevisionNode);
		TSharedRef<IDatasmithActorElement> ProductRevisionActorElement = FDatasmithSceneFactory::CreateActor(*CreateDatasmithName(ParentUUID, ProductRevisionUUID));
		ProductRevisionActorElement->SetLabel(*ProductRevisionLabel);
		RootActorElement->AddChild(ProductRevisionActorElement, EDatasmithActorAttachmentRule::KeepRelativeTransform);

		for (const FXmlNode* ProductRevisionChildNode : ProductRevisionNode->GetChildrenNodes())
		{
			if (ProductRevisionChildNode->GetTag() == TEXT("AssociatedDataSet"))
			{
				const FXmlNode* AssociatedDataSetNode = ProductRevisionChildNode;

				TSharedRef<IDatasmithActorElement> AssociatedDataSetActorElement = FDatasmithSceneFactory::CreateActor(*CreateDatasmithName(ProductRevisionActorElement->GetName(), GetAttributeId(AssociatedDataSetNode)));
				AssociatedDataSetActorElement->SetLabel(*GetLabel(AssociatedDataSetNode));
				ProductRevisionActorElement->AddChild(AssociatedDataSetActorElement, EDatasmithActorAttachmentRule::KeepRelativeTransform);

				TArray<FIdRef> DataSetIds = PlmXml::GetAttributeUriReferenceList(AssociatedDataSetNode, TEXT("dataSetRef"));

				for (const FIdRef& DataSetId : DataSetIds)
				{
					TSharedRef<IDatasmithActorElement> DataSetActorElement = FDatasmithSceneFactory::CreateActor(*CreateDatasmithName(AssociatedDataSetActorElement->GetName(), DataSetId.Value));
					DataSetActorElement->SetLabel(*(TEXT("DataSet_") + DataSetId.Value));
					AssociatedDataSetActorElement->AddChild(DataSetActorElement, EDatasmithActorAttachmentRule::KeepRelativeTransform);

					if (!ParsedPlmXml.DataSetNodes.Contains(DataSetId.Value))
					{
						UE_LOGF(LogDatasmithPlmXmlImport, Warning, "DataSet with id='%ls' referenced but not present in PLMXML", *DataSetId.Value)
					}
					else
					{
						// TODO: move this to parse once and 'instantiate' DataSet
						const FXmlNode* DataSetNode = ParsedPlmXml.DataSetNodes[DataSetId.Value];

						DataSetActorElement->SetLabel(*GetLabel(DataSetNode));

						TArray<FIdRef> ExternalFileIds = PlmXml::GetAttributeUriReferenceList(DataSetNode, TEXT("memberRefs"));

						for (const FIdRef& ExternalFileId : ExternalFileIds)
						{
							if (!ParsedPlmXml.ExternalFileNodes.Contains(ExternalFileId.Value))
							{
								UE_LOGF(LogDatasmithPlmXmlImport, Warning, "ExternalFile with id='%ls' referenced but not present in PLMXML", *ExternalFileId.Value)
							}
							else
							{
								const FXmlNode* ExternalFileNode = ParsedPlmXml.ExternalFileNodes[ExternalFileId.Value];
								FString LocationText = ExternalFileNode->GetAttribute(TEXT("locationRef"));

								FString FullPath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(ImportContext.MainPlmXmlFilePath) / LocationText);

								// note, it's not representation in terms of plmxml, but geometry data
								PlmXml::FImportedMesh ImportedRepresentation;
								ImportedRepresentation.ImportedMeshId = ImportContext.MeshLoader.AddMeshToLoad(FullPath);

								if (ImportedRepresentation.IsValid())
								{
									// Record instance info to instantiate later(after finishing InstanceGraph traversal and geometry loading)

									PlmXml::FImportedRepresentationInstance RepresentationInstance;
									RepresentationInstance.ImportedRepresentation = ImportedRepresentation;

									// Keep representation relative to its actor
									RepresentationInstance.bOverrideWorldTransform = false;
									
									// TODO: transform? RepresentationInstance.Transform = ;
									RepresentationInstance.Parent = DataSetActorElement; // TODO: ActorElement -
									// TODO: name - should be unique(made from hierarchy from root to dataset/file)
									RepresentationInstance.InstanceDatasmithName = PlmXml::CreateDatasmithName(DataSetActorElement->GetName(), TEXT("ExternalFile_") + ExternalFileId.Value);

									ImportContext.RepresentationInstances.Add(RepresentationInstance);
								}
							}
						}
					}
				}
			}
		}
		return FImportedProductRevision{ ProductRevisionUUID, ProductRevisionLabel, ProductRevisionActorElement };
	}

	TSharedRef<IDatasmithActorElement> CreateActor(const TSharedRef<IDatasmithActorElement>& ParentActorElement, FString Name, const TCHAR* Label=nullptr)
	{
		const TSharedRef<IDatasmithActorElement> ActorElement = FDatasmithSceneFactory::CreateActor(*CreateDatasmithName(ParentActorElement->GetName(), Name));
		if (Label)
		{
			ActorElement->SetLabel(Label);
		}
		ParentActorElement->AddChild(ActorElement);
		
		return ActorElement;
	}
	

	bool AddProductViewVariants(FImportContext& ImportContext)
	{
		// Variants 
		TSharedRef<IDatasmithLevelVariantSetsElement> LVS = ImportContext.LVS;

		if (ImportContext.ParsedPlmXml.ProductViewNodes.Num() > 0)
		{
			// Make a variant set to contain a variant per existing ProductView
			TSharedPtr<IDatasmithVariantSetElement> ProductViewVarSet = FDatasmithSceneFactory::CreateVariantSet(TEXT("ProductView"));
			LVS->AddVariantSet(ProductViewVarSet.ToSharedRef());

			const TSharedRef<IDatasmithActorElement> RootActorElementForProductViews = FDatasmithSceneFactory::CreateActor(TEXT("ProductViews"));
			RootActorElementForProductViews->SetLabel(TEXT("ProductViews"));
			ImportContext.DatasmithScene->AddActor(RootActorElementForProductViews);

			// TODO: add this to no-ProductDef path too
			// A variant per ProductView
			for (const FXmlNode* ProductViewNode : ImportContext.ParsedPlmXml.ProductViewNodes)
			{
				const TSharedRef<IDatasmithActorElement> ProductViewActorElement = CreateActor(RootActorElementForProductViews, *GetAttributeId(ProductViewNode), *GetLabel(ProductViewNode));
			
				// Collect all actors used in this ProductView via Occurrence node
				TSet<TSharedRef<IDatasmithActorElement>> ActorOptions;

				// TODO: rootRefs
				for (const FXmlNode* ProductViewChildNode : ProductViewNode->GetChildrenNodes())
				{
					if (ProductViewChildNode->GetTag() == TEXT("Occurrence"))
					{
					
						// TODO: rootRefs - if present use it to collect occurrence tree, else take all occurrences
						const FXmlNode* OccurrenceNode = ProductViewChildNode;
						const FString DatasmithName = *GetAttributeId(OccurrenceNode);
						const TSharedRef<IDatasmithActorElement> OccurrenceActorElement = CreateActor(ProductViewActorElement, DatasmithName, *GetLabel(OccurrenceNode));

						ParseUserDataToDatasmithMetadata(OccurrenceNode, OccurrenceActorElement, ImportContext.DatasmithScene);

						// Leaf Occurence actor could change depending what instanceRefs setup
						TSharedRef<IDatasmithActorElement> OccurrenceLeafActorElement = OccurrenceActorElement;
						TArray<FIdRef> InstanceRefs = PlmXml::GetAttributeUriReferenceList(OccurrenceNode, TEXT("instanceRefs"));
						if (InstanceRefs.Num())
						{

							// Every 'Occurrence' adds an object to a 'view'(ProductView, PRV only view or others too?) that should
							// be displayed when this 'view' is active(ro whatever it's called, 'shown'). Difference Occurrence's can reference same 'instance' object but
							// modify it, with, for example transform. transformRef is in the doc. or child nodes Transform(takes precedence over transformRef) - absolute, additional Representation, EntityMaterial, AssociatedAttachment>.
							// see 7.5.3 <Occurrence> for details

							// Few examples, contradicting with PLMXML spec:
							// - instancedRef is supposed to be path through instance graph, but 'stand' example has instanceRefs not on InstanceGraph at all, instanceRefs referencing ProductRevision(which is not ProductRevisionView - another contraditiction with the docs)
							// - PLMXML_Occurrence(UE-90772) has Occurrence's instanceRefs not being "a path through instanceGraph starting from a rootRef", doesn't start from rootRef

							TSharedPtr<PlmXml::FImportedInstanceTreeNode> FoundImportedInstanceTreeNode = ImportContext.ImportedInstanceTreeRootNode;
							for (const FIdRef& InstanceRef : InstanceRefs)
							{
								// Check if instance path is valid take the child instance
								if (FoundImportedInstanceTreeNode.IsValid())
								{
									TSharedPtr<PlmXml::FImportedInstanceTreeNode>* Ptr = FoundImportedInstanceTreeNode->ChildInstances.Find(InstanceRef.Value);
									if (!ensure(Ptr))
									{
										UE_LOGF(LogDatasmithPlmXmlImport, Warning, "instanceRef '%ls' doesn't correspond any ProductInstance", *InstanceRef.Value);
									}
									else
									{
										FoundImportedInstanceTreeNode = *Ptr;
									}
								}
								else
								{
									// In case PLMXML is not well-formed and instance path doesnt exist in instanceGraph just take an instance node(by its id)
									if (TSharedPtr<PlmXml::FImportedInstanceTreeNode>* Ptr = ImportContext.ImportedInstanceTreeNodes.Find(InstanceRef.Value))
									{
										FoundImportedInstanceTreeNode = *Ptr;
									}
								}
							}
							if (FoundImportedInstanceTreeNode.IsValid() && FoundImportedInstanceTreeNode->DatasmithActorElement.IsValid())
							{
								OccurrenceLeafActorElement = FoundImportedInstanceTreeNode->DatasmithActorElement.ToSharedRef();
								ActorOptions.Add(OccurrenceLeafActorElement);
							}
						}

						FIdRef InstancedRef = PlmXml::GetUriReferenceId(OccurrenceNode, TEXT("instancedRef"));

						if (InstancedRef)
						{
							if (ImportContext.ParsedPlmXml.ProductRevisionNodes.Contains(InstancedRef.Value))
							{
								FTransform Transform;
								for (const FXmlNode* OccurrenceChildNode : OccurrenceNode->GetChildrenNodes())
								{
									if (OccurrenceChildNode->GetTag() == TEXT("Transform"))
									{
										Transform = ParseTransform(OccurrenceChildNode);
									}
								}

								// Occurrence Transform element sets absolute transform
								// TODO: right now it's only implemented for existing <Transform> element. Which replaces transform of the referenced object and does it as absolute transform.
								// need also:
								// - transformRef(optional, <Transform> takes precedence over it)
								PlmXml::SetActorWorldTransform(OccurrenceActorElement, Transform);

								const FXmlNode* ProductRevisionNode = ImportContext.ParsedPlmXml.ProductRevisionNodes[InstancedRef.Value];
								FImportedProductRevision ImportedProductRevision = ParseProductRevision(ImportContext, OccurrenceLeafActorElement, ProductRevisionNode, OccurrenceActorElement->GetName());
							}
						}

						// TODO: Transform for Occurrence may come from parent Occurrence(by occurrenceRefs)'s referenced instance(partRef?)
						for (const FXmlNode* OccurrenceNodeChildNode : OccurrenceNode->GetChildrenNodes())
						{
							ParseRepresentation(ImportContext, OccurrenceNodeChildNode, OccurrenceLeafActorElement, DatasmithName);
						}
					}
				}

				FString OptionName = PlmXml::GetAttributeId(ProductViewNode); // TODO: make better name, when we have examples with multiple ProductView
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*OptionName);
				ProductViewVarSet->AddVariant(NewVar.ToSharedRef());

				// Add every ProductInstance actor element to the variant set, setting it visible if it's included in the ProductView 
				for (const PlmXml::FImportedProductInstance& ProductInstance : ImportContext.ProductInstances)
				{
					TSharedPtr<IDatasmithPropertyCaptureElement> PropertyCapture = FDatasmithSceneFactory::CreatePropertyCapture();
					PropertyCapture->SetCategory(EDatasmithPropertyCategory::Visibility);

					bool bVisible = ActorOptions.Contains(ProductInstance.ActorElement);
					PropertyCapture->SetRecordedData((uint8*)&bVisible, sizeof(bool));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(ProductInstance.ActorElement);
					Binding->AddPropertyCapture(PropertyCapture.ToSharedRef());
					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}
		}
		return true;
	}

	bool ParseProductDef(const FXmlNode* RootNode, FImportContext& ImportContext)
	{
		const FXmlNode* ProductDefNode = RootNode->FindChildNode("ProductDef");
		if (!ProductDefNode)
		{
			UE_LOGF(LogDatasmithPlmXmlImport, Warning, "PlmXml file has no ProductDef tag");
			return false;
		}

		// TODO: multiple InstanceGraph per ProductDef allowed by spec
		const FXmlNode* InstanceGraphNode = ProductDefNode->FindChildNode("InstanceGraph");
		if (!InstanceGraphNode)
		{
			UE_LOGF(LogDatasmithPlmXmlImport, Warning, "PlmXml file has no InstanceGraph node");
			return false;
		}

		// PLMXMLSchema.xsd
		// rootInstanceRef: if the graph of Instances and StructureRevisionViews
		//    in the InstanceGraph has a single root, which is an Instance,
		//    this may be use to indicate it.
		// rootRefs : however, in general there may be more than one root, and the
		//    roots may be Instances or StructureRevisionViews.The use of
		//    this attribute to specify the root(s) is preferred.
		// 
		// So the search of rootRefs is done first. If it doesn't exist, then rootInstanceRef is search
		ImportContext.InstanceGraphRootRefs = PlmXml::GetAttributeIDREFS(InstanceGraphNode, TEXT("rootRefs"));
		if (!ImportContext.InstanceGraphRootRefs.Num())
		{
			ImportContext.InstanceGraphRootRefs = PlmXml::GetAttributeIDREFS(InstanceGraphNode, TEXT("rootInstanceRef"));
		}

		if (!ImportContext.InstanceGraphRootRefs.Num())
		{
			UE_LOGF(LogDatasmithPlmXmlImport, Warning, "PlmXml file has no rootRefs node");
			return false;
		}

		PlmXml::FParsedProductDef ProductDef;

		// Read all ProductInstance, ProductRevisionView, ProductView and memorize them, they are referenced in PlmXml graph by their 'id's
		for (const FXmlNode* InstanceGraphChildNode : InstanceGraphNode->GetChildrenNodes())
		{
			FString IdText = PlmXml::GetAttributeId(InstanceGraphChildNode);

			if (InstanceGraphChildNode->GetTag() == TEXT("ProductInstance"))
			{
				ProductDef.ProductInstanceNodes.Add(IdText, InstanceGraphChildNode);
			}
			else if (InstanceGraphChildNode->GetTag() == TEXT("ProductRevisionView"))
			{
				ProductDef.ProductRevisionViewNodes.Add(IdText, InstanceGraphChildNode);
			}
		}
		for (const FXmlNode* ProductDefChildNode : ProductDefNode->GetChildrenNodes())
		{
			if (ProductDefChildNode->GetTag() == TEXT("ProductView"))
			{
				ImportContext.ParsedPlmXml.ProductViewNodes.Add(ProductDefChildNode);
			}
		}

		// Traverse PlmXml InstanceGraph and make ActorElement tree from it. Collect all geometry to load later.
		FProductDefImportContext ProductDefImportContext(ImportContext, ProductDef);
		PlmXml::FTraverseRootContext RootContext(ProductDefImportContext, 0);
		for (const FIdRef& RootRef : ImportContext.InstanceGraphRootRefs)
		{
			
			if (ProductDefImportContext.HasProductInstance(RootRef))
			{
				PlmXml::TraverseProductInstance(PlmXml::MakeTraverseProductInstanceContext(RootContext, RootContext.ImportContext.GetProductInstance(RootRef)));
			}
			else if (ProductDefImportContext.HasProductRevisionView(RootRef)) // in case root ref is not a product instance
			{
				FTransform RootTransform;
				RootTransform.SetIdentity();
				PlmXml::TraverseProductRevisionView(PlmXml::MakeTraverseContext(RootContext, RootTransform, ProductDefImportContext.GetProductRevisionView(RootRef)));
			}
		}

		return true;
	}
}

bool FDatasmithPlmXmlImporter::OpenFile(const FString& InFilePath, const FDatasmithSceneSource& Source, FDatasmithTessellationOptions& TessellationOptions)
{
	FXmlFile PlmxmlFile;

	if (!PlmxmlFile.LoadFile(InFilePath))
	{
		UE_LOGF(LogDatasmithPlmXmlImport, Error, "Couldn't open PlmXml file: %ls", *PlmxmlFile.GetLastError());
		return false;
	}

	const FXmlNode* RootNode = PlmxmlFile.GetRootNode();
	if (!RootNode)
	{
		UE_LOGF(LogDatasmithPlmXmlImport, Error, "PlmXml file has no Root node");
		return false;
	}

	MeshLoader = MakeUnique<PlmXml::FPlmXmlMeshLoader>(DatasmithScene, TessellationOptions);

	PlmXml::FImportContext ImportContext(InFilePath, *MeshLoader.Get(), DatasmithScene, Source);

	PlmXml::FParsedPlmXml& ParsedPlmXml = ImportContext.ParsedPlmXml;

	// Read all ExternalFile, DataSet and memoize them, some are referenced from other parts of PlmXml by their 'id's
	for (const FXmlNode* RootChildNode : RootNode->GetChildrenNodes())
	{
		FString IdText = PlmXml::GetAttributeId(RootChildNode);

		if (RootChildNode->GetTag() == TEXT("ExternalFile"))
		{
			ParsedPlmXml.ExternalFileNodes.Add(IdText, RootChildNode);
		}
		else if (RootChildNode->GetTag() == TEXT("DataSet"))
		{
			ParsedPlmXml.DataSetNodes.Add(IdText, RootChildNode);
		}
		else if (RootChildNode->GetTag() == TEXT("ProductRevision"))
		{
			ParsedPlmXml.ProductRevisionNodes.Add(IdText, RootChildNode);
		}
		else if (RootChildNode->GetTag() == TEXT("ProductView"))
		{
			ParsedPlmXml.ProductViewNodes.Add(RootChildNode);
		}
	}
	
	bool bProductDefParsed = PlmXml::ParseProductDef(RootNode, ImportContext);
	bool bProductViewVariantsAdded = AddProductViewVariants(ImportContext);


	if (!(bProductDefParsed || bProductViewVariantsAdded))
	{
		return false;
	}

	ImportContext.DatasmithScene->AddLevelVariantSets(ImportContext.LVS);
	
	// Load all geometry referenced in InstanceGraph. This is done in one go in order to fully use multiprocessing of DatasmithDispatcher
	ImportContext.MeshLoader.Process(ImportContext.Source);

	// Instantiate Representations using loaded geometry
	for (const PlmXml::FImportedRepresentationInstance& RepresentationInstance : ImportContext.RepresentationInstances)
	{
		TSharedPtr<IDatasmithActorElement> ActorElementPtr = ImportContext.MeshLoader.InstantiateMesh(RepresentationInstance.InstanceDatasmithName, RepresentationInstance.ImportedRepresentation.ImportedMeshId);

		if (ActorElementPtr.IsValid())
		{
			TSharedRef<IDatasmithActorElement> ActorElement = ActorElementPtr.ToSharedRef();

			if (RepresentationInstance.bOverrideWorldTransform)
			{
				// Set Actor World Transform
				PlmXml::SetActorWorldTransform(ActorElement, RepresentationInstance.Transform);

				// Add actor as child keeping its transform set above as absolute
				RepresentationInstance.Parent->AddChild(ActorElement, EDatasmithActorAttachmentRule::KeepWorldTransform);
			}
			else
			{
				// Add actor as child interpreting its transform as relative
				// TODO: this needs to be relative - FTransform::Multiply(WorldTransform, ParentTransform, LocalTransform);
				RepresentationInstance.Parent->AddChild(ActorElement, EDatasmithActorAttachmentRule::KeepRelativeTransform);
			}
			
		}
	}

	return true;
}

bool FDatasmithPlmXmlImporter::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	return MeshLoader->LoadStaticMesh(MeshElement, OutMeshPayload);
}

void FDatasmithPlmXmlImporter::UnloadScene()
{
	if (MeshLoader.IsValid())
	{
		MeshLoader->UnloadScene();
	}
}
#undef LOCTEXT_NAMESPACE
