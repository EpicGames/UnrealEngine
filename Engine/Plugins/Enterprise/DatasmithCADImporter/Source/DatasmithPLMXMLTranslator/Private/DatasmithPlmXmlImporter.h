// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FDatasmithSceneSource;
class IDatasmithScene;
class IDatasmithMeshElement;
struct FDatasmithMeshElementPayload;
struct FDatasmithTessellationOptions;

namespace PlmXml
{
	class FPlmXmlMeshLoaderWithDatasmithDispatcher;
	using FPlmXmlMeshLoader = FPlmXmlMeshLoaderWithDatasmithDispatcher;
}

class FDatasmithPlmXmlImporter : FNoncopyable
{
public:
	explicit FDatasmithPlmXmlImporter(TSharedRef<IDatasmithScene> OutScene);
	~FDatasmithPlmXmlImporter();

	bool OpenFile(const FString& InFilePath, const FDatasmithSceneSource& Source, FDatasmithTessellationOptions& TessellationOptions);
	bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload);

	void UnloadScene();
private:
	TSharedRef<IDatasmithScene> DatasmithScene;

	TUniquePtr<PlmXml::FPlmXmlMeshLoader> MeshLoader;
	
};

