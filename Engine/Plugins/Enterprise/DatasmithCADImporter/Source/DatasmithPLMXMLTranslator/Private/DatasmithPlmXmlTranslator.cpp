// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithPlmXmlTranslator.h"
#include "DatasmithPlmXmlImporter.h"
#include "DatasmithPlmXmlTranslatorModule.h"

#include "CADInterfacesModule.h"

#include "CADData.h"
#include "CADOptions.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneSource.h"
#include "DatasmithImportOptions.h"
#include "IDatasmithSceneElements.h"

#include "CoreGlobals.h"
#if WITH_EDITOR
#include "Editor.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogDatasmithXMLPLMTranslator, Log, All)

void FDatasmithPlmXmlTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
{
#if WITH_EDITOR
	if (GIsEditor && !GEditor->PlayWorld && !GIsPlayInEditorWorld)
	{
		TFunction<bool()> GetCADInterfaceAvailability = []() -> bool
		{
			if (ICADInterfacesModule::GetAvailability() == ECADInterfaceAvailability::Unavailable)
			{
				UE_LOGF(LogDatasmithXMLPLMTranslator, Warning, "CAD Interface module is unavailable. Most of CAD formats (except to Rhino and Alias formats) cannot be imported.");
				return false;
			}
			return true;
		};
		static bool bIsCADInterfaceAvailable = GetCADInterfaceAvailability();

		OutCapabilities.bIsEnabled = true;
		OutCapabilities.bParallelLoadStaticMeshSupported = true;

		TArray<FFileFormatInfo>& Formats = OutCapabilities.SupportedFileFormats;
		Formats.Emplace(TEXT("plmxml"), TEXT("PLMXML"));
		Formats.Emplace(TEXT("xml"), TEXT("PLMXML"));

		return;
	}
#endif

	OutCapabilities.bIsEnabled = false;
}

bool FDatasmithPlmXmlTranslator::IsSourceSupported(const FDatasmithSceneSource& Source)
{
	if (Source.GetSourceFileExtension() != TEXT("xml"))
	{
		return true;
	}

	return Datasmith::CheckXMLFileSchema(Source.GetSourceFile(), TEXT("PLMXML"));
}

bool FDatasmithPlmXmlTranslator::LoadScene(TSharedRef<IDatasmithScene> OutScene)
{
	using namespace CADLibrary;

	const FString& FilePath = GetSource().GetSourceFile();

	FFileDescriptor FileDescriptor(*FPaths::ConvertRelativePathToFull(FilePath));

	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "PlmXml translation [%ls].", *FileDescriptor.GetSourcePath());
	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, " - Parsing Library:      %ls", TEXT("TechSoft"));
	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, " - Tessellation Library: %ls" , FImportParameters::bGDisableCADKernelTessellation ? TEXT("TechSoft") : TEXT("CADKernel"));
	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, " - Cache mode:           %ls" , FImportParameters::bGEnableCADCache ? (FImportParameters::bGOverwriteCache ? TEXT("Override") : TEXT("Enabled")) : TEXT("Disabled"));
	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, " - Processing:           %ls" , FImportParameters::bGEnableCADCache ? (GMaxImportThreads == 1 ? TEXT("Sequencial") : TEXT("Parallel")) : TEXT("Sequencial"));

	TFunction<double(double, double, const TCHAR*)> CheckParameterValue = [](double Value, double MinValue, const TCHAR* ParameterName) -> double
	{
		if (Value < MinValue)
		{
			UE_LOGF(LogDatasmithXMLPLMTranslator, Warning, "%ls value (%f) of tessellation parameters is smaller than the minimal value %f. It's value is modified to respect the limit", ParameterName, Value, MinValue);
			return MinValue;
		}
		return Value;
	};

	CommonTessellationOptions.ChordTolerance = CheckParameterValue(CommonTessellationOptions.ChordTolerance, UE::DatasmithTessellation::MinTessellationChord, TEXT("Chord tolerance"));
	CommonTessellationOptions.MaxEdgeLength = FMath::IsNearlyZero(CommonTessellationOptions.MaxEdgeLength) ? 0. : CheckParameterValue(CommonTessellationOptions.MaxEdgeLength, UE::DatasmithTessellation::MinTessellationEdgeLength, TEXT("Max Edge Length"));
	CommonTessellationOptions.NormalTolerance = CheckParameterValue(CommonTessellationOptions.NormalTolerance, UE::DatasmithTessellation::MinTessellationAngle, TEXT("Max Angle"));

	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, " - Import parameters:");
	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "     - ChordTolerance:     %f", CommonTessellationOptions.ChordTolerance);
	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "     - MaxEdgeLength:      %f", CommonTessellationOptions.MaxEdgeLength);
	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "     - MaxNormalAngle:     %f", CommonTessellationOptions.NormalTolerance);

	FString StitchingTechnique;
	switch ((EStitchingTechnique)CommonTessellationOptions.StitchingTechnique)
	{
	case EStitchingTechnique::StitchingHeal:
		StitchingTechnique = TEXT("Heal");
		break;
	case EStitchingTechnique::StitchingSew:
		StitchingTechnique = TEXT("Sew");
		break;
	default:
		StitchingTechnique = TEXT("None");
		break;
	}
	UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "     - StitchingTechnique: %ls", *StitchingTechnique);

	if (!FImportParameters::bGDisableCADKernelTessellation)
	{
		UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "     - Stitching Options:");
		UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "         - ForceSew:              %ls", FImportParameters::bGStitchingForceSew ? TEXT("True") : TEXT("False"));
		UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "         - RemoveThinFaces:       %ls", FImportParameters::bGStitchingRemoveThinFaces ? TEXT("True") : TEXT("False"));
		UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "         - RemoveDuplicatedFaces: %ls", FImportParameters::bGStitchingRemoveDuplicatedFaces ? TEXT("True") : TEXT("False"));
		UE_LOGF(LogDatasmithXMLPLMTranslator, Display, "         - ForceFactor:           %f", FImportParameters::GStitchingForceFactor);
	}

	OutScene->SetHost(TEXT("PlmXmlTranslator"));
	OutScene->SetProductName(TEXT("PlmXml"));

    Importer = MakeUnique<FDatasmithPlmXmlImporter>(OutScene);

	if (!Importer->OpenFile(FilePath, GetSource(), CommonTessellationOptions))
	{
		return false;
	}

	return true;
}

void FDatasmithPlmXmlTranslator::UnloadScene()
{
	if (Importer)
	{
		Importer->UnloadScene();
		Importer.Reset();
	}
}

bool FDatasmithPlmXmlTranslator::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	if (ensure(Importer.IsValid()))
	{
		return Importer->LoadStaticMesh(MeshElement, OutMeshPayload);
	}

	return false;
}

void FDatasmithPlmXmlTranslator::GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	TObjectPtr<UDatasmithCommonTessellationOptions> CommonTessellationOptionsPtr = Datasmith::MakeOptionsObjectPtr<UDatasmithCommonTessellationOptions>();

	CommonTessellationOptionsPtr->Options = CommonTessellationOptions;

	Options.Add(CommonTessellationOptionsPtr);
}

void FDatasmithPlmXmlTranslator::SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	for (const TObjectPtr<UDatasmithOptionsBase>& OptionPtr : Options)
	{
		if (UDatasmithCommonTessellationOptions* TessellationOptionsObject = Cast<UDatasmithCommonTessellationOptions>(OptionPtr))
		{
			CommonTessellationOptions = TessellationOptionsObject->Options;
		}
	}
}
