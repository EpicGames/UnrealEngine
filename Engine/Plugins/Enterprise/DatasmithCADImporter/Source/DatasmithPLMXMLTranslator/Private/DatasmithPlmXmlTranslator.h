// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithTranslator.h"
#include "DatasmithPlmXmlImporter.h"
#include "DatasmithImportOptions.h"
#include "CoreMinimal.h"

class IDatasmithScene;
class IDatasmithMeshElement;
class FDatasmithPlmXmlImporter;
class UDatasmithCommonTessellationOptions;

class FDatasmithPlmXmlTranslator : public IDatasmithTranslator
{
public:

	// IDatasmithTranslator interface
	virtual FName GetFName() const override { return "DatasmithPlmXmlTranslator"; };
	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;
	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;
	virtual void UnloadScene() override;

	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) override;

	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	virtual void SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	virtual bool IsSourceSupported(const FDatasmithSceneSource& Source) override;
	//~ End IDatasmithTranslator interface

private:
    TUniquePtr<FDatasmithPlmXmlImporter> Importer;
	FDatasmithTessellationOptions CommonTessellationOptions;
};
