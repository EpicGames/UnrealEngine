// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithPlmXmlTranslatorModule.h"
#include "DatasmithPlmXmlTranslator.h"

#include "DatasmithTranslator.h"
#include "CoreMinimal.h"

const TCHAR* IDatasmithPlmXmlTranslatorModule::ModuleName = TEXT("DatasmithPlmXmlTranslator");

class FPlmXmlTranslatorModule : public IDatasmithPlmXmlTranslatorModule
{
public:
	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModule(TEXT("DatasmithImporter"));
		Datasmith::RegisterTranslator<FDatasmithPlmXmlTranslator>();
	}

	virtual void ShutdownModule() override
	{
		Datasmith::UnregisterTranslator<FDatasmithPlmXmlTranslator>();
	}
};

IMPLEMENT_MODULE(FPlmXmlTranslatorModule, DatasmithPLMXMLTranslator);
