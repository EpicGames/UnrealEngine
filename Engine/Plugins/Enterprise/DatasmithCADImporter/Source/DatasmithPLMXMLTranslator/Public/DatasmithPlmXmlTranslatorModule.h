// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class IDatasmithPlmXmlTranslatorModule : public IModuleInterface
{
public:
	static const TCHAR* ModuleName;

	/**
	* Singleton-like access to the module
	*
	* @return Returns module singleton instance, loading it on demand if needed
	*/
	static IDatasmithPlmXmlTranslatorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDatasmithPlmXmlTranslatorModule>(ModuleName);
	}

	/**
	* Checks to see if this module is loaded and ready. It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}
};
