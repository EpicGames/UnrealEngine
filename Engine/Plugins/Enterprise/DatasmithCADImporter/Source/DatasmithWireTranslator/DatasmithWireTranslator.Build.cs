// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class DatasmithWireTranslator : ModuleRules
{
	public DatasmithWireTranslator(ReadOnlyTargetRules Target) : base(Target)
	{
		//bUseUnity = false;
		//OptimizeCode = CodeOptimization.Never;
		//PCHUsage = ModuleRules.PCHUsageMode.NoPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DatasmithContent",
				"DatasmithTranslator",
				"ParametricSurface",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
 					"MessageLog",
 					"UnrealEd",
				}
			);
		}
	}
}