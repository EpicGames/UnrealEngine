// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithWireTranslator.h"

#include "IWireInterface.h"

#include "GenericPlatform/GenericPlatformTLS.h"
#include "HAL/ConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <windows.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogDatasmithWireTranslator, Log, All);

#define LOCTEXT_NAMESPACE "DatasmithWireTranslator"

static TAutoConsoleVariable<bool> CVarAliasThreadSafe(
	TEXT("ds.WireTranslator.ThreadSafe"),
	false,
	TEXT("If true, the translator will be called in more than one thread. Default false.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarAliasUseNative(
	TEXT("ds.WireTranslator.UseNative"),
	false,
	TEXT("If true, the AlaisStudio tessellator will be used. Default false.\n"),
	ECVF_Default);

static TMap<uint32, FInterfaceMaker> RegisteredInterfaces;

namespace WireTranslator
{
	TSharedPtr<IWireInterface> GetInterfaceFromFile(const TCHAR* Filename)
	{
		TSharedPtr<IWireInterface> WireInterface;

		for (const TPair<uint32, FInterfaceMaker>& Entry : RegisteredInterfaces)
		{
			WireInterface = Entry.Value();
			if (WireInterface->Initialize(Filename))
			{
				return WireInterface;
			}
		}

		return TSharedPtr<IWireInterface>();
	}

	bool IsFileSupported(const TCHAR* Filename)
	{
		TSharedPtr<IWireInterface> WireInterface;

		for (const TPair<uint32, FInterfaceMaker>& Entry : RegisteredInterfaces)
		{
			WireInterface = Entry.Value();
			if (WireInterface->Initialize(Filename))
			{
				return true;
			}
		}

		return false;
	}
}

uint64 IWireInterface::AliasVersion = 0;

static TAutoConsoleVariable<FString> CVarAliasDllDirectory(
	TEXT("WireTranslator.Binary.Directory"),
	TEXT(""),
	TEXT("Controls which Alias API dll to load.\n"),
	ECVF_Default);

uint64 IWireInterface::GetRequiredAliasVersion(uint64 AliasMaxVersion)
{
#if PLATFORM_WINDOWS
	static bool bInitialized = [&AliasMaxVersion]() -> bool
		{
			const TCHAR* AliasDllName = TEXT("libalias_api.dll");

			const FString AliasDllDirectory = CVarAliasDllDirectory->GetString();
			if (!AliasDllDirectory.IsEmpty())
			{
				FString AliasDllPath = AliasDllDirectory / AliasDllName;
				FPaths::MakePlatformFilename(AliasDllPath);
				if (FPaths::FileExists(AliasDllPath))
				{
					uint64 AliasDllVersion = FPlatformMisc::GetFileVersion(*AliasDllPath);
					if (AliasDllVersion < AliasMaxVersion && AliasDllVersion > 0)
					{
						if (void* DllHandle = ::LoadLibraryExW(*AliasDllPath, nullptr, LOAD_LIBRARY_SEARCH_DLL_LOAD_DIR | LOAD_LIBRARY_SEARCH_SYSTEM32))
						{
							AliasVersion = AliasDllVersion;
							return true;
						}
					}
				}
			}

			FString BestAliasDllPath;
			uint64 AliasBestVersion = 0;

			TArray<FString> PathArray;
			FPlatformMisc::GetEnvironmentVariable(TEXT("PATH")).ParseIntoArray(PathArray, FPlatformMisc::GetPathVarDelimiter());
			for (const FString& Path : PathArray)
			{
				const FString AliasDllPath = Path / AliasDllName;
				if (FPaths::FileExists(AliasDllPath))
				{
					uint64 AliasDllVersion = FPlatformMisc::GetFileVersion(*AliasDllPath);
					if (AliasDllVersion < AliasMaxVersion && AliasDllVersion > AliasBestVersion)
					{
						AliasBestVersion = AliasDllVersion;
						BestAliasDllPath = Path;
					}
				}
			}

			if (!BestAliasDllPath.IsEmpty())
			{
				FString AliasDllPath = BestAliasDllPath / AliasDllName;
				FPaths::MakePlatformFilename(AliasDllPath);
				if(void* DllHandle = ::LoadLibraryExW(*AliasDllPath, nullptr, LOAD_LIBRARY_SEARCH_DLL_LOAD_DIR | LOAD_LIBRARY_SEARCH_SYSTEM32))
				{
					AliasVersion = AliasBestVersion;
				}
			}

			return true;
		}();
#endif

	return AliasVersion;
}

void IWireInterface::RegisterInterface(int16 MajorVersion, int16 MinorVersion, FInterfaceMaker&& MakeInterface)
{
	uint32 Version = (MajorVersion << 16) + MinorVersion;
	RegisteredInterfaces.Emplace(Version, MoveTemp(MakeInterface));
}

FDatasmithWireTranslator::FDatasmithWireTranslator()
{
}

bool FDatasmithWireTranslator::CanTranslate()
{
	static const bool bCanTranslate = []() -> bool
		{
			if (IWireInterface::AliasVersion == 0)
			{
				return false;
			}

			return RegisteredInterfaces.Num() > 0 ? true : false;
		}();

		return bCanTranslate;
}

void FDatasmithWireTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
{
	if (!CanTranslate())
	{
		OutCapabilities.bIsEnabled = false;
		return;
	}

	const int32 MajorVersion = (IWireInterface::AliasVersion & 0xffff0000) >> 16;
	const int32 MinorVersion = IWireInterface::AliasVersion & 0x0000ffff;

	const FString FormatDescription = FString::Printf(TEXT("AliasStudio %d.%d model files"), MajorVersion, MinorVersion);

	OutCapabilities.SupportedFileFormats.Add(FFileFormatInfo{ TEXT("wire"), *FormatDescription });

	const IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("ds.Wiretranslator.ThreadSafe"));
	OutCapabilities.bParallelLoadStaticMeshSupported = ConsoleVariable ? ConsoleVariable->GetBool() : false;

	OutCapabilities.bIsEnabled = true;
}

bool FDatasmithWireTranslator::IsSourceSupported(const FDatasmithSceneSource& Source)
{
	return WireTranslator::IsFileSupported(*Source.GetSourceFile());
}

bool FDatasmithWireTranslator::LoadScene(TSharedRef<IDatasmithScene> OutScene)
{
	ensure(WireImportOptions);
	WireInterface = WireTranslator::GetInterfaceFromFile(*GetSource().GetSourceFile());
	if (!WireInterface.IsValid())
	{
		return false;
	}

	WireInterface->SetImportSettings(WireImportOptions->Settings);

	static const FString CacheRootDir = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("WireTranslator"), TEXT("Cache")));
	
	const FString OutputPath = FPaths::Combine(CacheRootDir, GetSource().GetSceneName());
	IFileManager::Get().MakeDirectory(*OutputPath, true);

	WireInterface->SetOutputPath(OutputPath);

	return WireInterface->Load(OutScene);
}

void FDatasmithWireTranslator::UnloadScene()
{
	WireInterface.Reset();
}

bool FDatasmithWireTranslator::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	return ensure(WireInterface.IsValid()) ? WireInterface->LoadStaticMesh(MeshElement, OutMeshPayload, CommonTessellationOptions) : false;
}

void FDatasmithWireTranslator::GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	UDatasmithWireOptions* ImportOptions = Datasmith::MakeOptionsObjectPtr<UDatasmithWireOptions>();
	ImportOptions->LoadConfig();
	Options.Add(ImportOptions);
}

void FDatasmithWireTranslator::SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	FParametricSurfaceTranslator::SetSceneImportOptions(Options);

	WireImportOptions = nullptr;

	for (const TObjectPtr<UDatasmithOptionsBase>& OptionPtr : Options)
	{
		if (UDatasmithWireOptions* ImportOptions = Cast<UDatasmithWireOptions>(OptionPtr))
		{
			ImportOptions->SaveConfig(CPF_Config);
			WireImportOptions = ImportOptions;
			CommonTessellationOptions = ImportOptions->Settings;
		}
	}
}

class FDatasmithWireTranslatorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		Datasmith::RegisterTranslator<FDatasmithWireTranslator>();
	}

	virtual void ShutdownModule() override
	{
		Datasmith::UnregisterTranslator<FDatasmithWireTranslator>();
	}
};

IMPLEMENT_MODULE(FDatasmithWireTranslatorModule, DatasmithWireTranslator);


#undef LOCTEXT_NAMESPACE // "DatasmithWireTranslator"
