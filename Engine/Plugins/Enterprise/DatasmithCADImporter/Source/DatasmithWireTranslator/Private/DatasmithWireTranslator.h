// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IWireInterface.h"

#include "ParametricSurfaceTranslator.h"

#include "DatasmithWireTranslator.generated.h"


UCLASS(BlueprintType, config = EditorPerProjectUserSettings)
class UDatasmithWireOptions : public UDatasmithOptionsBase
{
	GENERATED_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Wire Translation Options", meta = (ShowOnlyInnerProperties))
	FWireSettings Settings;
};

class FDatasmithWireTranslator : public FParametricSurfaceTranslator
{
public:
	FDatasmithWireTranslator();

	// Begin IDatasmithTranslator overrides
	virtual FName GetFName() const override { return "DatasmithWireTranslator"; };

	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;

	virtual bool IsSourceSupported(const FDatasmithSceneSource& Source) override;

	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;
	virtual void UnloadScene() override;

	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) override;
	// End IDatasmithTranslator overrides

	// Begin ADatasmithCoreTechTranslator overrides
	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	virtual void SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;

protected:
	virtual void InitCommonTessellationOptions(FDatasmithTessellationOptions& TessellationOptions) override
	{
		TessellationOptions.StitchingTechnique = EDatasmithCADStitchingTechnique::StitchingNone;
	}
	// End ADatasmithCoreTechTranslator overrides

private:
	bool CanTranslate();

private:
	TSharedPtr<IWireInterface> WireInterface;
	TObjectPtr<UDatasmithWireOptions> WireImportOptions;
};
