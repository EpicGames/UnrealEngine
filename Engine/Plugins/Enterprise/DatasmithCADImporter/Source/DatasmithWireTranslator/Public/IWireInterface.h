// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithImportOptions.h"

#include "UObject/ObjectMacros.h"

#include "IWireInterface.generated.h"

class IDatasmithMeshElement;
class IDatasmithScene;

struct FDatasmithTessellationOptions;
struct FDatasmithMeshElementPayload;

typedef TFunction<TSharedPtr<class IWireInterface>()> FInterfaceMaker;

USTRUCT(BlueprintType)
struct FWireSettings : public FDatasmithTessellationOptions
{
	GENERATED_BODY()

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Wire Translation Options", meta = (ToolTip = "If set to true, the first level's actors in the outliner are the layers. Default is true"))
	bool bUseLayerAsActor = true;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Wire Translation Options", meta = (ToolTip = "If set to true, all geometry nodes under a group are merged. Default is true."))
	bool bMergeGeometryByGroup = true;

	uint32 GetHash() const
	{
		return HashCombine(FDatasmithTessellationOptions::GetHash(), GetTypeHash(bUseLayerAsActor));
	}

	bool bAliasUseNative = false;
};

class IWireInterface
{
public:
	virtual ~IWireInterface() {}

	virtual bool Initialize(const TCHAR* Filename = nullptr) = 0;

	virtual bool Load(TSharedPtr<IDatasmithScene> Scene) = 0;

	virtual void SetImportSettings(const FWireSettings& Settings) = 0;
	virtual void SetOutputPath(const FString& Path) = 0;
	virtual bool LoadStaticMesh(const TSharedPtr<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload, const FDatasmithTessellationOptions& InTessellationOptions) = 0;

	static uint64 AliasVersion;

	static DATASMITHWIRETRANSLATOR_API uint64 GetRequiredAliasVersion(uint64 AliasMaxVersion = 0xffffffffffffffffull);
	static DATASMITHWIRETRANSLATOR_API void RegisterInterface(int16 MajorVersion, int16 MinorVersion, FInterfaceMaker&& MakeInterface);
};
