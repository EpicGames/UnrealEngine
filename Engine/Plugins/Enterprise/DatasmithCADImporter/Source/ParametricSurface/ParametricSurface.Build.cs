// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ParametricSurface : ModuleRules
	{
		public ParametricSurface(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"CADInterfaces",
					"CADKernel",
					"CADLibrary",
					"CADTools",
					"DatasmithContent",
					"DatasmithTranslator",
					"Engine",
					"Json",
					"MeshDescription",
					"StaticMeshDescription",
				}
			);
		}
	}
}
