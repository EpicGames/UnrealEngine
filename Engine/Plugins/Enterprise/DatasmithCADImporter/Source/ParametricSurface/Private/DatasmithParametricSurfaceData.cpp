// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithParametricSurfaceData.h"

#include "CADOptions.h"

#include "Misc/FileHelper.h"
#include "UObject/EnterpriseObjectVersion.h"

#define LOCTEXT_NAMESPACE "ParametricSurfaceData"

bool UDatasmithParametricSurfaceData::SetFile(const TCHAR* FilePath)
{
	if (FPaths::FileExists(FilePath))
	{
		TArray<uint8> ByteArray;

		if (FFileHelper::LoadFileToArray(ByteArray, FilePath))
		{
			RawData = MoveTemp(ByteArray);
			return true;
		}
	}

	return false;
}

void UDatasmithParametricSurfaceData::SetImportParameters(const CADLibrary::FImportParameters& InSceneParameters)
{
	SceneParameters.ModelCoordSys = uint8(InSceneParameters.GetModelCoordSys());
}

void UDatasmithParametricSurfaceData::SetMeshParameters(const CADLibrary::FMeshParameters& InMeshParameters)
{
	MeshParameters.bNeedSwapOrientation = InMeshParameters.bNeedSwapOrientation;
	MeshParameters.bIsSymmetric = InMeshParameters.bIsSymmetric;
	MeshParameters.SymmetricNormal = (FVector) InMeshParameters.SymmetricNormal;
	MeshParameters.SymmetricOrigin = (FVector) InMeshParameters.SymmetricOrigin;
}

void UDatasmithParametricSurfaceData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEnterpriseObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsSaving() || (Ar.IsLoading() && Ar.CustomVer(FEnterpriseObjectVersion::GUID) >= FEnterpriseObjectVersion::CoreTechParametricSurfaceOptim))
	{
		Ar << RawData;
	}

	if (RawData_DEPRECATED.Num() && RawData.Num() == 0)
	{
		RawData = MoveTemp(RawData_DEPRECATED);
	}
}

#undef LOCTEXT_NAMESPACE // "ParametricSurfaceData"

