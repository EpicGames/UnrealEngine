// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParametricSurfaceModule.h"

#include "TechSoft/TechSoftParametricSurface.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/CoreRedirects.h"

#define LOCTEXT_NAMESPACE "ParametricSurfaceModule"

void FParametricSurfaceModule::StartupModule()
{
}

FParametricSurfaceModule& FParametricSurfaceModule::Get()
{
	return FModuleManager::LoadModuleChecked< FParametricSurfaceModule >(PARAMETRICSURFACE_MODULE_NAME);
}

bool FParametricSurfaceModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(PARAMETRICSURFACE_MODULE_NAME);
}

UDatasmithParametricSurfaceData* FParametricSurfaceModule::CreateParametricSurface()
{
	return Datasmith::MakeAdditionalData<UTechSoftParametricSurfaceData>();
}

IMPLEMENT_MODULE(FParametricSurfaceModule, ParametricSurface)

#undef LOCTEXT_NAMESPACE // "ParametricSurfaceModule"

