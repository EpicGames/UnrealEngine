// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParametricSurfaceTranslator.h"

#include "DatasmithParametricSurfaceData.h"
#include "ParametricSurfaceModule.h"

#include "CADOptions.h"

#include "DatasmithImportOptions.h"
#include "IDatasmithSceneElements.h"

#include "Misc/FileHelper.h"

FParametricSurfaceTranslator::FParametricSurfaceTranslator()
{
	// Initialize bUseCADKernel with current value of CVar ds.CADTranslator.DisableCADKernelTessellation
	CommonTessellationOptions.bUseCADKernel = !CADLibrary::FImportParameters::bGDisableCADKernelTessellation;
}

void FParametricSurfaceTranslator::GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	FString Extension = GetSource().GetSourceFileExtension();
	if (Extension == TEXT("cgr") || Extension == TEXT("3dxml"))
	{
		return;
	}

	TObjectPtr<UDatasmithCommonTessellationOptions> CommonTessellationOptionsPtr = Datasmith::MakeOptionsObjectPtr<UDatasmithCommonTessellationOptions>();
	check(CommonTessellationOptionsPtr);
	InitCommonTessellationOptions(CommonTessellationOptionsPtr->Options);

	Options.Add(CommonTessellationOptionsPtr);
}

void FParametricSurfaceTranslator::SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	for (const TObjectPtr<UDatasmithOptionsBase>& OptionPtr : Options)
	{
		if (UDatasmithCommonTessellationOptions* TessellationOptionsObject = Cast<UDatasmithCommonTessellationOptions>(OptionPtr))
		{
			CommonTessellationOptions = TessellationOptionsObject->Options;
			TessellationOptionsObject->SaveConfig(CPF_Config);
		}
	}
}

bool ParametricSurfaceUtils::AddSurfaceData(const TCHAR* MeshFilePath, const CADLibrary::FImportParameters& ImportParameters, const CADLibrary::FMeshParameters& InMeshParameters, const FDatasmithTessellationOptions& InCommonTessellationOptions, FDatasmithMeshElementPayload& OutMeshPayload)
{
	if (MeshFilePath && IFileManager::Get().FileExists(MeshFilePath))
	{
		UDatasmithParametricSurfaceData* ParametricSurfaceData = FParametricSurfaceModule::CreateParametricSurface();

		if (!ParametricSurfaceData || !ParametricSurfaceData->SetFile(MeshFilePath))
		{
			return false;
		}

		ParametricSurfaceData->SetImportParameters(ImportParameters);
		ParametricSurfaceData->SetMeshParameters(InMeshParameters);
		ParametricSurfaceData->SetLastTessellationOptions(InCommonTessellationOptions);

		OutMeshPayload.AdditionalData.Add(ParametricSurfaceData);

		return true;
	}

	return false;
}
