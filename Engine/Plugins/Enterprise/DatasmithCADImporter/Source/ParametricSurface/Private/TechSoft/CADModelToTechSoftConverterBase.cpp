// Copyright Epic Games, Inc. All Rights Reserved.

#include "CADModelToTechSoftConverterBase.h"


#include "CADData.h"
#include "CADMeshDescriptionHelper.h"
#include "ParametricSurfaceTranslator.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "TechSoftInterface.h"
#include "TechSoftUtils.h"
#include "TUniqueTechSoftObj.h"

bool FCADModelToTechSoftConverterBase::RepairTopology()
{
#ifdef USE_TECHSOFT_SDK
	// Apply stitching if applicable
	if (ImportParameters.GetStitchingTechnique() != CADLibrary::StitchingNone)
	{
		CADLibrary::TUniqueTSObj<A3DSewOptionsData> SewOptionsData;
		SewOptionsData->m_bComputePreferredOpenShellOrientation = false;

		A3DRiBrepModel** OutNewBReps;
		uint32 OutNewBRepCount;
		const double SewTolerance = CADLibrary::FImportParameters::GStitchingTolerance;
		const double FileUnit = 1.; // BReps from Rhino and Alias are converted in mm
		CADLibrary::TechSoftInterface::SewBReps((A3DRiBrepModel**) RiRepresentationItems.GetData(), RiRepresentationItems.Num(), SewTolerance, FileUnit, SewOptionsData.GetPtr(), &OutNewBReps, OutNewBRepCount);

		RiRepresentationItems.Empty(OutNewBRepCount);
		for (uint32 Index = 0; Index < OutNewBRepCount; ++Index)
		{
			RiRepresentationItems.Add(OutNewBReps[Index]);
		}
	}
#endif
	return true;
}

void FCADModelToTechSoftConverterBase::InitializeProcess()
{
	RiRepresentationItems.Empty();
	ModelFile.Reset();
}


bool FCADModelToTechSoftConverterBase::SaveModel(const TCHAR* InFolderPath, TSharedPtr<IDatasmithMeshElement> MeshElement)
{
#ifdef USE_TECHSOFT_SDK
	FString FilePath = FPaths::Combine(InFolderPath, MeshElement->GetName()) + TEXT(".prc");

	FString JsonString;
	{
		// Save body unit and default color and material attributes in a json string
		// This will be used when the file is reloaded
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		JsonObject->SetNumberField(JSON_ENTRY_BODY_UNIT, 0.1); // ALL BRep from Alias or Rhino are defined in mm

		TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&JsonString);

		FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);
	}

	ModelFile = CADLibrary::TechSoftUtils::SaveBodiesToPrcFile(RiRepresentationItems.GetData(), RiRepresentationItems.Num(), FilePath, JsonString);

	MeshElement->SetFile(*FilePath);
#endif
	return true;
}

bool FCADModelToTechSoftConverterBase::Tessellate(const CADLibrary::FMeshParameters& InMeshParameters, FMeshDescription& OutMeshDescription)
{
	CADLibrary::FBodyMesh BodyMesh;
	BodyMesh.BodyID = 1;

#ifdef USE_TECHSOFT_SDK
	for (A3DRiRepresentationItem* Representation : RiRepresentationItems)
	{
		const double BodyUnit = 0.1; // CAD Models from (Wire or Rhino) imported in TechSoft are imported in mm so BodyUnit = 0.1 
		CADLibrary::TechSoftUtils::FillBodyMesh(Representation, ImportParameters, BodyUnit, BodyMesh);
	}
#endif

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	CADLibrary::TechSoftUtils::RestoreMaterials(JsonObject, BodyMesh);

	if (BodyMesh.Faces.Num() == 0)
	{
		return false;
	}

	CADLibrary::FMeshConversionContext MeshConversionContext(ImportParameters, InMeshParameters);
	return CADLibrary::ConvertBodyMeshToMeshDescription(MeshConversionContext, BodyMesh, OutMeshDescription);
}

void FCADModelToTechSoftConverterBase::AddSurfaceDataForMesh(const TCHAR* InFilePath, const CADLibrary::FMeshParameters& InMeshParameters, const FDatasmithTessellationOptions& InTessellationOptions, FDatasmithMeshElementPayload& OutMeshPayload) const
{
	ParametricSurfaceUtils::AddSurfaceData(InFilePath, ImportParameters, InMeshParameters, InTessellationOptions, OutMeshPayload);
}

