// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechSoftParametricSurface.h"

#include "CADInterfacesModule.h"
#include "CADOptions.h"
#include "DatasmithAdditionalData.h"
#include "DatasmithPayload.h"
#include "TechSoftInterface.h"
#include "TechSoftUtils.h"

#include "Engine/StaticMesh.h"
#include "HAL/PlatformFileManager.h"
#include "IDatasmithSceneElements.h"
#include "CADMeshDescriptionHelper.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "StaticMeshAttributes.h"

bool UTechSoftParametricSurfaceData::Tessellate(UStaticMesh& StaticMesh, const FDatasmithRetessellationOptions& RetessellateOptions)
{
	using namespace CADLibrary;

	bool bSuccessfulTessellation = false;

#if WITH_EDITOR
	// make a temporary file as TechSoft can only deal with files.
	FString CachePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectIntermediateDir(), TEXT("Retessellate")));
	IFileManager::Get().MakeDirectory(*CachePath, true);

	int32 Hash = GetTypeHash(StaticMesh.GetPathName());
	FString ResourceFile = FPaths::ConvertRelativePathToFull(FPaths::Combine(CachePath, FString::Printf(TEXT("0x%08x.prc"), Hash)));

	FFileHelper::SaveArrayToFile(RawData, *ResourceFile);

	// Previous MeshDescription is get to be able to create a new one with the same order of PolygonGroup (the matching of color and partition is currently based on their order)
	if (FMeshDescription* DestinationMeshDescription = StaticMesh.GetMeshDescription(0))
	{
		CADLibrary::FImportParameters ImportParameters((FDatasmithUtils::EModelCoordSystem)SceneParameters.ModelCoordSys);
		ImportParameters.SetTesselationParameters(RetessellateOptions.ChordTolerance, RetessellateOptions.MaxEdgeLength, RetessellateOptions.NormalTolerance, (CADLibrary::EStitchingTechnique)RetessellateOptions.StitchingTechnique);

		CADLibrary::FMeshParameters Parameters{ MeshParameters.bNeedSwapOrientation, MeshParameters.bIsSymmetric, (FVector3f)MeshParameters.SymmetricNormal, (FVector3f)MeshParameters.SymmetricOrigin };
		FMeshConversionContext Context(ImportParameters, Parameters, RetessellateOptions.GetGeometricTolerance(true));

		CADLibrary::FTechSoftInterface& TechSoftInterface = CADLibrary::FTechSoftInterface::Get();
		bSuccessfulTessellation = TechSoftInterface.InitializeKernel(*FPaths::EnginePluginsDir());
		if (bSuccessfulTessellation)
		{
			CADLibrary::FBodyMesh BodyMesh;
			bSuccessfulTessellation = CADLibrary::TechSoftUtils::GetBodyFromPcrFile(ResourceFile, ImportParameters, BodyMesh);

			if (bSuccessfulTessellation)
			{
				if (RetessellateOptions.RetessellationRule == EDatasmithCADRetessellationRule::SkipDeletedSurfaces)
				{
					CADLibrary::GetExistingPatches(*DestinationMeshDescription, Context.PatchesToMesh);
				}

				FMeshDescription MeshDescription;
				FStaticMeshAttributes MeshDescriptionAttributes(MeshDescription);
				MeshDescriptionAttributes.Register();
				bSuccessfulTessellation = CADLibrary::ConvertBodyMeshToMeshDescription(Context, BodyMesh, MeshDescription);

				if (bSuccessfulTessellation)
				{
	
					TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();
					FMeshSectionInfoMap& SectionInfoMap = StaticMesh.GetSectionInfoMap();

					for (FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
					{
						FMeshSectionInfo Section = SectionInfoMap.Get(0, PolygonGroupID.GetValue());
						int32 MaterialIndex = StaticMesh.GetMaterialIndex(MaterialSlotNames[PolygonGroupID]);
						if (MaterialIndex < 0)
						{
							MaterialIndex = 0;
						}
						Section.MaterialIndex = MaterialIndex;
						SectionInfoMap.Set(0, PolygonGroupID.GetValue(), Section);
					}

					*DestinationMeshDescription = MoveTemp(MeshDescription);
				}
			}
		}
	}

	// Remove temporary file
	FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*ResourceFile);

	if (!bSuccessfulTessellation)
	{
		ensureMsgf(false, TEXT("Error during mesh conversion"));
		return false;
	}
#endif
	return bSuccessfulTessellation;
}


