// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithParametricSurfaceData.h"
#include "DatasmithAdditionalData.h"
#include "DatasmithCustomAction.h"
#include "DatasmithImportOptions.h"
#include "DatasmithUtils.h"

#include "TechSoftParametricSurface.generated.h"

UCLASS(meta = (DisplayName = "TechSoft Parametric Surface Data"))
class PARAMETRICSURFACE_API UTechSoftParametricSurfaceData : public UDatasmithParametricSurfaceData
{
	GENERATED_BODY()

public:
	virtual bool Tessellate(UStaticMesh& StaticMesh, const FDatasmithRetessellationOptions& RetessellateOptions) override;
};
