// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CADOptions.h"

class IDatasmithMeshElement;
struct FDatasmithMeshElementPayload;
struct FDatasmithTessellationOptions;

namespace CADLibrary
{
class FImportParameters;
struct FMeshParameters;

/**
 * Interface to convert CAD Model (like Alias, Rhino, ...) into the internal CAD Modeler (CADKernel, ...)
 * The derived class must also implement IAddXXXBrepModel according to the CAD model (IAddAliasBrepModel, IAddRhinoBrepModel, ...)
 * The derived class is instantiated for an import session. 
 * It processes a subset of the model corresponding to a static mesh actor.
 * The main step of the process is:
 *    - InitializeProcess
 *    - Fill the modeler database with the subset of the model: 
 *        - this step is done with AddBRep function of the format interface (IAddAliasBrepModel, IAddRhinoBrepModel)
 *        - many bodies can be added to the model to generate only one static mesh actor
 *    - RepairTopology if needed
 *    - SaveBrep(const FString& FilePath) for retessellate purpose
 *    - Tessellate
 *
 * In a second step, the saved file is associated to its UStaticMesh with AddSurfaceDataForMesh function
 */
struct FCADModelGeometry
{
	int32 Type = -1;
};

class ICADModelConverter
{
public:
	virtual ~ICADModelConverter() = default;

	virtual void InitializeProcess() = 0;

	virtual bool RepairTopology() = 0;
	virtual bool SaveModel(const TCHAR* OutputPath, TSharedPtr<IDatasmithMeshElement> MeshElement) = 0;
	virtual bool Tessellate(const CADLibrary::FMeshParameters& InMeshParameters, FMeshDescription& OutMeshDescription) = 0;

	/**
	 * Set Import parameters,
	 * Tack care to set scale factor before because import parameters will be scale according to scale factor
	 * @param ChordTolerance : SAG
	 * @param MaxEdgeLength : max length of element's edge
	 * @param NormalTolerance : Angle between two adjacent triangles
	 * @param StitchingTechnique : CAD topology correction technique
	 * @param bScaleUVMap : Scale the UV map to a world unit.
	 */
	virtual void SetImportParameters(double ChordTolerance, double MaxEdgeLength, double NormalTolerance, CADLibrary::EStitchingTechnique StitchingTechnique) = 0;

	virtual bool AddGeometry(const FCADModelGeometry& Geometry) = 0;
	virtual bool IsSessionValid() = 0;

	virtual void AddSurfaceDataForMesh(const TCHAR* InFilePath, const FMeshParameters& InMeshParameters, const FDatasmithTessellationOptions& InTessellationOptions, FDatasmithMeshElementPayload& OutMeshPayload) const = 0;

};

}

