// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CADModelConverter.h"
#include "CADOptions.h"
#include "IDatasmithSceneElements.h"

#include "TechSoftInterface.h"

struct FDatasmithMeshElementPayload;
struct FDatasmithTessellationOptions;

typedef void A3DAsmPartDefinition;
typedef void A3DAsmProductOccurrence;
typedef void A3DEntity;
typedef void A3DRiRepresentationItem;
typedef void A3DTopoBrepData;


class PARAMETRICSURFACE_API FCADModelToTechSoftConverterBase : public CADLibrary::ICADModelConverter
{
public:

	FCADModelToTechSoftConverterBase(CADLibrary::FImportParameters InImportParameters)
		: ImportParameters(InImportParameters)
	{
	}

	virtual void InitializeProcess() override;

	virtual bool RepairTopology() override;

	virtual bool SaveModel(const TCHAR* InFolderPath, TSharedPtr<IDatasmithMeshElement> MeshElement) override;

	virtual bool Tessellate(const CADLibrary::FMeshParameters& InMeshParameters, FMeshDescription& OutMeshDescription) override;

	virtual void SetImportParameters(double ChordTolerance, double MaxEdgeLength, double NormalTolerance, CADLibrary::EStitchingTechnique StitchingTechnique) override
	{
		ImportParameters.SetTesselationParameters(ChordTolerance, MaxEdgeLength, NormalTolerance, StitchingTechnique);
	}

	virtual bool IsSessionValid() override
	{
		return true;
	}

	virtual bool AddGeometry(const CADLibrary::FCADModelGeometry& Geometry) override { return false; }

	virtual void AddSurfaceDataForMesh(const TCHAR* InFilePath, const CADLibrary::FMeshParameters& InMeshParameters, const FDatasmithTessellationOptions& InTessellationOptions, FDatasmithMeshElementPayload& OutMeshPayload) const override;

protected:
	CADLibrary::FImportParameters ImportParameters;
	TArray<A3DRiRepresentationItem*> RiRepresentationItems;
	CADLibrary::FUniqueTechSoftModelFile ModelFile;
};

