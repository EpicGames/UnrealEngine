// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CADOptions.h"

#include "DatasmithAdditionalData.h"
#include "DatasmithCustomAction.h"
#include "DatasmithImportOptions.h"
#include "DatasmithUtils.h"

#include "DatasmithParametricSurfaceData.generated.h"

namespace CADLibrary
{
	class FImportParameters;
	struct FMeshParameters;
}

USTRUCT(BlueprintType)
struct PARAMETRICSURFACE_API FParametricSceneParameters
{
	GENERATED_BODY()

	// value from FDatasmithUtils::EModelCoordSystem
	UPROPERTY()
	uint8 ModelCoordSys = (uint8)FDatasmithUtils::EModelCoordSystem::ZUp_LeftHanded;

	UPROPERTY()
	float MetricUnit = 0.01f;

	UPROPERTY()
	float ScaleFactor = 1.0f;
};

USTRUCT()
struct PARAMETRICSURFACE_API FParametricMeshParameters
{
	GENERATED_BODY()

	UPROPERTY()
	bool bNeedSwapOrientation = false;

	UPROPERTY()
	bool bIsSymmetric = false;

	UPROPERTY()
	FVector SymmetricOrigin = FVector::ZeroVector;

	UPROPERTY()
	FVector SymmetricNormal = FVector::ZeroVector;

	operator CADLibrary::FMeshParameters()
	{
		CADLibrary::FMeshParameters Parameters;

		Parameters.bNeedSwapOrientation = bNeedSwapOrientation;
		Parameters.bIsSymmetric = bIsSymmetric;
		Parameters.SymmetricNormal = (FVector3f) SymmetricNormal;
		Parameters.SymmetricOrigin = (FVector3f) SymmetricOrigin;

		return Parameters;
	}
};

UCLASS(meta = (DisplayName = "Datasmith Parametric Surface Data"))
class PARAMETRICSURFACE_API UDatasmithParametricSurfaceData : public UDatasmithAdditionalData
{
	GENERATED_BODY()

public:

	virtual bool IsValid()
	{
		return RawData.Num() > 0;
	}

	virtual bool SetFile(const TCHAR* FilePath);

	virtual void SetImportParameters(const CADLibrary::FImportParameters& InSceneParameters);

	virtual void SetMeshParameters(const CADLibrary::FMeshParameters& InMeshParameters);

	virtual const FDatasmithTessellationOptions& GetLastTessellationOptions() const
	{
		return LastTessellationOptions;
	}

	virtual void SetLastTessellationOptions(const FDatasmithTessellationOptions& InTessellationOptions)
	{
		LastTessellationOptions = InTessellationOptions;
	}

	virtual bool Tessellate(UStaticMesh& StaticMesh, const FDatasmithRetessellationOptions& RetessellateOptions)
	{
		return false;
	}

protected:
	virtual void Serialize(FArchive& Ar) override;

protected:
	UPROPERTY()
	FParametricSceneParameters SceneParameters;

	UPROPERTY()
	FParametricMeshParameters MeshParameters;

	UPROPERTY(EditAnywhere, Category = NURBS)
	FDatasmithTessellationOptions LastTessellationOptions;

	UPROPERTY()
	TArray<uint8> RawData_DEPRECATED;

	// Too costly to serialize as a UPROPERTY, will use custom serialization.
	TArray<uint8> RawData;
};

