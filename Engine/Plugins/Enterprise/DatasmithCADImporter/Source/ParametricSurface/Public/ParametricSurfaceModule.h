// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#define PARAMETRICSURFACE_MODULE_NAME TEXT("ParametricSurface")

/**
 * This module exposes additional features for assets containing Parametric data.
 */
class FParametricSurfaceModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;

	static FParametricSurfaceModule& Get();
	static bool IsAvailable();

	static PARAMETRICSURFACE_API class UDatasmithParametricSurfaceData* CreateParametricSurface();
};
