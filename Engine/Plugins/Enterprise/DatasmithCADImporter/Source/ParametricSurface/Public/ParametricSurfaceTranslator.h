// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithTranslator.h"
#include "UObject/ObjectMacros.h"

class UDatasmithCommonTessellationOptions;

namespace CADLibrary
{
	class FImportParameters;
	struct FMeshParameters;
}

class PARAMETRICSURFACE_API FParametricSurfaceTranslator : public IDatasmithTranslator
{
public:

	FParametricSurfaceTranslator();

	// Begin IDatasmithTranslator overrides
	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	virtual void SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	// End IDatasmithTranslator overrides

protected:
	const FDatasmithTessellationOptions& GetCommonTessellationOptions()
	{
		return CommonTessellationOptions;
	}

	/** 
	 * Call when the UDatasmithCommonTessellationOptions object is created. This is the unique opportunity for
	 * child class to overwrite some values
	 * @param TessellationOptions Reference on member of UDatasmithCommonTessellationOptions
	 */
	virtual void InitCommonTessellationOptions(FDatasmithTessellationOptions& TessellationOptions) {}

protected:
	FDatasmithTessellationOptions CommonTessellationOptions;
};

namespace ParametricSurfaceUtils
{
	PARAMETRICSURFACE_API bool AddSurfaceData(const TCHAR* MeshFilePath, const CADLibrary::FImportParameters& InSceneParameters, const CADLibrary::FMeshParameters& InMeshParameters, const FDatasmithTessellationOptions& InCommonTessellationOptions, FDatasmithMeshElementPayload& OutMeshPayload);
}