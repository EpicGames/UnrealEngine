// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ParametricSurfaceExtension : ModuleRules
	{
		public ParametricSurfaceExtension(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"CADLibrary",
					"DatasmithContent",
					"Engine",
					"MeshDescription",
					"ParametricSurface",
					"PhysicsCore",
					"StaticMeshDescription",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"ContentBrowser",
						"DataprepCore",
						"DatasmithImporter",
						"EditorFramework",
						"Slate",
						"SlateCore",
						"StaticMeshEditor",
						"ToolMenus",
						"UnrealEd",
					}
				);
			}
		}
	}
}
