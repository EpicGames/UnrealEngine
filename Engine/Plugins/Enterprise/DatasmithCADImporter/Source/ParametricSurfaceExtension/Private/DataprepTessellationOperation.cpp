// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepTessellationOperation.h"

#include "Components/StaticMeshComponent.h"
#include "ParametricSurfaceBlueprintLibrary.h"
#include "DatasmithAdditionalData.h"
#include "DatasmithAssetImportData.h"
#include "DatasmithUtils.h"
#include "CADMeshDescriptionHelper.h"

#include "IDataprepProgressReporter.h"

#include "ActorEditorUtils.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshAttributes.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "Logging/LogMacros.h"
#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "DataprepTessellationOperation"

DEFINE_LOG_CATEGORY_STATIC(LogCADLibrary, Log, All)

void UDataprepTessellationOperation::PostLoad()
{
	if( HasAnyFlags(RF_WasLoaded) && TessellationSettings_DEPRECATED.ChordTolerance != -MAX_FLT)
	{
		ChordTolerance = TessellationSettings_DEPRECATED.ChordTolerance;
		MaxEdgeLength = TessellationSettings_DEPRECATED.MaxEdgeLength;
		NormalTolerance = TessellationSettings_DEPRECATED.NormalTolerance;
		// Mark TessellationSettings_DEPRECATED as non usable
		TessellationSettings_DEPRECATED.ChordTolerance = -MAX_FLT;
	}

	Super::PostLoad();
}

void UDataprepTessellationOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
	// Collect start time to log amount of time spent to import incoming file
	uint64 StartTime = FPlatformTime::Cycles64();

	TSet<UStaticMesh*> SelectedMeshes;

	for (UObject* Object : InContext.Objects)
	{
		if ( UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object) )
		{
			SelectedMeshes.Add( StaticMesh );
		}
		else if (AActor* Actor = Cast<AActor>(Object) )
		{
			TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents( Actor );
			for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
			{
				if((StaticMesh = StaticMeshComponent->GetStaticMesh()) != nullptr)
				{
					SelectedMeshes.Add( StaticMesh );
				}
			}
		}
	}

	if(	!IsCancelled() && SelectedMeshes.Num() > 0)
	{
		FDatasmithRetessellationOptions TessellationSettings( ChordTolerance, MaxEdgeLength, NormalTolerance);

		TSharedPtr<FDataprepWorkReporter> Task = CreateTask( LOCTEXT( "LogCADLibrary_Tessellating", "Tessellating meshes ..." ), (float)SelectedMeshes.Num() );

		TArray<UObject*> ModifiedStaticMeshes;
		ModifiedStaticMeshes.Reserve( SelectedMeshes.Num() );

		for( UStaticMesh* StaticMesh : SelectedMeshes )
		{
			if( IsCancelled() )
			{
				break;
			}

			Task->ReportNextStep( FText::Format( LOCTEXT( "LogCADLibrary_Tessellating_One_Mesh", "Tessellating {0} ..."), FText::FromString( StaticMesh->GetName() ) ) );

			if( StaticMesh->IsMeshDescriptionValid( 0 ) )
			{
				FText OutReason;
				if( UParametricSurfaceBlueprintLibrary::RetessellateStaticMeshWithNotification( StaticMesh, TessellationSettings, false, OutReason ) )
				{
					ModifiedStaticMeshes.Add( StaticMesh );
					if (!OutReason.IsEmpty())
					{
						FText WarningMsg = FText::Format(LOCTEXT("DataprepTessellationOperation_TessellationCompletedWithWarning", "{0}"), OutReason);
						LogInfo(WarningMsg);
					}
				}
				else
				{
					FText WarningMsg = FText::Format( LOCTEXT( "DataprepTessellationOperation_TessellationFailed", "{0}" ), OutReason );
					LogWarning( WarningMsg );
				}
			}
			else
			{
				FText WarningMsg = FText::Format( LOCTEXT( "DataprepTessellationOperation_EmptyMesh", "No triangles in static mesh {0}" ), FText::FromString( StaticMesh->GetName() ) );
				LogWarning( WarningMsg );
			}
		}

		// Log time spent to import incoming file in minutes and seconds
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

		int ElapsedMin = int(ElapsedSeconds / 60.0);
		ElapsedSeconds -= 60.0 * (double)ElapsedMin;
		UE_LOGF( LogCADLibrary, Log, "Tessellation of %d out of %d static mesh(es) in [%d min %.3f s]", ModifiedStaticMeshes.Num(), SelectedMeshes.Num(), ElapsedMin, ElapsedSeconds );

		if(ModifiedStaticMeshes.Num() > 0)
		{
			AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
		}
	}
}

#undef LOCTEXT_NAMESPACE
