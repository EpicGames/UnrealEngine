// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepOperation.h"
#include "DatasmithImportOptions.h"
#include "DatasmithUtils.h"

#include "DataprepTessellationOperation.generated.h"

namespace CADLibrary
{
	class FImportParameters;
}
struct FMeshDescription;
class UDatasmithParametricSurfaceData;

UCLASS(Experimental, Category = MeshOperation, Meta = (DisplayName="Datasmith Tessellation", ToolTip = "For each static mesh to process, retessellate the mesh if the object contains the required data") )
class UDataprepTessellationOperation : public UDataprepOperation
{
	GENERATED_BODY()

	UDataprepTessellationOperation()
		: ChordTolerance(0.2f)
		, MaxEdgeLength(0.0f)
		, NormalTolerance(20.0f)
		// Mark TessellationSettings_DEPRECATED as non usable
		, TessellationSettings_DEPRECATED(-MAX_FLT)
	{
	}

	// UObject interface
	/** Overridden for backward compatibility with 4.23. To be removed in 4.25 */
	virtual void PostLoad() override;
	// End of UObject interface

public:
	/**
	 * Maximum distance between any point on a triangle generated by the tessellation process and the actual surface.
	 * The lower the value the more triangles.
	 * Default value is 0.2.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tessellation Options", meta = (Units = cm, ToolTip = "Maximum distance between any generated triangle and the original surface. Smaller values make more triangles.", ClampMin = "0.0"))
	float ChordTolerance;

	/**
	 * Maximum length of edges of triangles generated by the tessellation process.
	 * The length is in scene/model unit. The smaller the more triangles are generated.
	 * Value of 0 means no constraint on length of edges
	 * Default value is 0.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Tessellation Options", meta = (Units = cm, DisplayName = "Max Edge Length", ToolTip = "Maximum length of any edge in the generated triangles. Smaller values make more triangles.", ClampMin = "0.0"))
	float MaxEdgeLength;

	/**
	 * Maximum angle between the normal of two triangles generated by the tessellation process.
	 * The angle is expressed in degree. The smaller the more triangles are generated.
	 * Default value is 20 degrees.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Tessellation Options", meta = (Units = deg, ToolTip = "Maximum angle between adjacent triangles. Smaller values make more triangles.", ClampMin = "0.0", ClampMax = "90.0"))
	float NormalTolerance;

	/** Version 4.24 : Deprecated - not serialized anymore */
	UPROPERTY()
	FDatasmithTessellationOptions TessellationSettings_DEPRECATED;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};
