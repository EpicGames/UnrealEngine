// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParametricRetessellateAction.h"

#include "ParametricRetessellateAction_Impl.h"

#include "DatasmithAdditionalData.h"
#include "DatasmithParametricSurfaceData.h"
#include "DatasmithStaticMeshImporter.h" // Call to BuildStaticMesh
#include "DatasmithTranslator.h"
#include "UI/DatasmithDisplayHelper.h"

#include "Algo/AnyOf.h"
#include "Async/ParallelFor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "IStaticMeshEditor.h"
#include "Misc/ScopedSlowTask.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/ToolkitManager.h"


#define LOCTEXT_NAMESPACE "ParametricRetessellateAction"

const FText FParametricRetessellateAction_Impl::Label = LOCTEXT("RetessellateActionLabel", "Retessellate");
const FText FParametricRetessellateAction_Impl::Tooltip = LOCTEXT("RetessellateActionTooltip", "Tessellate the original NURBS surfaces to re-generate the mesh geometry");

const FText& UParametricRetessellateAction::GetLabel()
{
	return FParametricRetessellateAction_Impl::Label;
}

const FText& UParametricRetessellateAction::GetTooltip()
{
	return FParametricRetessellateAction_Impl::Tooltip;
}

bool UParametricRetessellateAction::CanApplyOnAssets(const TArray<FAssetData>& SelectedAssets)
{
	return FParametricRetessellateAction_Impl::CanApplyOnAssets(SelectedAssets);
}

void UParametricRetessellateAction::ApplyOnAssets(const TArray<FAssetData>& SelectedAssets)
{
	return FParametricRetessellateAction_Impl::ApplyOnAssets(SelectedAssets);
}

bool FParametricRetessellateAction_Impl::CanApplyOnAssets(const TArray<FAssetData>& SelectedAssets)
{
	return Algo::AnyOf(SelectedAssets, [](const FAssetData& Asset) { return Datasmith::GetAdditionalData<UDatasmithParametricSurfaceData>(Asset) != nullptr; });
}

void FParametricRetessellateAction_Impl::ApplyOnAssets(const TArray<FAssetData>& SelectedAssets)
{
	TFunction<void(UStaticMesh*)> FinalizeChanges = [](UStaticMesh* StaticMesh) -> void
	{
		StaticMesh->PostEditChange();
		StaticMesh->MarkPackageDirty();

		// Refresh associated editor 
		TSharedPtr<IToolkit> EditingToolkit = FToolkitManager::Get().FindEditorForAsset(StaticMesh);
		if (IStaticMeshEditor* StaticMeshEditorInUse = StaticCastSharedPtr<IStaticMeshEditor>(EditingToolkit).Get())
		{
			StaticMeshEditorInUse->RefreshTool();
		}
	};

	TStrongObjectPtr<UParametricRetessellateActionOptions> RetessellateOptions(Datasmith::MakeOptions<UParametricRetessellateActionOptions>());

	bool bSameOptionsForAll = false;
	int32 NumAssetsToProcess = SelectedAssets.Num();
	bool bAskForSameOption = NumAssetsToProcess > 1;

	TArray<UStaticMesh*> TessellatedMeshes;
	TessellatedMeshes.Reserve(SelectedAssets.Num());

	TUniquePtr<FScopedSlowTask> Progress;
	int32 AssetIndex = -1;
	for (const FAssetData& Asset : SelectedAssets)
	{
		AssetIndex++;
		if (UDatasmithParametricSurfaceData* ParametricSurfaceData = Datasmith::GetAdditionalData<UDatasmithParametricSurfaceData>(Asset))
		{
			if (ParametricSurfaceData->IsValid())
			{
				if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset.GetAsset()))
				{
					if (!bSameOptionsForAll)
					{
						Datasmith::FDisplayParameters Parameters;
						Parameters.bAskForSameOption = bAskForSameOption;
						Parameters.WindowTitle = LOCTEXT("OptionWindow_WindowTitle", "Datasmith Retessellation Options");
						Parameters.FileLabel = FText::Format(LOCTEXT("OptionWindow_AssetLabel", "Tessellate StaticMesh: {0}"), FText::FromString(StaticMesh->GetName()));
						Parameters.FileTooltip = FText::FromString(StaticMesh->GetPathName());
						Parameters.ProceedButtonLabel = LOCTEXT("OptionWindow_ProceedButtonLabel", "Tessellate");
						Parameters.ProceedButtonTooltip = LOCTEXT("OptionWindow_ProceedButtonTooltip", "Retessellate this mesh based on included nurbs data");
						Parameters.CancelButtonLabel = LOCTEXT("OptionWindow_CancelButtonLabel", "Cancel");
						Parameters.CancelButtonTooltip = LOCTEXT("OptionWindow_CancelButtonTooltip", "Cancel the retessellation operation");

						bAskForSameOption = false; // ask only the fist time
						RetessellateOptions->Options = ParametricSurfaceData->GetLastTessellationOptions();
						Datasmith::FDisplayResult Result = Datasmith::DisplayOptions(RetessellateOptions, Parameters);
						if (!Result.bValidated)
						{
							return;
						}
						bSameOptionsForAll |= Result.bUseSameOption;
					}
					ParametricSurfaceData->SetLastTessellationOptions(RetessellateOptions->Options);

					int32 RemainingAssetsToProcess = NumAssetsToProcess - AssetIndex;
					if (bSameOptionsForAll && !Progress.IsValid() && RemainingAssetsToProcess > 1)
					{
						Progress = MakeUnique<FScopedSlowTask>(RemainingAssetsToProcess);
						Progress->MakeDialog(true);
					}
					if (Progress)
					{
						if (Progress->ShouldCancel())
						{
							return;
						}

						FText Text = FText::Format(LOCTEXT("RetessellateAssetMessage", "Tessellate StaticMesh ({0}/{1}): {2}"),
							AssetIndex,
							NumAssetsToProcess,
							FText::FromString(StaticMesh->GetName())
						);
						Progress->EnterProgressFrame(1, Text);
					}

					if (StaticMesh->GetMeshDescription(0) == nullptr)
					{
						StaticMesh->CreateMeshDescription(0);
					}

					if (StaticMesh->GetMeshDescription(0) != nullptr)
					{
						StaticMesh->Modify();
						StaticMesh->PreEditChange(nullptr);

						if (ParametricSurfaceData != nullptr && ParametricSurfaceData->Tessellate(*StaticMesh, RetessellateOptions->Options))
						{
							TessellatedMeshes.Add(StaticMesh);
						}
						else
						{
							FinalizeChanges(StaticMesh);
						}
					}
				}
			}
		}
	}

	// Make sure lightmap settings are valid
	if (TessellatedMeshes.Num() > 1)
	{
		ParallelFor(TessellatedMeshes.Num(), [&](int32 Index)
			{
				FDatasmithStaticMeshImporter::PreBuildStaticMesh(TessellatedMeshes[Index]);
			});
	}
	else if (TessellatedMeshes.Num() > 0)
	{
		FDatasmithStaticMeshImporter::PreBuildStaticMesh(TessellatedMeshes[0]);
	}

	FDatasmithStaticMeshImporter::BuildStaticMeshes(TessellatedMeshes);

	for (UStaticMesh* StaticMesh : TessellatedMeshes)
	{
		FinalizeChanges(StaticMesh);
	}

	// Workaround to force to clean all references to the physics data and so to release old physics data.
	// https://jira.it.epicgames.com/browse/UE-166555
	GWorld->GetPhysicsScene()->Flush();
}

TSet<UStaticMesh*> GetReferencedStaticMeshes(const TArray<AActor*>& SelectedActors)
{
	TSet<UStaticMesh*> ReferencedStaticMeshes;

	for (const AActor* Actor : SelectedActors)
	{
		if (Actor)
		{
			for (const auto& Component : Actor->GetComponents())
			{
				if (UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>(Component))
				{
					ReferencedStaticMeshes.Add(SMC->GetStaticMesh());
				}
			}
		}
	}

	return ReferencedStaticMeshes;
}

bool UParametricRetessellateAction::CanApplyOnActors(const TArray<AActor*>& SelectedActors)
{
	const TSet<UStaticMesh*> ReferencedStaticMeshes = GetReferencedStaticMeshes(SelectedActors);
	return Algo::AnyOf(ReferencedStaticMeshes, [](const UStaticMesh* Mesh) { return Datasmith::GetAdditionalData<UDatasmithParametricSurfaceData>(FAssetData(Mesh)); });
}

void UParametricRetessellateAction::ApplyOnActors(const TArray<AActor*>& SelectedActors)
{
	TArray<FAssetData> AssetData;
	Algo::Transform(GetReferencedStaticMeshes(SelectedActors), AssetData, [](UStaticMesh* Mesh) { return FAssetData(Mesh); });
	return ApplyOnAssets(AssetData);
}

#undef LOCTEXT_NAMESPACE
