// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/Object.h"

class FParametricRetessellateAction_Impl
{
public:
	static const FText Label;
	static const FText Tooltip;

	static bool CanApplyOnAssets(const TArray<FAssetData>& SelectedAssets);
	static void ApplyOnAssets(const TArray<FAssetData>& SelectedAssets);
};