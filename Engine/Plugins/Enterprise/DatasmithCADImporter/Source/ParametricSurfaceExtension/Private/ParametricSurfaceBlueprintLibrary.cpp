// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParametricSurfaceBlueprintLibrary.h"

#include "ParametricRetessellateAction.h"
#include "DatasmithParametricSurfaceData.h"
#include "DatasmithStaticMeshImporter.h" // Call to BuildStaticMesh
#include "DatasmithUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Chaos/ChaosScene.h"
#include "Engine/StaticMesh.h"
#include "IStaticMeshEditor.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/PhysScene.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshAttributes.h"
#include "Toolkits/ToolkitManager.h"


#define LOCTEXT_NAMESPACE "ParametricRetessellateAction"


bool UParametricSurfaceBlueprintLibrary::RetessellateStaticMesh(UStaticMesh* StaticMesh, const FDatasmithRetessellationOptions& TessellationSettings, FText& FailureReason)
{
	return RetessellateStaticMeshWithNotification(StaticMesh, TessellationSettings, true, FailureReason);
}

bool UParametricSurfaceBlueprintLibrary::RetessellateStaticMeshWithNotification(UStaticMesh* StaticMesh, const FDatasmithRetessellationOptions& TessellationSettings, bool bApplyChanges, FText& FailureReason)
{
	bool bTessellationOutcome = false;

	int32 LODIndex = 0;

	FAssetData AssetData( StaticMesh );
	if (UDatasmithParametricSurfaceData* DatasmithParametricSurfaceData = Datasmith::GetAdditionalData<UDatasmithParametricSurfaceData>(AssetData))
	{
		// Make sure MeshDescription exists
		FMeshDescription* DestinationMeshDescription = StaticMesh->GetMeshDescription( LODIndex );
		if (DestinationMeshDescription == nullptr)
		{
			DestinationMeshDescription = StaticMesh->CreateMeshDescription(0);
		}

		if (DestinationMeshDescription)
		{
			if (bApplyChanges)
			{
				StaticMesh->Modify();
				StaticMesh->PreEditChange( nullptr );
			}

			const int32 OldNumberOfUVChannels = FStaticMeshAttributes(*DestinationMeshDescription).GetVertexInstanceUVs().GetNumChannels();
			if (DatasmithParametricSurfaceData->Tessellate(*StaticMesh, TessellationSettings))
			{
				const int32 NumberOfUVChannels = FStaticMeshAttributes(*DestinationMeshDescription).GetVertexInstanceUVs().GetNumChannels();
				if (NumberOfUVChannels < OldNumberOfUVChannels)
				{
					FailureReason = FText::Format(NSLOCTEXT("BlueprintRetessellation", "UVChannelsDestroyed", "Tessellation operation on Static Mesh {0} is destroying all UV channels above channel #{1}"), FText::FromString(StaticMesh->GetName()), NumberOfUVChannels - 1);
				}

				// Post static mesh has changed
				if (bApplyChanges)
				{
					FDatasmithStaticMeshImporter::PreBuildStaticMesh(StaticMesh); // handle uvs stuff
					FDatasmithStaticMeshImporter::BuildStaticMesh(StaticMesh);

					StaticMesh->PostEditChange();

					UStaticMesh::FCommitMeshDescriptionParams Params;
					Params.bMarkPackageDirty = true;
					Params.bUseHashAsGuid = true;
					StaticMesh->CommitMeshDescription(0, Params);
					StaticMesh->ClearMeshDescription(0);

					// Refresh associated editor
					TSharedPtr<IToolkit> EditingToolkit = FToolkitManager::Get().FindEditorForAsset(StaticMesh);
					if (IStaticMeshEditor* StaticMeshEditorInUse = StaticCastSharedPtr<IStaticMeshEditor>(EditingToolkit).Get())
					{
						StaticMeshEditorInUse->RefreshTool();
					}
				}
				// No posting required, just make sure the new tessellation is committed
				else
				{
					// Invalidate physics data as the mesh is rebuild.
					StaticMesh->GetBodySetup()->InvalidatePhysicsData();

					UStaticMesh::FCommitMeshDescriptionParams Params;
					Params.bMarkPackageDirty = false;
					Params.bUseHashAsGuid = true;
					StaticMesh->CommitMeshDescription( LODIndex, Params );
					StaticMesh->ClearMeshDescription(0); // important

					// Workaround to force to clean all references to the physics data and so to release old physics data.
					// https://jira.it.epicgames.com/browse/UE-166555
					for (int32 idx = 0; idx < GEditor->GetWorldContexts().Num(); ++idx)
					{
						const FWorldContext& Context = GEditor->GetWorldContexts()[idx];
						if (FPhysScene* PhysScene = Context.World()->GetPhysicsScene())
						{
							PhysScene->Flush();
						}
					}
				}

				// Save last tessellation settings
				DatasmithParametricSurfaceData->SetLastTessellationOptions(TessellationSettings);

				bTessellationOutcome = true;
			}
			else
			{
				FailureReason = NSLOCTEXT("BlueprintRetessellation", "LoadFailed", "Cannot generate mesh from parametric surface data");
			}
		}
		else
		{
			FailureReason = NSLOCTEXT("BlueprintRetessellation", "MeshDescriptionMissing", "Cannot create mesh description");
		}
	}
	else
	{
		FailureReason = NSLOCTEXT("BlueprintRetessellation", "MissingData", "No tessellation data attached to the static mesh");
	}

	return bTessellationOutcome;
}

#undef LOCTEXT_NAMESPACE