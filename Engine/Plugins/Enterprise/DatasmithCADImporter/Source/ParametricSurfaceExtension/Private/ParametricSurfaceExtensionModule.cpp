// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParametricSurfaceExtensionModule.h"

#include "ParametricRetessellateAction.h"
#include "ParametricRetessellateAction_Impl.h"

#include "ContentBrowserMenuContexts.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IStaticMeshEditor.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "StaticMeshEditorModule.h"
#include "Styling/AppStyle.h"
#include "ToolMenu.h"
#include "UObject/StrongObjectPtr.h"


#define LOCTEXT_NAMESPACE "ParametricSurfaceExtensionModule"


/** UI extension that displays a Retessellate action in the StaticMeshEditor */
namespace StaticMeshEditorExtenser
{
	bool CanExecute(UStaticMesh* Target)
	{
		TArray<FAssetData> AssetData;
		AssetData.Emplace(Target);

		FParametricRetessellateAction_Impl RetessellateAction;
		return RetessellateAction.CanApplyOnAssets(AssetData);
	}

	void Execute(UStaticMesh* Target)
	{
		TArray<FAssetData> AssetData;
		AssetData.Emplace(Target);

		FParametricRetessellateAction_Impl RetessellateAction;
		RetessellateAction.ApplyOnAssets(AssetData);
	}

	void ExtendAssetMenu(FMenuBuilder& MenuBuilder, UStaticMesh* Target)
	{
		MenuBuilder.AddMenuEntry(
			FParametricRetessellateAction_Impl::Label,
			FParametricRetessellateAction_Impl::Tooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&StaticMeshEditorExtenser::Execute, Target),
				FCanExecuteAction::CreateStatic(&StaticMeshEditorExtenser::CanExecute, Target)
			)
		);
	}

	TSharedRef<FExtender> CreateExtenderForObjects(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> Objects)
	{
		TSharedRef<FExtender> Extender = MakeShareable(new FExtender);

		if (UStaticMesh* Target = Objects.Num() ? Cast<UStaticMesh>(Objects[0]) : nullptr)
		{
			Extender->AddMenuExtension(
				"AssetEditorActions",
				EExtensionHook::Position::After,
				CommandList,
				FMenuExtensionDelegate::CreateStatic(&StaticMeshEditorExtenser::ExtendAssetMenu, Target)
			);
		}

		return Extender;
	}

	void ExecuteRetessellation(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = InContext.FindContext<UContentBrowserAssetContextMenuContext>())
		{
			FParametricRetessellateAction_Impl::ApplyOnAssets(CBContext->SelectedAssets);
		}
	}

	bool CanExecuteRetessellation(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = InContext.FindContext<UContentBrowserAssetContextMenuContext>())
		{
			return FParametricRetessellateAction_Impl::CanApplyOnAssets(CBContext->SelectedAssets);
		}

		return false;
	}

	void Register()
	{
		if (IsRunningCommandlet())
		{
			return;
		}

		IStaticMeshEditorModule& StaticMeshEditorModule = FModuleManager::Get().LoadModuleChecked<IStaticMeshEditorModule>("StaticMeshEditor");
		TArray<FAssetEditorExtender>& ExtenderDelegates = StaticMeshEditorModule.GetMenuExtensibilityManager()->GetExtenderDelegates();
		ExtenderDelegates.Add(FAssetEditorExtender::CreateStatic(&StaticMeshEditorExtenser::CreateExtenderForObjects));

		UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UStaticMesh::StaticClass());

		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("Retessallation_Execute", "Retessellate");
					const TAttribute<FText> ToolTip = LOCTEXT("Retessalltion_Execute_Tooltip", "Retessellate.");
					const FSlateIcon SlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Adjust");
					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteRetessellation);
					UIAction.IsActionVisibleDelegate = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteRetessellation);
					InSection.AddMenuEntry("FixTextureValidation", Label, ToolTip, SlateIcon, UIAction);
				}
			}));
	}
};

FParametricSurfaceExtensionModule& FParametricSurfaceExtensionModule::Get()
{
	return FModuleManager::LoadModuleChecked< FParametricSurfaceExtensionModule >(PARAMETRICSURFACEEXTENSION_MODULE_NAME);
}

bool FParametricSurfaceExtensionModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(PARAMETRICSURFACEEXTENSION_MODULE_NAME);
}

void FParametricSurfaceExtensionModule::StartupModule()
{
	if (!IsRunningCommandlet())
	{
		StaticMeshEditorExtenser::Register();
	}
}

IMPLEMENT_MODULE(FParametricSurfaceExtensionModule, ParametricSurfaceExtension);

#undef LOCTEXT_NAMESPACE // "ParametricSurfaceExtensionModule"

