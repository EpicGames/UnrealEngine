// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithCustomAction.h"
#include "DatasmithImportOptions.h"

#include "ParametricRetessellateAction.generated.h"

UCLASS()
class PARAMETRICSURFACEEXTENSION_API UParametricRetessellateAction : public UDatasmithCustomActionBase
{
	GENERATED_BODY()

public:
	virtual const FText& GetLabel() override;
	virtual const FText& GetTooltip() override;

	virtual bool CanApplyOnAssets(const TArray<FAssetData>& SelectedAssets) override;

	virtual void ApplyOnAssets(const TArray<FAssetData>& SelectedAssets) override;

	virtual bool CanApplyOnActors(const TArray<AActor*>& SelectedActors) override;

	virtual void ApplyOnActors(const TArray<AActor*>& SelectedActors) override;
};


UCLASS(config = Editor, Transient)
class PARAMETRICSURFACEEXTENSION_API UParametricRetessellateActionOptions : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "NotVisible", meta = (ShowOnlyInnerProperties))
	FDatasmithRetessellationOptions Options;
};
