// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithImportOptions.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/TextProperty.h"

#include "ParametricSurfaceBlueprintLibrary.generated.h"

class UStaticMesh;

UCLASS(Blueprintable, BlueprintType, meta = (DisplayName = "CAD Surface Operations Library"))
class PARAMETRICSURFACEEXTENSION_API UParametricSurfaceBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Re-tessellate LOD 0 of a static mesh if it contains parametric surface data.
	 * @param	StaticMesh				Static mesh to re-tessellate.
	 * @param	TessellationSettings	Tessellation settings to use.
	 * @param	FailureReason			Text describing the reason of failure.
	 * @return True if successful, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Datasmith | Surface Operations" )
	static bool RetessellateStaticMesh(UStaticMesh* StaticMesh, const FDatasmithRetessellationOptions& TessellationSettings, FText& FailureReason);

	/**
	 * Re-tessellate LOD 0 of a static mesh if it contains parametric surface data.
	 * This implementations allows to skip post edit operations (bApplyChanges=false),
	 * the caller is then responsible to handle those operations.
	 * @param	StaticMesh				Static mesh to re-tessellate.
	 * @param	TessellationSettings	Tessellation settings to use.
	 * @param	bApplyChanges			Indicates if change must be notified.
	 * @param	FailureReason			Text describing the reason of failure, or a warning if the operation was successful.
	 * @return True if successful, false otherwise
	 */
	UFUNCTION(Category = "Datasmith | Surface Operations" )
	static bool RetessellateStaticMeshWithNotification(UStaticMesh* StaticMesh, const FDatasmithRetessellationOptions& TessellationSettings, bool bApplyChanges, FText& FailureReason);
};
