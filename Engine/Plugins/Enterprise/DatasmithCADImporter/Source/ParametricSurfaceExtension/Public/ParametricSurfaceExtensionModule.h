// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#define PARAMETRICSURFACEEXTENSION_MODULE_NAME TEXT("ParametricSurfaceExtension")

/**
 * This module exposes additional features for assets containing CoreTech data.
 */
class FParametricSurfaceExtensionModule : public IModuleInterface
{
public:
    static FParametricSurfaceExtensionModule& Get();
    static bool IsAvailable();

private:
	virtual void StartupModule() override;
};
