// Copyright Epic Games, Inc. All Rights Reserved.

#include "AliasModelToCADKernelConverter.h"

#ifdef USE_OPENMODEL

#include "OpenModelUtils.h"

#include "CADData.h"
#include "CADKernelTools.h"
#include "HAL/ConsoleManager.h"
#include "HAL/PlatformMemory.h"
#include "CADMeshDescriptionHelper.h"

#include "Core/Session.h"
#include "Geo/Curves/NURBSCurveData.h"
#include "Geo/GeoEnum.h"
#include "Geo/Surfaces/NurbsSurfaceData.h"
#include "Geo/Surfaces/Surface.h"

#include "Math/Point.h"
#include "Mesh/Structure/ModelMesh.h"

#include "Topo/Body.h"
#include "Topo/Model.h"
#include "Topo/Shell.h"
#include "Topo/TopologicalEdge.h"
#include "Topo/TopologicalFace.h"
#include "Topo/TopologicalLoop.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

// Alias API wrappes object in AlObjects. This is an abstract base class which holds a reference to an anonymous data structure.
// The only way to compare two AlObjects is to compare their data structure. That is the reason why private fields are made public. 
#define private public
#include "AlCurve.h"
#include "AlDagNode.h"
#include "AlShell.h"
#include "AlShellNode.h"
#include "AlSurface.h"
#include "AlSurfaceNode.h"
#include "AlTrimBoundary.h"
#include "AlTrimCurve.h"
#include "AlTrimRegion.h"
#include "AlTM.h"

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define LOCTEXT_NAMESPACE "WireInterface"

namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{
using namespace UE::CADKernel;
using ECADKernelSewOption = UE::CADKernel::ESewOption;

namespace AliasToCADKernelUtils
{
	template<typename Surface_T>
	TSharedPtr<FSurface> AddNURBSSurface(double GeometricTolerance, Surface_T& AliasSurface, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix)
	{
		FNurbsSurfaceHomogeneousData NURBSData;
		NURBSData.bSwapUV= true;
		NURBSData.bIsRational = true;

		NURBSData.PoleUCount = AliasSurface.uNumberOfCVsInclMultiples();
		NURBSData.PoleVCount = AliasSurface.vNumberOfCVsInclMultiples();

		NURBSData.UDegree = AliasSurface.uDegree();  // U order of the surface
		NURBSData.VDegree = AliasSurface.vDegree();  // V order of the surface

		int32 KnotSizeU = AliasSurface.realuNumberOfKnots() + 2;
		int32 KnotSizeV = AliasSurface.realvNumberOfKnots() + 2;

		NURBSData.UNodalVector.SetNumUninitialized(KnotSizeU);
		NURBSData.VNodalVector.SetNumUninitialized(KnotSizeV);

		AliasSurface.realuKnotVector(NURBSData.UNodalVector.GetData() + 1);
		AliasSurface.realvKnotVector(NURBSData.VNodalVector.GetData() + 1);

		NURBSData.UNodalVector[0] = NURBSData.UNodalVector[1];
		NURBSData.UNodalVector[KnotSizeU - 1] = NURBSData.UNodalVector[KnotSizeU - 2];
		NURBSData.VNodalVector[0] = NURBSData.VNodalVector[1];
		NURBSData.VNodalVector[KnotSizeV - 1] = NURBSData.VNodalVector[KnotSizeV - 2];

		const int32 CoordinateCount = NURBSData.PoleUCount * NURBSData.PoleVCount * 4;
		NURBSData.HomogeneousPoles.SetNumUninitialized(CoordinateCount);

		if (InObjectReference == EAliasObjectReference::WorldReference)
		{
			AliasSurface.CVsWorldPositionInclMultiples(NURBSData.HomogeneousPoles.GetData());
		}
		else if (InObjectReference == EAliasObjectReference::ParentReference)
		{
			AlTM TranformMatrix(InAlMatrix);
			AliasSurface.CVsAffectedPositionInclMultiples(TranformMatrix, NURBSData.HomogeneousPoles.GetData());
		}
		else  // EAliasObjectReference::LocalReference
		{
			AliasSurface.CVsUnaffectedPositionInclMultiples(NURBSData.HomogeneousPoles.GetData());
		}

		// convert cm to mm
		double* Poles = NURBSData.HomogeneousPoles.GetData();
		for (int32 Index = 0; Index < NURBSData.HomogeneousPoles.Num(); Index +=4)
		{
			Poles[Index + 0] *= UNIT_CONVERSION_CM_TO_MM;
			Poles[Index + 1] *= UNIT_CONVERSION_CM_TO_MM;
			Poles[Index + 2] *= UNIT_CONVERSION_CM_TO_MM;
		}

		return FSurface::MakeNurbsSurface(GeometricTolerance, NURBSData);
	}
}

FAliasModelToCADKernelConverter::FAliasModelToCADKernelConverter(const FDatasmithTessellationOptions& Options, CADLibrary::FImportParameters InImportParameters)
	: FCADModelToCADKernelConverterBase(InImportParameters)
{
	SetTolerances(Options.GetGeometricTolerance(true), Options.GetStitchingTolerance(true));
}

TSharedPtr<FTopologicalEdge> FAliasModelToCADKernelConverter::AddEdge(const AlTrimCurve& AliasTrimCurve, TSharedPtr<FSurface>& CarrierSurface)
{
	FNurbsCurveData NurbsCurveData;

	NurbsCurveData.Degree = AliasTrimCurve.degree();
	int32 ControlPointCount = AliasTrimCurve.numberOfCVs();

	NurbsCurveData.Dimension = 2;
	NurbsCurveData.bIsRational = true;

	int32 KnotCount = AliasTrimCurve.realNumberOfKnots() + 2;

	NurbsCurveData.Weights.SetNumUninitialized(ControlPointCount);
	NurbsCurveData.Poles.SetNumUninitialized(ControlPointCount);
	NurbsCurveData.NodalVector.SetNumUninitialized(KnotCount);

	using AlPoint = double[3];
	// Notice that each CV has three coordinates - the three coordinates describe 2D parameter space, with a homogeneous coordinate.
	// Each control point is u, v and w, where u and v are parameter space and w is the homogeneous coordinate.
	AliasTrimCurve.CVsUVPosition(NurbsCurveData.NodalVector.GetData() + 1, (AlPoint*) NurbsCurveData.Poles.GetData());

	AliasTrimCurve.realKnotVector(NurbsCurveData.NodalVector.GetData() + 1);
	NurbsCurveData.NodalVector[0] = NurbsCurveData.NodalVector[1];
	NurbsCurveData.NodalVector[KnotCount - 1] = NurbsCurveData.NodalVector[KnotCount - 2];

	for (int32 Index = 0; Index < ControlPointCount; ++Index)
	{
		NurbsCurveData.Weights[Index] = NurbsCurveData.Poles[Index].Z;
		NurbsCurveData.Poles[Index].Z = 0;
	}

	TSharedPtr<FCurve> Nurbs = FCurve::MakeNurbsCurve(NurbsCurveData);

	TSharedRef<FRestrictionCurve> RestrictionCurve = FCurve::MakeShared<FRestrictionCurve>(CarrierSurface.ToSharedRef(), Nurbs.ToSharedRef());
	TSharedPtr<FTopologicalEdge> Edge = FTopologicalEdge::Make(RestrictionCurve);
	if (!Edge.IsValid())
	{
		return TSharedPtr<FTopologicalEdge>();
	}

	// Only TrimCurve with twin need to be in the map used in LinkEdgesLoop 
	TAlObjectPtr<AlTrimCurve> TwinCurve(AliasTrimCurve.getTwinCurve());
	if (TwinCurve.IsValid())
	{
		AlEdge2CADKernelEdge.Add(AliasTrimCurve.fSpline, Edge);
	}

	return Edge;
}

TSharedPtr<FTopologicalLoop> FAliasModelToCADKernelConverter::AddLoop(const AlTrimBoundary& TrimBoundary, TSharedPtr<FSurface>& CarrierSurface, const bool bIsExternal)
{
	TArray<TSharedPtr<FTopologicalEdge>> Edges;
	TArray<UE::CADKernel::EOrientation> Directions;

	TAlObjectPtr<AlTrimCurve> TrimCurve(TrimBoundary.firstCurve());
	statusCode Status = TrimCurve ? sSuccess : sObjectNotFound;
	while (Status == sSuccess)
	{
		TSharedPtr<FTopologicalEdge> Edge = AddEdge(*TrimCurve, CarrierSurface);
		if (Edge.IsValid())
		{
			Edges.Add(Edge);
			Directions.Emplace(UE::CADKernel::EOrientation::Front);
		}

		Status = TrimCurve->nextCurveD();
	}

	if (Edges.Num() == 0)
	{
		return TSharedPtr<FTopologicalLoop>();
	}

	TSharedPtr<FTopologicalLoop> Loop = FTopologicalLoop::Make(Edges, Directions, bIsExternal, GeometricTolerance);
	return Loop;
}

void FAliasModelToCADKernelConverter::LinkEdgesLoop(const AlTrimBoundary& TrimBoundary, FTopologicalLoop& Loop)
{
	TAlObjectPtr<AlTrimCurve> TrimCurve(TrimBoundary.firstCurve());
	statusCode Status = TrimCurve ? sSuccess : sObjectNotFound;
	while (Status == sSuccess)
	{
		TSharedPtr<FTopologicalEdge>* Edge = AlEdge2CADKernelEdge.Find(TrimCurve->fSpline);
		if (!Edge || !Edge->IsValid() || (*Edge)->IsDeleted() || (*Edge)->IsDegenerated())
		{
			Status = TrimCurve->nextCurveD();
			continue;
		}

		ensureWire(&Loop == (*Edge)->GetLoop());

		// Link edges
		TAlObjectPtr<AlTrimCurve> TwinCurve(TrimCurve->getTwinCurve());
		if (TwinCurve.IsValid())
		{
			if (TSharedPtr<FTopologicalEdge>* TwinEdge = AlEdge2CADKernelEdge.Find(TwinCurve->fSpline))
			{
				if (TwinEdge->IsValid() && !(*TwinEdge)->IsDeleted() && !(*TwinEdge)->IsDegenerated())
				{
					(*Edge)->LinkIfCoincident(**TwinEdge, EdgeLengthTolerance, SquareTolerance);
				}
			}
		}

		Status = TrimCurve->nextCurveD();
	}
}

TSharedPtr<FTopologicalFace> FAliasModelToCADKernelConverter::AddTrimRegion(const AlTrimRegion& TrimRegion, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix, bool bInOrientation)
{
	TSharedPtr<FSurface> Surface = AliasToCADKernelUtils::AddNURBSSurface(GeometricTolerance, TrimRegion, InObjectReference, InAlMatrix);
	if (!Surface.IsValid())
	{
		return TSharedPtr<FTopologicalFace>();
	}

	bool bIsExternal = true;
	TArray<TSharedPtr<FTopologicalLoop>> Loops;
	TAlObjectPtr<AlTrimBoundary> TrimBoundary(TrimRegion.firstBoundary());

	statusCode Status = TrimBoundary ? sSuccess : sObjectNotFound;
	while (Status == sSuccess)
	{
		TSharedPtr<FTopologicalLoop> Loop = AddLoop(*TrimBoundary, Surface, bIsExternal);
		if (Loop.IsValid())
		{
			LinkEdgesLoop(*TrimBoundary, *Loop);
			Loops.Add(Loop);
			bIsExternal = false;
		}

		Status = TrimBoundary->nextBoundaryD();
	}

	if (Loops.Num() == 0)
	{
		return TSharedPtr<FTopologicalFace>();
	}

	TSharedRef<FTopologicalFace> Face = FEntity::MakeShared<FTopologicalFace>(Surface);
	Face->SetPatchId(LastFaceId++);

	int32 DoubtfulLoopOrientationCount = 0;
	Face->AddLoops(Loops, DoubtfulLoopOrientationCount);

	if (Face->GetLoops().Num() == 0)
	{
		Face->SetAsDegenerated();
		Face->Delete();

		return TSharedPtr<FTopologicalFace>();
	}

	return Face;
}

void FAliasModelToCADKernelConverter::AddFace(const AlSurface& Surface, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix, bool bInOrientation, TSharedRef<FShell>& Shell)
{
	TAlObjectPtr<AlTrimRegion> TrimRegion(Surface.firstTrimRegion());
	statusCode Status = TrimRegion ? sSuccess : sObjectNotFound;
	if (Status == sSuccess)
	{
		while(Status == sSuccess)
		{
			TSharedPtr<FTopologicalFace> Face = AddTrimRegion(*TrimRegion, InObjectReference, InAlMatrix, bInOrientation);
			if (Face.IsValid())
			{
				Shell->Add(Face.ToSharedRef(), bInOrientation ? UE::CADKernel::EOrientation::Front : UE::CADKernel::EOrientation::Back);
			}

			Status = TrimRegion->nextRegionD();
		}
	}
	else
	{
		TSharedPtr<FSurface> CADKernelSurface = AliasToCADKernelUtils::AddNURBSSurface(GeometricTolerance, Surface, InObjectReference, InAlMatrix);
		if (CADKernelSurface.IsValid())
		{
			TSharedRef<FTopologicalFace> Face = FEntity::MakeShared<FTopologicalFace>(CADKernelSurface);
			Face->ApplyNaturalLoops();
			// Surface can be too thin based on tolerances. Skip it
			// #cadkernel_check: Warn user a surface was too thin
			if (Face->GetLoops().Num() > 0)
			{
				Shell->Add(Face, bInOrientation ? UE::CADKernel::EOrientation::Front : UE::CADKernel::EOrientation::Back);
			}
			// #wire_import: Log that this face was not added
		}
	}
}

void FAliasModelToCADKernelConverter::AddShell(const AlShell& InShell, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix, bool bInOrientation, TSharedRef<FShell>& CADKernelShell)
{
	TAlObjectPtr<AlTrimRegion> TrimRegion(InShell.firstTrimRegion());
	statusCode Status = TrimRegion ? sSuccess : sObjectNotFound;
	while (Status == sSuccess)
	{
		TSharedPtr<FTopologicalFace> Face = AddTrimRegion(*TrimRegion, InObjectReference, InAlMatrix, bInOrientation);
		if (Face.IsValid())
		{
			CADKernelShell->Add(Face.ToSharedRef(), bInOrientation ? UE::CADKernel::EOrientation::Front : UE::CADKernel::EOrientation::Back);
		}

		Status = TrimRegion->nextRegionD();
	}
}

bool FAliasModelToCADKernelConverter::AddBRep(const FAlDagNodePtr& DagNode, const FColor& Color, EAliasObjectReference InObjectReference)
{
	uint32 ColorId = (uint32)CADLibrary::BuildColorUId(Color);
	return AddBRep(DagNode, ColorId, InObjectReference);
}

bool FAliasModelToCADKernelConverter::AddBRep(const FAlDagNodePtr& DagNode, uint32 SlotID, EAliasObjectReference InObjectReference)
{
	if (!DagNode.IsValid())
	{
		return false;
	}

	AlEdge2CADKernelEdge.Empty();

	boolean bAlOrientation;
	DagNode->getSurfaceOrientation(bAlOrientation);
	bool bOrientation = !(bool)bAlOrientation;

	AlMatrix4x4 AlMatrix;
	if (InObjectReference == EAliasObjectReference::ParentReference)
	{
		DagNode->localTransformationMatrix(AlMatrix);
	}

	TSharedRef<FShell> CADKernelShell = FEntity::MakeShared<FShell>();

	TAlObjectPtr<AlShell> Shell;
	if (DagNode.GetShell(Shell))
	{
		AddShell(*Shell, InObjectReference, AlMatrix, bOrientation, CADKernelShell);
	}
	else
	{
		TAlObjectPtr<AlSurface> Surface;
		if (DagNode.GetSurface(Surface))
		{
			AddFace(*Surface, InObjectReference, AlMatrix, bOrientation, CADKernelShell);
		}
	}

	if (CADKernelShell->FaceCount() > 0)
	{
		TSharedRef<FBody> CADKernelBody = FEntity::MakeShared<FBody>();

		CADKernelBody->SetColorId(SlotID);
		CADKernelBody->AddShell(CADKernelShell);
		CADKernelBody->CompleteMetaData();

		CADKernelSession.GetModel().Add(CADKernelBody);

		return true;
	}

	// #wire_import: Log that no face was added to the model
	return false;
}

bool FAliasModelToCADKernelConverter::Tessellate(const CADLibrary::FMeshParameters& InMeshParameters, FMeshDescription& OutMeshDescription)
{
	FModel& Model = CADKernelSession.GetModel();

	CADLibrary::FMeshConversionContext Context(ImportParameters, InMeshParameters, GeometricTolerance);

	return CADLibrary::FCADKernelTools::Tessellate(Model, Context, OutMeshDescription);
}

bool FAliasModelToCADKernelConverter::RepairTopology()
{
	using namespace CADLibrary;
	// Apply stitching if applicable
	if (ImportParameters.GetStitchingTechnique() != StitchingNone)
	{
		ECADKernelSewOption SewOptionValue = (ECADKernelSewOption)CADLibrary::SewOption::GetFromImportParameters();

#if !WIRE_THINFACE_ENABLED
		SewOptionValue = ECADKernelSewOption((uint8)SewOptionValue & (uint8)~ECADKernelSewOption::RemoveThinFaces);
#endif

		FTopomakerOptions TopomakerOptions(SewOptionValue, StitchingTolerance, FImportParameters::GStitchingForceFactor);

		FTopomaker Topomaker(CADKernelSession, TopomakerOptions);
		Topomaker.Sew();
		Topomaker.SplitIntoConnectedShells();
		Topomaker.OrientShells();
	}

	return true;
}

bool FAliasModelToCADKernelConverter::AddGeometry(const CADLibrary::FCADModelGeometry& Geometry)
{
	if (Geometry.Type == (int32)ECADModelGeometryType::DagNode)
	{
		const FDagNodeGeometry& DagNodeGeometry = static_cast<const FDagNodeGeometry&>(Geometry);

		return AddBRep(DagNodeGeometry.DagNode, 0, DagNodeGeometry.Reference);
	}
	else if (Geometry.Type == (int32)ECADModelGeometryType::BodyNode)
	{
		const FBodyNodeGeometry& BodyNodeGeometry = static_cast<const FBodyNodeGeometry&>(Geometry);

		bool bBodyAdded = false;
		BodyNodeGeometry.BodyNode->IterateOnDagNodes([&](const FAlDagNodePtr& DagNode)
			{
				const bool bBRepAdded = AddBRep(DagNode, BodyNodeGeometry.BodyNode->GetSlotIndex(DagNode), BodyNodeGeometry.Reference);
				if (!bBRepAdded)
				{
					UE_LOGF(LogWireInterface, Warning, "Failed to add DagNode %ls to StaticMesh.", *DagNode.GetName());
				}
				bBodyAdded |= bBRepAdded;
			});

		ensureWire(bBodyAdded);
		return bBodyAdded;
	}

	return false;
}

}

#endif

#undef LOCTEXT_NAMESPACE // "WireInterface"
