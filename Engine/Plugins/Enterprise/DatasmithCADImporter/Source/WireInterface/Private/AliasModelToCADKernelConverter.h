// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifdef USE_OPENMODEL
#include "OpenModelUtils.h"

#include "CADModelToCADKernelConverterBase.h"

#include "Core/Session.h"
#include "Core/Types.h"

class AlDagNode;
class AlShell;
class AlSurface;
class AlTrimBoundary;
class AlTrimCurve;
class AlTrimRegion;

namespace UE::CADKernel
{
	class FShell;
	class FSurface;
	class FTopologicalEdge;
	class FTopologicalFace;
	class FTopologicalLoop;
}

namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{

class FAliasModelToCADKernelConverter : public FCADModelToCADKernelConverterBase
{

public:
	FAliasModelToCADKernelConverter(const FDatasmithTessellationOptions& Options, CADLibrary::FImportParameters InImportParameters);

	// Begin FCADModelToCADKernelConverterBase overrides
	virtual bool Tessellate(const CADLibrary::FMeshParameters& InMeshParameters, FMeshDescription& OutMeshDescription) override;
	virtual bool RepairTopology() override;

	virtual bool AddGeometry(const CADLibrary::FCADModelGeometry& Geometry) override;
	// End FCADModelToCADKernelConverterBase overrides

	bool AddBRep(const FAlDagNodePtr& DagNode, const FColor& Color, EAliasObjectReference ObjectReference);
	bool AddBRep(const FAlDagNodePtr& DagNode, uint32 SlotID, EAliasObjectReference ObjectReference);

protected:
	TSharedPtr<UE::CADKernel::FTopologicalEdge> AddEdge(const AlTrimCurve& TrimCurve, TSharedPtr<UE::CADKernel::FSurface>& CarrierSurface);

	TSharedPtr<UE::CADKernel::FTopologicalLoop> AddLoop(const AlTrimBoundary& TrimBoundary, TSharedPtr<UE::CADKernel::FSurface>& CarrierSurface, const bool bIsExternal);
	
	/**
	 * Build face's links with its neighbor have to be done after the loop is finalize.
	 * This is to avoid to link an edge with another and then to delete it...
	 */
	void LinkEdgesLoop(const AlTrimBoundary& TrimBoundary, UE::CADKernel::FTopologicalLoop& Loop);

	TSharedPtr<UE::CADKernel::FTopologicalFace> AddTrimRegion(const AlTrimRegion& InTrimRegion, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix, bool bInOrientation);
	void AddFace(const AlSurface& InSurface, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix, bool bInOrientation, TSharedRef<UE::CADKernel::FShell>& Shell);
	void AddShell(const AlShell& InShell, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix, bool bInOrientation, TSharedRef<UE::CADKernel::FShell>& Shell);

protected:
	int32 LastFaceId = 1;
	TMap<void*, TSharedPtr<UE::CADKernel::FTopologicalEdge>>  AlEdge2CADKernelEdge;
};

}
#endif