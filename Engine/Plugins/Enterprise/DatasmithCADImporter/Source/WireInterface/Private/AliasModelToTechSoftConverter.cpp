// Copyright Epic Games, Inc. All Rights Reserved.

#include "AliasModelToTechSoftConverter.h"

#ifdef USE_OPENMODEL
#include "OpenModelUtils.h"

#include "HAL/PlatformMemory.h"
#include "Math/Color.h"
#include "CADMeshDescriptionHelper.h"
#include "TechSoftInterface.h"
#include "TechSoftUtils.h"
#include "TUniqueTechSoftObj.h"



#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

// Alias API wrappes object in AlObjects. This is an abstract base class which holds a reference to an anonymous data structure.
// The only way to compare two AlObjects is to compare their data structure. That is the reason why private fields are made public. 
#define private public
#include "AlCurve.h"
#include "AlDagNode.h"
#include "AlShell.h"
#include "AlShellNode.h"
#include "AlSurface.h"
#include "AlSurfaceNode.h"
#include "AlTrimBoundary.h"
#include "AlTrimCurve.h"
#include "AlTrimRegion.h"
#include "AlTM.h"

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define LOCTEXT_NAMESPACE "WireInterface"

namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{

#ifdef USE_TECHSOFT_SDK

namespace AliasToTechSoftUtils
{

enum EAxis { U, V };

template<typename Surface_T>
A3DSurfBase* AddNURBSSurface(const Surface_T& AliasSurface, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix)
{
	CADLibrary::TUniqueTSObj<A3DSurfNurbsData> NurbsSurfaceData;

	NurbsSurfaceData->m_eKnotType = A3DEKnotType::kA3DKnotTypeUnspecified;
	NurbsSurfaceData->m_eSurfaceForm = A3DEBSplineSurfaceForm::kA3DBSplineSurfaceFormUnspecified;

	NurbsSurfaceData->m_uiUDegree = AliasSurface.uDegree();
	NurbsSurfaceData->m_uiVDegree = AliasSurface.vDegree();

	NurbsSurfaceData->m_uiUCtrlSize = AliasSurface.uNumberOfCVsInclMultiples();
	NurbsSurfaceData->m_uiVCtrlSize = AliasSurface.vNumberOfCVsInclMultiples();
	NurbsSurfaceData->m_uiUKnotSize = AliasSurface.realuNumberOfKnots() + 2;
	NurbsSurfaceData->m_uiVKnotSize = AliasSurface.realvNumberOfKnots() + 2;

	TArray<A3DDouble> UNodalVector;
	TArray<A3DDouble> VNodalVector;

	TFunction<void(EAxis, TArray<A3DDouble>&)> FillNodalVector = [&](EAxis Axis, TArray<A3DDouble>& OutNodalVector)
	{
		TArray<double> NodalVector;
		if(Axis == EAxis::U)
		{
			OutNodalVector.Reserve(NurbsSurfaceData->m_uiUKnotSize);
			NodalVector.SetNumUninitialized(NurbsSurfaceData->m_uiUKnotSize - 2);
			AliasSurface.realuKnotVector(NodalVector.GetData());
		}
		else
		{
			OutNodalVector.Reserve(NurbsSurfaceData->m_uiVKnotSize);
			NodalVector.SetNumUninitialized(NurbsSurfaceData->m_uiVKnotSize - 2);
			AliasSurface.realvKnotVector(NodalVector.GetData());
		}

		OutNodalVector.Add(NodalVector[0]);
		for(double Value : NodalVector)
		{
			OutNodalVector.Add(Value);
		}
		OutNodalVector.Add(NodalVector.Last());

		if (Axis == EAxis::U)
		{
			NurbsSurfaceData->m_pdUKnots = OutNodalVector.GetData();
		}
		else
		{
			NurbsSurfaceData->m_pdVKnots = OutNodalVector.GetData();
		}
	};

	FillNodalVector(EAxis::U, UNodalVector);
	FillNodalVector(EAxis::V, VNodalVector);

	const int32 CoordinateCount = NurbsSurfaceData->m_uiUCtrlSize * NurbsSurfaceData->m_uiVCtrlSize * 4;
	TArray<double> HomogeneousPoles;
	HomogeneousPoles.SetNumUninitialized(CoordinateCount);

	if (InObjectReference == EAliasObjectReference::WorldReference)
	{
		AliasSurface.CVsWorldPositionInclMultiples(HomogeneousPoles.GetData());
	}
	else if (InObjectReference == EAliasObjectReference::ParentReference)
	{
		AlTM TranformMatrix(InAlMatrix);
		AliasSurface.CVsAffectedPositionInclMultiples(TranformMatrix, HomogeneousPoles.GetData());
	}
	else  // EAliasObjectReference::LocalReference
	{
		AliasSurface.CVsUnaffectedPositionInclMultiples(HomogeneousPoles.GetData());
	}

	int32 PoleCount = NurbsSurfaceData->m_uiUCtrlSize * NurbsSurfaceData->m_uiVCtrlSize;
	TArray<A3DDouble> Weights;
	TArray<A3DVector3dData> ControlPoints;
	ControlPoints.SetNum(PoleCount);
	Weights.SetNum(PoleCount);
	A3DVector3dData* ControlPointPtr = ControlPoints.GetData();
	A3DDouble* WeightPtr = Weights.GetData();
	
	NurbsSurfaceData->m_pdWeights = WeightPtr;
	NurbsSurfaceData->m_pCtrlPts = ControlPointPtr;
	
	TFunction<void(const double*, A3DVector3dData&, A3DDouble&)> SetA3DPole = [](const double* HomogeneousPole, A3DVector3dData & OutTSPoint, A3DDouble & OutWeight)
	{
		OutTSPoint.m_dX = UE_TO_CADKERNEL(HomogeneousPole[0]);  // cm (Alias MetricUnit) to mm
		OutTSPoint.m_dY = UE_TO_CADKERNEL(HomogeneousPole[1]);
		OutTSPoint.m_dZ = UE_TO_CADKERNEL(HomogeneousPole[2]);
		OutWeight = HomogeneousPole[3];
	};

	double* AliasPoles = HomogeneousPoles.GetData();
	for (int32 Index = 0; Index < PoleCount; ++Index, AliasPoles += 4, ++ControlPointPtr, ++WeightPtr)
	{
		SetA3DPole(AliasPoles, *ControlPointPtr, *WeightPtr);
	}

	return CADLibrary::TechSoftInterface::CreateSurfaceNurbs(*NurbsSurfaceData);
}

} // ns AliasToTechsoftUtils

A3DCrvBase* FAliasModelToTechSoftConverter::CreateCurve(const AlTrimCurve& AliasTrimCurve)
{
	CADLibrary::TUniqueTSObj<A3DCrvNurbsData> NurbsCurveData;

	NurbsCurveData->m_bIs2D = true;
	NurbsCurveData->m_bRational = true;
	NurbsCurveData->m_uiDegree = AliasTrimCurve.degree();

	int ControlPointCount = AliasTrimCurve.numberOfCVs();
	int32 KnotCount = AliasTrimCurve.realNumberOfKnots();

	using AlPoint = TStaticArray<double, 3>;

	TArray<AlPoint> AliasPoles;
	TArray<double> AliasNodalVector;
	AliasNodalVector.SetNum(ControlPointCount);
	AliasPoles.SetNumUninitialized(ControlPointCount*4);

	// Notice that each CV has three coordinates - the three coordinates describe 2D parameter space, with a homogeneous coordinate.
	// Each control point is u, v and w, where u and v are parameter space and w is the homogeneous coordinate.
	AliasTrimCurve.CVsUVPosition(AliasNodalVector.GetData(), (double(*)[3])AliasPoles.GetData());

	AliasNodalVector.SetNum(KnotCount);
	AliasTrimCurve.realKnotVector(AliasNodalVector.GetData());

	TArray<A3DDouble> NodalVector;
	NodalVector.Reserve(KnotCount+2);
	NodalVector.Add(AliasNodalVector[0]);
	for (double Value : AliasNodalVector)
	{
		NodalVector.Add(Value);
	}
	NodalVector.Add(AliasNodalVector.Last());

	TArray<double> Weights;
	Weights.SetNumUninitialized(ControlPointCount);

	AliasNodalVector.SetNumUninitialized(KnotCount);

	TArray<A3DVector3dData> ControlPointArray;
	TArray<A3DDouble> WeightArray;

	ControlPointArray.SetNumUninitialized(ControlPointCount);
	WeightArray.SetNumUninitialized(ControlPointCount);

	A3DVector3dData* ControlPointPtr = ControlPointArray.GetData();
	A3DDouble* WeightPtr = WeightArray.GetData();
	AlPoint* AliasPolePtr = AliasPoles.GetData();

	TFunction<void(const AlPoint&, A3DVector3dData&, A3DDouble&)> SetA3DPole = [](const AlPoint& AliasPole, A3DVector3dData& OutTSPoint, A3DDouble& OutWeight)
	{
		OutTSPoint.m_dX = AliasPole[0];
		OutTSPoint.m_dY = AliasPole[1];
		OutTSPoint.m_dZ = 0;
		OutWeight = AliasPole[2];
	};

	for (int32 Index = 0; Index < ControlPointCount; ++Index, ++AliasPolePtr, ++ControlPointPtr, ++WeightPtr)
	{
		SetA3DPole(*AliasPolePtr, *ControlPointPtr, *WeightPtr);
	}

	NurbsCurveData->m_eKnotType = kA3DKnotTypeUnspecified;
	NurbsCurveData->m_eCurveForm = kA3DBSplineCurveFormUnspecified;

	NurbsCurveData->m_pCtrlPts = ControlPointArray.GetData();
	NurbsCurveData->m_uiCtrlSize = ControlPointArray.Num();

	NurbsCurveData->m_pdWeights = WeightArray.GetData();
	NurbsCurveData->m_uiWeightSize = WeightArray.Num();

	NurbsCurveData->m_pdKnots = NodalVector.GetData();
	NurbsCurveData->m_uiKnotSize = NodalVector.Num();

	return CADLibrary::TechSoftInterface::CreateCurveNurbs(*NurbsCurveData);
}

A3DTopoCoEdge* FAliasModelToTechSoftConverter::CreateEdge(const AlTrimCurve& TrimCurve)
{
	A3DCrvBase* NurbsCurvePtr = CreateCurve(TrimCurve);
	if (NurbsCurvePtr == nullptr)
	{
		return nullptr;
	}

	A3DTopoEdge* EdgePtr = CADLibrary::TechSoftUtils::CreateTopoEdge();
	if (EdgePtr == nullptr)
	{
		return nullptr;
	}

	CADLibrary::TUniqueTSObj<A3DTopoCoEdgeData> CoEdgeData;

	CoEdgeData->m_pUVCurve = NurbsCurvePtr;
	CoEdgeData->m_pEdge = EdgePtr;
	CoEdgeData->m_ucOrientationWithLoop = TrimCurve.isReversed();
	CoEdgeData->m_ucOrientationUVWithLoop = 1;

	A3DTopoCoEdge* CoEdgePtr = CADLibrary::TechSoftInterface::CreateTopoCoEdge(*CoEdgeData);

	// Only TrimCurve with twin need to be in the map used in LinkEdgesLoop 
	TAlObjectPtr<AlTrimCurve> TwinCurve(TrimCurve.getTwinCurve());
	if (TwinCurve.IsValid())
	{
		AlEdgeToTSCoEdge.Add(TrimCurve.fSpline, CoEdgePtr);
	}
	return CoEdgePtr;
}

A3DTopoLoop* FAliasModelToTechSoftConverter::CreateTopoLoop(const AlTrimBoundary& TrimBoundary)
{
	TArray<A3DTopoCoEdge*> Edges;
	Edges.Reserve(20);

	TAlObjectPtr<AlTrimCurve> TrimCurve(TrimBoundary.firstCurve());
	statusCode Status = TrimCurve ? sSuccess : sObjectNotFound;

	while(Status == sSuccess)
	{
		A3DTopoCoEdge* Edge = CreateEdge(*TrimCurve);
		if (Edge != nullptr)
		{
			Edges.Add(Edge);
		}

		Status = TrimCurve->nextCurveD();
	}

	if (Edges.Num() == 0)
	{
		return nullptr;
	}

	CADLibrary::TUniqueTSObj<A3DTopoLoopData> LoopData;

	LoopData->m_ppCoEdges = Edges.GetData();
	LoopData->m_uiCoEdgeSize = Edges.Num();
	LoopData->m_ucOrientationWithSurface = 1;

	return CADLibrary::TechSoftInterface::CreateTopoLoop(*LoopData);
}

void FAliasModelToTechSoftConverter::LinkEdgesLoop(const AlTrimBoundary& TrimBoundary)
{
	TAlObjectPtr<AlTrimCurve> TrimCurve(TrimBoundary.firstCurve());
	statusCode Status = TrimCurve ? sSuccess : sObjectNotFound;

	while (Status == sSuccess)
	{
		if (A3DTopoCoEdge** Edge = AlEdgeToTSCoEdge.Find(TrimCurve->fSpline))
		{
			TAlObjectPtr<AlTrimCurve> TwinCurve(TrimCurve->getTwinCurve());
			if (TwinCurve.IsValid())
			{
				if (A3DTopoCoEdge** TwinEdge = AlEdgeToTSCoEdge.Find(TwinCurve->fSpline))
				{
					CADLibrary::TechSoftInterface::LinkCoEdges(*Edge, *TwinEdge);
					break;
				}
			}
		}

		Status = TrimCurve->nextCurveD();
	}
}

A3DTopoFace* FAliasModelToTechSoftConverter::AddTrimRegion(const AlTrimRegion& InTrimRegion, const FColor& Color, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix)
{
	A3DSurfBase* CarrierSurface = AliasToTechSoftUtils::AddNURBSSurface(InTrimRegion, InObjectReference, InAlMatrix);
	if (CarrierSurface == nullptr)
	{
		return nullptr;
	}

	TArray<A3DTopoLoop*> Loops;
	Loops.Reserve(5);

	TAlObjectPtr<AlTrimBoundary> TrimBoundary(InTrimRegion.firstBoundary());
	statusCode Status = TrimBoundary ? sSuccess : sObjectNotFound;

	while(Status == sSuccess)
	{
		A3DTopoLoop* Loop = CreateTopoLoop(*TrimBoundary);
		if (Loop != nullptr)
		{
			Loops.Add(Loop);
			LinkEdgesLoop(*TrimBoundary);
		}

		Status = TrimBoundary->nextBoundaryD();
	}

	if (Loops.Num() == 0)
	{
		return nullptr;
	}

	CADLibrary::TUniqueTSObj<A3DTopoFaceData> Face;
	Face->m_pSurface = CarrierSurface;
	Face->m_bHasTrimDomain = false;
	Face->m_ppLoops = Loops.GetData();
	Face->m_uiLoopSize = Loops.Num();
	Face->m_uiOuterLoopIndex = 0;
	Face->m_dTolerance = 0.01; //mm

	A3DTopoFace* FacePtr = CADLibrary::TechSoftInterface::CreateTopoFace(*Face);

	CADLibrary::TechSoftUtils::SetEntityGraphicsColor(FacePtr, Color);

	return FacePtr;
}
#endif

bool FAliasModelToTechSoftConverter::AddBRep(const FAlDagNodePtr& DagNode, uint32 SlotID, EAliasObjectReference InObjectReference)
{
	FColor Color(SlotID);

	return AddBRep(DagNode, Color, InObjectReference);
}

bool FAliasModelToTechSoftConverter::AddBRep(const FAlDagNodePtr& DagNode, const FColor& Color, EAliasObjectReference InObjectReference)
{
#ifdef USE_TECHSOFT_SDK
	AlEdgeToTSCoEdge.Empty();

	AlMatrix4x4 AlMatrix;
	if (InObjectReference == EAliasObjectReference::ParentReference)
	{
		DagNode->localTransformationMatrix(AlMatrix);
	}

	boolean bAlOrientation;
	DagNode->getSurfaceOrientation(bAlOrientation);
	bool bOrientation = !(bool)bAlOrientation;

	TArray<A3DTopoFace*> TSFaces;
	TSFaces.Reserve(100);

	TAlObjectPtr<AlShell> Shell;
	if (DagNode.GetShell(Shell))
	{
		TAlObjectPtr<AlTrimRegion> TrimRegion(Shell->firstTrimRegion());
		statusCode Status = TrimRegion ? sSuccess : sObjectNotFound;
		while(Status == sSuccess)
		{
			A3DTopoFace* TSFace = AddTrimRegion(*TrimRegion, Color, InObjectReference, AlMatrix);
			if (TSFace != nullptr)
			{
				TSFaces.Add(TSFace);
			}

			Status = TrimRegion->nextRegionD();
		}
	}
	else
	{
		TAlObjectPtr<AlSurface> Surface;
		if (DagNode.GetSurface(Surface))
		{
			TAlObjectPtr<AlTrimRegion> TrimRegion(Surface->firstTrimRegion());
			statusCode Status = TrimRegion ? sSuccess : sObjectNotFound;
			if (Status == sSuccess)
			{
				while (Status == sSuccess)
				{
					A3DTopoFace* TSFace = AddTrimRegion(*TrimRegion, Color, InObjectReference, AlMatrix);
					if (TSFace != nullptr)
					{
						TSFaces.Add(TSFace);
					}

					Status = TrimRegion->nextRegionD();
				}
			}
			else
			{
				A3DSurfBase* TSSurface = AliasToTechSoftUtils::AddNURBSSurface(*Surface, InObjectReference, AlMatrix);
				if (TSSurface != nullptr)
				{
					A3DTopoFace* TSFace = CADLibrary::TechSoftUtils::CreateTopoFaceWithNaturalLoop(TSSurface);
					CADLibrary::TechSoftUtils::SetEntityGraphicsColor(TSFace, Color);

					if (TSFace != nullptr)
					{
						TSFaces.Add(TSFace);
					}
				}
			}
		}
	}

	if (TSFaces.IsEmpty())
	{
		return false;
	}

	A3DTopoShell* TopoShellPtr = nullptr;
	{
		TArray<A3DUns8> FaceOrientations;
		FaceOrientations.Init(bOrientation, TSFaces.Num());

		CADLibrary::TUniqueTSObj<A3DTopoShellData> TopoShellData;
		TopoShellData->m_bClosed = false;
		TopoShellData->m_ppFaces = TSFaces.GetData();
		TopoShellData->m_uiFaceSize = TSFaces.Num();
		TopoShellData->m_pucOrientationWithShell = FaceOrientations.GetData();

		TopoShellPtr = CADLibrary::TechSoftInterface::CreateTopoShell(*TopoShellData);

		if (TopoShellPtr == nullptr)
		{
			return false;
		}
	}

	A3DRiRepresentationItem* RiRepresentationItem = CADLibrary::TechSoftUtils::CreateRIBRep(TopoShellPtr);

	if (RiRepresentationItem != nullptr)
	{
		RiRepresentationItems.Add(RiRepresentationItem);
		return true;
	}
#endif
	return false;
}

bool FAliasModelToTechSoftConverter::AddGeometry(const CADLibrary::FCADModelGeometry& Geometry)
{
	if (Geometry.Type == (int32)ECADModelGeometryType::DagNode)
	{
		const FDagNodeGeometry& DagNodeGeometry = static_cast<const FDagNodeGeometry&>(Geometry);

		return AddBRep(DagNodeGeometry.DagNode, 0, DagNodeGeometry.Reference);
	}
	else if (Geometry.Type == (int32)ECADModelGeometryType::BodyNode)
	{
		const FBodyNodeGeometry& BodyNodeGeometry = static_cast<const FBodyNodeGeometry&>(Geometry);

		bool bBodyAdded = false;
		BodyNodeGeometry.BodyNode->IterateOnDagNodes([&](const FAlDagNodePtr& DagNode)
			{
				// #wire_import: TODO: Inform user one DagNode was not imported
				const bool bBRepAdded = AddBRep(DagNode, BodyNodeGeometry.BodyNode->GetSlotIndex(DagNode), BodyNodeGeometry.Reference);
				if (!bBRepAdded)
				{
					UE_LOGF(LogWireInterface, Warning, "Failed to add DagNode %ls to StaticMesh.", *DagNode.GetName());
				}
				bBodyAdded |= bBRepAdded;
			});

		return bBodyAdded;
	}

	return false;
}

}
#endif

#undef LOCTEXT_NAMESPACE // "WireInterface"
