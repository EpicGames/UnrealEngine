// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifdef USE_OPENMODEL
#include "OpenModelUtils.h"

#include "CADModelToTechSoftConverterBase.h"

#include "Core/Session.h"
#include "Core/Types.h"

class AlDagNode;
class AlShell;
class AlSurface;
class AlTrimBoundary;
class AlTrimCurve;
class AlTrimRegion;

typedef void A3DTopoCoEdge;
typedef void A3DTopoFace;
typedef void A3DTopoLoop;
typedef void A3DTopoCoEdge;
typedef void A3DCrvBase;

namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{

class FAliasModelToTechSoftConverter : public FCADModelToTechSoftConverterBase
{

public:
	FAliasModelToTechSoftConverter(CADLibrary::FImportParameters InImportParameters)
		: FCADModelToTechSoftConverterBase(InImportParameters)
	{
	}

	virtual bool AddGeometry(const CADLibrary::FCADModelGeometry& Geometry) override;

	bool AddBRep(const FAlDagNodePtr& DagNode, const FColor& Color, EAliasObjectReference ObjectReference);
	bool AddBRep(const FAlDagNodePtr& DagNode, uint32 SlotID, EAliasObjectReference ObjectReference);

protected:
	TMap<void*, A3DTopoCoEdge*> AlEdgeToTSCoEdge;

	A3DTopoFace* AddTrimRegion(const AlTrimRegion& InTrimRegion, const FColor& Color, EAliasObjectReference InObjectReference, const AlMatrix4x4& InAlMatrix);
	A3DTopoLoop* CreateTopoLoop(const AlTrimBoundary& TrimBoundary);
	A3DTopoCoEdge* CreateEdge(const AlTrimCurve& TrimCurve);
	A3DCrvBase* CreateCurve(const AlTrimCurve& TrimCurve);

	void LinkEdgesLoop(const AlTrimBoundary& TrimBoundary);
};

}
#endif

