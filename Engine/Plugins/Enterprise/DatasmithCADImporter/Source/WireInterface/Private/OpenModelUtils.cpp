// Copyright Epic Games, Inc. All Rights Reserved.
#include "OpenModelUtils.h"

#ifdef USE_OPENMODEL
#include "CADOptions.h"
#include "DatasmithUtils.h"
#include "DatasmithTranslator.h"

#include "MeshAttributes.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include "AlChannel.h"
#include "AlDagNode.h"
#include "AlGroupNode.h"
#include "AlLayer.h"
#include "AlLinkItem.h"
#include "AlList.h"
#include "AlMesh.h"
#include "AlMeshNode.h"
#include "AlPersistentID.h"
#include "AlRetrieveOptions.h"
#include "AlShader.h"
#include "AlShadingFieldItem.h"
#include "AlShell.h"
#include "AlShellNode.h"
#include "AlSurface.h"
#include "AlSurfaceNode.h"
#include "AlTesselate.h"
#include "AlTrimRegion.h"
#include "AlTM.h"
#include "AlUniverse.h"

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif


namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{
	DEFINE_LOG_CATEGORY(LogWireInterface);

	bool FPatchMesh::Initialize()
	{
		if (MeshNodes.IsEmpty())
		{
			return false;
		}

		if (!bInitialized)
		{
			Hash = GetTypeHash(Name);

			for (const FAlDagNodePtr& MeshNode : MeshNodes)
			{
				Hash = HashCombine(Hash, MeshNode.GetHash());
			}

			UniqueID = TEXT("PatchMesh") + FString::FromInt(Hash);

			bInitialized = true;
		}

		return bInitialized;
	}

	int32 FBodyNode::GetSlotIndex(const FAlDagNodePtr& DagNode)
	{
		// #wire_import: Add support for AlShell
		if (DagNode.IsASurface())
		{
			TAlObjectPtr<AlSurface> Surface;
			if (DagNode.GetSurface(Surface))
			{
				TAlObjectPtr<AlShader> Shader(Surface->firstShader());
				return Shader ? ShaderNameToSlotIndex[Shader.GetName()] : 0;
			}
		}

		ensureWire(false);
		return 0;
	}

	bool FBodyNode::Initialize()
	{
		if (DagNodes.IsEmpty())
		{
			return false;
		}

		if (!bInitialized)
		{
			Hash = GetTypeHash(Name);

			int32 SlotIndex = 0;
			TMap<FString, AlLayer*> LayerSet;

			TFunction<void(const TAlObjectPtr<AlShader>&)> RegisterShader = [this, &SlotIndex](const TAlObjectPtr<AlShader>& Shader)
				{
					if (Shader && !ShaderNameToSlotIndex.Contains(Shader.GetName()))
					{
						ShaderNameToSlotIndex.Add(Shader.GetName(), SlotIndex);
						SlotIndexToShader.Add(SlotIndex++, Shader);
					}
				};

			for (const FAlDagNodePtr& DagNode : DagNodes)
			{
				TAlObjectPtr<AlSurface> Surface;
				if (DagNode.GetSurface(Surface))
				{
					TAlObjectPtr<AlShader> Shader(Surface->firstShader());
					ensureWire(Shader);
					RegisterShader(Shader);
				}
				else
				{
					TAlObjectPtr<AlShell> Shell;
					if (DagNode.GetShell(Shell))
					{
						TAlObjectPtr<AlShader> Shader(Shell->firstShader());
						RegisterShader(Shader);
						// #wire_import: Do we have as many shaders than trim regions
#if WIRE_ENSURE_ENABLED
						int32 ShaderCount = 0;
						{
							statusCode Status = Shader.IsValid() ? sSuccess : sFailure;
							while (Status == sSuccess)
							{
								++ShaderCount;
								Status = Shell->nextShaderD(Shader.Get());
							};
						}
						TAlObjectPtr<AlTrimRegion> TrimRegion(Shell->firstTrimRegion());
						int32 TrimCount = 0;
						{
							statusCode Status = TrimRegion.IsValid() ? sSuccess : sFailure;
							while (Status == sSuccess)
							{
								++TrimCount;
								Status = TrimRegion->nextRegionD();
							};
						}
						ensureWire(ShaderCount == TrimCount);
#endif
					}
					else
					{
						ensure(false);
					}
				}


				if (DagNode.GetLayer())
				{
					LayerSet.Add(DagNode.GetLayerName(), DagNode.GetLayer().Get());
				}

				Hash = HashCombine(Hash, DagNode.GetHash());
			}

			ensureWire(LayerSet.Num() == 1);
			// #wire_import: TODO - Make sure Body's layer is the same as those of the added geometries

			UniqueID = TEXT("BodyNode") + FString::FromInt(Hash);

			bInitialized = true;
		}

		return bInitialized;
	}

	bool FBodyNode::AddNode(FAlDagNodePtr& DagNode)
	{
		if (DagNode.IsASurface() || DagNode.IsAShell())
		{
			DagNodes.Add(DagNode);
			return true;
		}
#if WIRE_ENSURE_ENABLED
		ensureWire(false);
#endif

		return false;
	}

	template<>
	uint32 TAlObjectPtr<AlLayer>::GetHash() const
	{
		uint32 Hash = GetTypeHash(GetName());
		if (IsValid())
		{
			Hash = HashCombine(Hash, GetTypeHash(Super::Get()->number()));
			Hash = HashCombine(Hash, GetTypeHash(Super::Get()->color()));
			Hash = HashCombine(Hash, (bool)Super::Get()->invisible() ? 1 : 0);
			Hash = HashCombine(Hash, (bool)Super::Get()->isSymmetric() ? 1 : 0);
		}

		return Hash;
	}

	bool OpenModelUtils::GetCsvLayerString(const TAlObjectPtr<AlLayer>& Layer, FString& CsvString)
	{
		if (!Layer)
		{
			return false;
		}

		CsvString = Layer.GetName();

		TAlObjectPtr<AlLayer> ParentLayer = FLayerContainer::FindOrAdd(Layer->parentLayer());
		while (ParentLayer)
		{
			FString ParentLayerName = ParentLayer.GetName();
			if (!ParentLayerName.IsEmpty())
			{
				CsvString += TEXT(",") + ParentLayerName;
			}

			ParentLayer = FLayerContainer::FindOrAdd(ParentLayer->parentLayer());
		}

		return !CsvString.IsEmpty();
	}

	bool OpenModelUtils::ActorHasContent(const TSharedPtr<IDatasmithActorElement>& ActorElement)
	{
		if (!ActorElement.IsValid())
		{
			return false;
		}

		return ActorElement->IsA(EDatasmithElementType::StaticMeshActor) || ActorElement->GetChildrenCount() > 0;
	}

	bool OpenModelUtils::IsValidActor(const TSharedPtr<IDatasmithActorElement>& ActorElement)
	{
		if (ActorElement != nullptr)
		{
			if (ActorElement->GetChildrenCount() > 0)
			{
				return true;
			}
			else if (ActorElement->IsA(EDatasmithElementType::StaticMeshActor))
			{
				const TSharedPtr<IDatasmithMeshActorElement>& MeshActorElement = StaticCastSharedPtr<IDatasmithMeshActorElement>(ActorElement);
				return FCString::Strlen(MeshActorElement->GetStaticMeshPathName()) > 0;
			}
		}
		return false;
	}

	bool OpenModelUtils::TransferAlMeshToMeshDescription(const AlMesh& AliasMesh, const TCHAR* SlotMaterialId, FMeshDescription& MeshDescription, CADLibrary::FMeshParameters& MeshParameters, const bool bMerge)
	{
		if (AliasMesh.numberOfVertices() == 0 || AliasMesh.numberOfTriangles() == 0)
		{
			return false;
		}

		if (!bMerge)
		{
			MeshDescription.Empty();
		}

		int32 NbStep = 1;
		FMatrix44f SymmetricMatrix;
		bool bIsSymmetricMesh = MeshParameters.bIsSymmetric;
		if (bIsSymmetricMesh)
		{
			NbStep = 2;
			SymmetricMatrix = FDatasmithUtils::GetSymmetricMatrix(MeshParameters.SymmetricOrigin, MeshParameters.SymmetricNormal);
		}

		// Gather all array data
		FStaticMeshAttributes Attributes(MeshDescription);
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
		TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();

		// Prepared for static mesh usage ?
		if (!VertexPositions.IsValid() || !VertexInstanceNormals.IsValid() || !VertexInstanceUVs.IsValid() || !PolygonGroupImportedMaterialSlotNames.IsValid())
		{
			return false;
		}

		bool bHasUVData = (AliasMesh.uvs() != nullptr);

		int VertexCount = AliasMesh.numberOfVertices();
		int TriangleCount = AliasMesh.numberOfTriangles();
		const int32 VertexInstanceCount = 3 * TriangleCount;

		TArray<FVertexID> VertexPositionIDs;
		VertexPositionIDs.SetNum(VertexCount * NbStep);

		// Reserve space for attributes
		// At this point, all the faces are triangles
		MeshDescription.ReserveNewVertices(VertexCount * NbStep);
		MeshDescription.ReserveNewVertexInstances(VertexInstanceCount * NbStep);
		MeshDescription.ReserveNewEdges(VertexInstanceCount * NbStep);
		MeshDescription.ReserveNewPolygons(TriangleCount * NbStep);

		MeshDescription.ReserveNewPolygonGroups(1);
		FPolygonGroupID PolyGroupId = MeshDescription.CreatePolygonGroup();
		FName ImportedSlotName = SlotMaterialId;
		PolygonGroupImportedMaterialSlotNames[PolyGroupId] = ImportedSlotName;

		// At least one UV set must exist.
		if (VertexInstanceUVs.GetNumChannels() == 0)
		{
			VertexInstanceUVs.SetNumChannels(1);
		}

		// Get Alias mesh info
		const float* AlVertices = AliasMesh.vertices();

		for (int32 Step = 0; Step < NbStep; Step++)
		{
			// Fill the vertex array
			if (Step == 0)
			{
				FVertexID* VertexPositionIDPtr = VertexPositionIDs.GetData();
				for (int Index = 0; Index < VertexCount; ++Index, ++VertexPositionIDPtr)
				{
					const float* CurVertex = AlVertices + 3 * Index;
					*VertexPositionIDPtr = MeshDescription.CreateVertex();
					// ConvertVector_ZUp_RightHanded
					VertexPositions[*VertexPositionIDPtr] = FVector3f(-CurVertex[0], CurVertex[1], CurVertex[2]);
				}
			}
			else
			{
				FVertexID* VertexPositionIDPtr = VertexPositionIDs.GetData() + VertexCount;
				for (int Index = 0, PositionIndex = VertexCount; Index < VertexCount; ++Index, ++VertexPositionIDPtr)
				{
					const float* CurVertex = AlVertices + 3 * Index;
					*VertexPositionIDPtr = MeshDescription.CreateVertex();
					// ConvertVector_ZUp_RightHanded
					VertexPositions[*VertexPositionIDPtr] = SymmetricMatrix.TransformPosition(FVector3f(-CurVertex[0], CurVertex[1], CurVertex[2]));
				}
			}

			FBox UVBBox(FVector(MAX_FLT), FVector(-MAX_FLT));

			const int32 CornerCount = 3; // only triangles
			FVertexID CornerVertexIDs[3];
			TArray<FVertexInstanceID> CornerVertexInstanceIDs;
			CornerVertexInstanceIDs.SetNum(3);

			// Get Alias mesh info
			const int* Triangles = AliasMesh.triangles();
			const float* AlNormals = AliasMesh.normals();
			const float* AlUVs = AliasMesh.uvs();

			// Get per-triangle data: indices, normals and uvs
			if (!MeshParameters.bNeedSwapOrientation == ((bool)Step))
			{
				for (int32 FaceIndex = 0; FaceIndex < TriangleCount; ++FaceIndex, Triangles += 3)
				{
					// Create Vertex instances and set their attributes
					for (int32 VertexIndex = 0, TIndex = 2; VertexIndex < CornerCount; ++VertexIndex, --TIndex)
					{
						CornerVertexIDs[VertexIndex] = VertexPositionIDs[Triangles[TIndex] + VertexCount * Step];
						CornerVertexInstanceIDs[VertexIndex] = MeshDescription.CreateVertexInstance(CornerVertexIDs[VertexIndex]);

						// Set the normal
						const float* CurNormal = &AlNormals[3 * Triangles[TIndex]];
						// ConvertVector_ZUp_RightHanded
						FVector3f UENormal(-CurNormal[0], CurNormal[1], CurNormal[2]);
						UENormal = UENormal.GetSafeNormal();
						if (Step > 0)
						{
							UENormal = SymmetricMatrix.TransformVector(UENormal);
						}
						else
						{
							UENormal *= -1.;
						}
						VertexInstanceNormals[CornerVertexInstanceIDs[VertexIndex]] = UENormal;
					}
					if (CornerVertexIDs[0] == CornerVertexIDs[1] || CornerVertexIDs[0] == CornerVertexIDs[2] || CornerVertexIDs[1] == CornerVertexIDs[2])
					{
						continue;
					}

					// Set the UV
					if (bHasUVData)
					{
						//for (int32 VertexIndex = 2; VertexIndex >= 0; --VertexIndex)
						for (int32 VertexIndex = 0, TIndex = 2; VertexIndex < CornerCount; ++VertexIndex, --TIndex)
						{
							FVector2D UVValues(AlUVs[2 * Triangles[TIndex] + 0], AlUVs[2 * Triangles[TIndex] + 1]);
							UVBBox += FVector(UVValues, 0.0f);
							VertexInstanceUVs.Set(CornerVertexInstanceIDs[VertexIndex], 0, FVector2f(UVValues));
						}
					}

					// Triangulate
					const FPolygonID NewPolygonID = MeshDescription.CreatePolygon(PolyGroupId, CornerVertexInstanceIDs);
				}
			}
			else
			{
				for (int32 FaceIndex = 0; FaceIndex < TriangleCount; ++FaceIndex, Triangles += 3)
				{
					// Create Vertex instances and set their attributes
					for (int32 VertexIndex = 0; VertexIndex < CornerCount; ++VertexIndex)
					{
						CornerVertexIDs[VertexIndex] = VertexPositionIDs[Triangles[VertexIndex] + VertexCount * Step];
						CornerVertexInstanceIDs[VertexIndex] = MeshDescription.CreateVertexInstance(CornerVertexIDs[VertexIndex]);

						// Set the normal
						const float* CurNormal = &AlNormals[3 * Triangles[VertexIndex]];

						// ConvertVector_ZUp_RightHanded
						FVector3f UENormal(-CurNormal[0], CurNormal[1], CurNormal[2]);
						UENormal = UENormal.GetSafeNormal();
						if (Step > 0)
						{
							UENormal = SymmetricMatrix.TransformVector(UENormal) * -1;
						}
						VertexInstanceNormals[CornerVertexInstanceIDs[VertexIndex]] = (FVector3f)UENormal;
					}
					if (CornerVertexIDs[0] == CornerVertexIDs[1] || CornerVertexIDs[0] == CornerVertexIDs[2] || CornerVertexIDs[1] == CornerVertexIDs[2])
					{
						continue;
					}

					// Set the UV
					if (bHasUVData)
					{
						for (int32 VertexIndex = 0; VertexIndex < CornerCount; ++VertexIndex)
						{
							FVector2D UVValues(AlUVs[2 * Triangles[VertexIndex] + 0], AlUVs[2 * Triangles[VertexIndex] + 1]);
							UVBBox += FVector(UVValues, 0.0f);
							VertexInstanceUVs.Set(CornerVertexInstanceIDs[VertexIndex], 0, FVector2f(UVValues));
						}
					}

					// Triangulate
					const FPolygonID NewPolygonID = MeshDescription.CreatePolygon(PolyGroupId, CornerVertexInstanceIDs);
				}
			}
		}

		return true;
	}

	FAlDagNodePtr OpenModelUtils::TesselateDagLeaf(const AlDagNode& DagLeaf, ETesselatorType TessType, double Tolerance)
	{
		AlDagNode* TesselatedNode = nullptr;
		statusCode TessStatus;

		switch (TessType)
		{
		case(ETesselatorType::Accurate):
			TessStatus = AlTesselate::chordHeightDeviationAccurate(TesselatedNode, &DagLeaf, Tolerance);
			break;
		case(ETesselatorType::Fast):
		default:
			TessStatus = AlTesselate::chordHeightDeviationFast(TesselatedNode, &DagLeaf, Tolerance);
			break;
		}

		return TessStatus == sSuccess ? TesselatedNode : FAlDagNodePtr();
	}

	CADLibrary::FMeshParameters OpenModelUtils::GetMeshParameters(const TAlObjectPtr<AlLayer>& Layer)
	{
		CADLibrary::FMeshParameters MeshParameters;

		if (Layer)
		{
			if (Layer->isSymmetric())
			{
				MeshParameters.bIsSymmetric = true;
				double Normal[3], Origin[3];
				Layer->symmetricNormal(Normal[0], Normal[1], Normal[2]);
				Layer->symmetricOrigin(Origin[0], Origin[1], Origin[2]);

				MeshParameters.SymmetricOrigin.X = (float)Origin[0];
				MeshParameters.SymmetricOrigin.Y = (float)Origin[1];
				MeshParameters.SymmetricOrigin.Z = (float)Origin[2];
				MeshParameters.SymmetricNormal.X = (float)Normal[0];
				MeshParameters.SymmetricNormal.Y = (float)Normal[1];
				MeshParameters.SymmetricNormal.Z = (float)Normal[2];
			}
		}

		return MeshParameters;
	}

	CADLibrary::FMeshParameters FAlDagNodePtr::GetMeshParameters() const
	{
		if (!IsValid())
		{
			return {};
		}

		CADLibrary::FMeshParameters MeshParameters = OpenModelUtils::GetMeshParameters(GetLayer());

		boolean bAlOrientation;
		AsADagNode()->getSurfaceOrientation(bAlOrientation);

		MeshParameters.bNeedSwapOrientation = IsAMesh() ? (bool)bAlOrientation : false;

		return MeshParameters;
	}

	TMap<AlLayer*, TAlObjectPtr<AlLayer>> FLayerContainer::LayerMap;

	TAlObjectPtr<AlLayer> FLayerContainer::FindOrAdd(AlLayer* Layer)
	{
		if (const TAlObjectPtr<AlLayer>* ValuePtr = LayerMap.Find(Layer))
		{
			return *ValuePtr;
		}

		TAlObjectPtr<AlLayer>& LayerPtr = LayerMap.Add(Layer);
		LayerPtr = Layer;

		return LayerPtr;
	}

	void FLayerContainer::Reset()
	{
		LayerMap.Reset();
	}
}

#endif


