// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifdef USE_OPENMODEL
#include "CADModelConverter.h"
#include "CADOptions.h"
#include "IDatasmithSceneElements.h"

#include "Misc/Optional.h"

#if WITH_EDITOR
#include "Editor.h"
#include "IMessageLogListing.h"
#include "Logging/TokenizedMessage.h"
#include "MessageLogModule.h"
#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include "AlShadingFields.h"
#include "AlDagNode.h"
#include "AlLayer.h"
#include "AlMesh.h"
#include "AlMeshNode.h"
#include "AlShader.h"
#include "AlShell.h"
#include "AlShellNode.h"
#include "AlSurface.h"
#include "AlSurfaceNode.h"
#include "AlPersistentID.h"

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

class IDatasmithActorElement;

struct FMeshDescription;

#define LAYER_TYPE          TEXT("Layer")
#define GROUPNODE_TYPE      TEXT("GroupNode")
#define MESH_TYPE           TEXT("Mesh")
#define MESHNODE_TYPE       TEXT("MeshNode")
#define SHADER_TYPE         TEXT("Shader")
#define SHELLNODE_TYPE      TEXT("ShellNode")
#define SHELL_TYPE          TEXT("Shell")
#define SURFACE_TYPE        TEXT("Surface")
#define SURFACENODE_TYPE    TEXT("SurfaceNode")

// Convert distance from UE (in cm) to CADKernel (in mm)
#define UNIT_CONVERSION_CM_TO_MM		10.
#define UE_TO_CADKERNEL(Distance)		(Distance * 10.)

#define WIRE_MEMORY_CHECK 0
#define WIRE_ENSURE_ENABLED 0
#if WIRE_ENSURE_ENABLED
#define ensureWire(InExpression) ensure(InExpression)
#else
#define ensureWire(InExpression)
#endif

namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{
	DECLARE_LOG_CATEGORY_EXTERN(LogWireInterface, Log, All)

#if WIRE_MEMORY_CHECK
	extern TSet<AlDagNode*> DagNodeSet;
	extern TSet<AlObject*> ObjectSet;
	extern int32 AllocatedObjects;
	extern int32 MaxAllocatedObjects;
#endif

	typedef double AlMatrix4x4[4][4];

	enum class ETesselatorType : uint8
	{
		Fast,
		Accurate,
	};

	enum class EAlShaderModelType : uint8
	{
		BLINN,
		LAMBERT,
		LIGHTSOURCE,
		PHONG,
	};

	template<typename T>
	class TAlObjectPtr : public TSharedPtr<T>
	{
		using Super = TSharedPtr<T>;

	public:
		TAlObjectPtr(T* Object = nullptr) : Super(Object)
		{
			static_assert(std::is_base_of<AlObject, T>::value);
#if WIRE_MEMORY_CHECK
			ensure(!Object || !ObjectSet.Contains(Object));
			if (IsValid() && Super::GetSharedReferenceCount() == 1)
			{
				++AllocatedObjects;
				if (AllocatedObjects > MaxAllocatedObjects) { MaxAllocatedObjects = AllocatedObjects; }
				ObjectSet.Add(Object);
			}
#endif
		}

		~TAlObjectPtr()
		{
#if WIRE_MEMORY_CHECK
			if (Super::IsValid() && Super::GetSharedReferenceCount() == 1)
			{
				ensure(ObjectSet.Contains(Super::Get()));
				--AllocatedObjects;
				ObjectSet.Remove(Super::Get());
			}
#endif
			Super::Reset();
		}

		bool IsValid() const { return Super::IsValid() && AlIsValid((const AlObject*)Super::Get()); }

		FString GetName() const
		{
			return IsValid() ? StringCast<TCHAR>(Super::Get()->name()).Get() : FString();
		}

		uint32 GetHash() const
		{
			if (IsValid())
			{
				uint32 NameHash = GetTypeHash(GetName());
				uint32 TypeHash = GetTypeHash(Super::Get()->type());
				return HashCombine(NameHash, TypeHash);
			}

			return -1;
		}

		FString GetUniqueID(const TCHAR* TypeName = TEXT("Object")) const
		{
			return FString(TypeName) + FString::FromInt(GetHash());
		}

		operator bool() const
		{
			return IsValid();
		}

		friend uint32 GetTypeHash(const TAlObjectPtr<T>& Object)
		{
			return Object.GetHash()/*GetTypeHash(DateTime.Ticks)*/;
		}
	};

	template<typename T>
	inline bool operator==(const TAlObjectPtr<T>& A, const TAlObjectPtr<T>& B)
	{
		return (bool)AlAreEqual((const AlObject*)A.Get(), (const AlObject*)B.Get());
	}

	template<typename T>
	inline bool operator!=(const TAlObjectPtr<T>& A, const TAlObjectPtr<T>& B)
	{
		return !((bool)AlAreEqual((const AlObject*)A.Get(), (const AlObject*)B.Get()));
	}

	template<>
	uint32 TAlObjectPtr<AlLayer>::GetHash() const;

	class FLayerContainer
	{
	public:
		static void Reset();
		static TAlObjectPtr<AlLayer> FindOrAdd(AlLayer* Layer);

	private:
		static TMap<AlLayer*, TAlObjectPtr<AlLayer>> LayerMap;
	};

	enum  class EDagNodeType : uint8
	{
		Unknown = 0x00,
		MeshType = 0x01,
		SurfaceType = 0x02,
		ShellType = 0x04,
		GroupType = 0x08,
		GeometryType = MeshType | SurfaceType | ShellType,
	};
	ENUM_CLASS_FLAGS(EDagNodeType);

	class FAlDagNodePtr : public TAlObjectPtr<AlDagNode>
	{
		using Super = TSharedPtr<AlDagNode>;

	public:
		FAlDagNodePtr(AlDagNode* DagNode = nullptr)
			: TAlObjectPtr<AlDagNode>(DagNode)
			, Type(EDagNodeType::Unknown)
		{
#if WIRE_MEMORY_CHECK
			ensure(!DagNode || !DagNodeSet.Contains(DagNode));
#endif

			if (AlIsValid(DagNode))
			{
				if (DagNode->type() == AlObjectType::kMeshNodeType && DagNode->asMeshNodePtr() != nullptr)
				{
					EnumAddFlags(Type, EDagNodeType::MeshType);
				}
				else if (DagNode->type() == AlObjectType::kSurfaceNodeType && DagNode->asSurfaceNodePtr() != nullptr)
				{
					EnumAddFlags(Type, EDagNodeType::SurfaceType);
				}
				else if (DagNode->type() == AlObjectType::kShellNodeType && DagNode->asShellNodePtr() != nullptr)
				{
					EnumAddFlags(Type, EDagNodeType::ShellType);
				}
				else if (DagNode->type() == AlObjectType::kGroupNodeType && DagNode->asGroupNodePtr() != nullptr)
				{
					EnumAddFlags(Type, EDagNodeType::GroupType);
				}

				CachedLayer = FLayerContainer::FindOrAdd(DagNode->layer());
				LayerName = CachedLayer ? FString(StringCast<TCHAR>(CachedLayer->name()).Get()) : FString();

				bCanDeleteObject = DagNode->parentNode() == nullptr;
			}
#if WIRE_MEMORY_CHECK
			if (DagNode)
			{
				DagNodeSet.Add(DagNode);
			}
#endif
		}

		~FAlDagNodePtr()
		{
			if (TAlObjectPtr<AlDagNode>::IsValid())
			{
				if (Super::GetSharedReferenceCount() == 1)
				{
					CachedLayer = nullptr;
					if (bCanDeleteObject)
					{
						Super::Get()->deleteObject();
					}

#if WIRE_MEMORY_CHECK
					ensure(AllocatedObjects > 0);
					ensure(DagNodeSet.Contains(Super::Get()));
					DagNodeSet.Remove(Super::Get());
#endif
				}
			}
			TAlObjectPtr<AlDagNode>::~TAlObjectPtr();
		}

		FString GetLayerName() const
		{
			return LayerName.IsSet() ? LayerName.GetValue() : FString();
		}

		const TAlObjectPtr<AlLayer>& GetLayer() const
		{
			return CachedLayer;
		}

		bool HasSymmetry() const
		{
			return CachedLayer ? (bool)CachedLayer->isSymmetric() : false;
		}

		bool IsVisible() const
		{
			return CachedLayer.IsValid() ? !((bool)CachedLayer->invisible()) : false;
		}

		AlDagNode* AsADagNode() const
		{
			return static_cast<AlDagNode*>(Super::Get());
		}

		bool HasGeometry() const
		{ 
			return EnumHasAnyFlags(Type, EDagNodeType::GeometryType);
		}

		bool IsAGroup() const
		{
			return EnumHasAnyFlags(Type, EDagNodeType::GroupType);
		}

		bool IsAMesh() const
		{
			return EnumHasAnyFlags(Type, EDagNodeType::MeshType);
		}

		bool IsASurface() const
		{
			return EnumHasAnyFlags(Type, EDagNodeType::SurfaceType);
		}

		bool IsAShell() const
		{
			return EnumHasAnyFlags(Type, EDagNodeType::ShellType);
		}

		bool GetMesh(TAlObjectPtr<AlMesh>& OutMesh) const
		{
			OutMesh = IsAMesh() ? AsADagNode()->asMeshNodePtr()->mesh() : TAlObjectPtr<AlMesh>();

			return OutMesh.IsValid();
		}

		bool GetSurface(TAlObjectPtr<AlSurface>& OutSurface) const
		{
			OutSurface = IsASurface() ? AsADagNode()->asSurfaceNodePtr()->surface() : TAlObjectPtr<AlSurface>();

			return OutSurface.IsValid();
		}

		bool GetShell(TAlObjectPtr<AlShell>& OutShell) const
		{
			OutShell = IsAShell() ? AsADagNode()->asShellNodePtr()->shell() : TAlObjectPtr<AlShell>();

			return OutShell.IsValid();
		}

		CADLibrary::FMeshParameters GetMeshParameters() const;

		void SetActorTransform(IDatasmithActorElement& ActorElement) const
		{
			// Node with symmetry cannot be baked with the global transform because the symmetry is done in the parent referential
			if (HasSymmetry())
			{
				return;
			}

			AlMatrix4x4 AlGlobalMatrix;
			AsADagNode()->globalTransformationMatrix(AlGlobalMatrix);

			FMatrix GlobalMatrix;
			double* MatrixFloats = (double*)GlobalMatrix.M;
			for (int32 IndexI = 0; IndexI < 4; ++IndexI)
			{
				for (int32 IndexJ = 0; IndexJ < 4; ++IndexJ)
				{
					MatrixFloats[IndexI * 4 + IndexJ] = AlGlobalMatrix[IndexI][IndexJ];
				}
			}

			FTransform GlobalTransform = FDatasmithUtils::ConvertTransform(FDatasmithUtils::EModelCoordSystem::ZUp_RightHanded, FTransform(GlobalMatrix));

			ActorElement.SetTranslation(GlobalTransform.GetTranslation());
			ActorElement.SetScale(GlobalTransform.GetScale3D());
			ActorElement.SetRotation(GlobalTransform.GetRotation());
		}

	private:
		mutable TOptional<FString> LayerName;
		mutable TAlObjectPtr<AlLayer> CachedLayer;
		bool bCanDeleteObject = false;
		EDagNodeType Type;
	};

	class FPatchMesh
	{
	public:
		FPatchMesh(const FString& InName, TAlObjectPtr<AlLayer>& InLayer, int32 Count)
			: Name(InName)
			, Layer(InLayer)
		{
			MeshNodes.Reserve(Count);
		}

		bool HasContent()
		{
			return Initialize() && !MeshNodes.IsEmpty();
		}

		bool HasSingleContent() const
		{
			return MeshNodes.Num() == 1;
		}

		bool GetSingleContent(FAlDagNodePtr& OutMeshNode)
		{
			if (MeshNodes.Num() == 1)
			{
				FAlDagNodePtr& MeshNode = MeshNodes.Last();
				OutMeshNode = MoveTemp(MeshNode);
				MeshNodes.SetNum(0, EAllowShrinking::No);
				return true;
			}

			return false;
		}

		const FString& GetName() const { return Name; }

		uint32 GetHash() const { return Hash; }

		const FString& GetUniqueID() const { return UniqueID; }

		const TAlObjectPtr<AlLayer>& GetLayer() const { return Layer; }

		void AddMeshNode(FAlDagNodePtr& MeshNode)
		{
			ensureWire(Layer == MeshNode.GetLayer());
			MeshNodes.Add(MeshNode);
		}

		void IterateOnMeshNodes(const TFunction<void(const FAlDagNodePtr& MeshNode)>& Callback) const
		{
			for (const FAlDagNodePtr& MeshNode : MeshNodes)
			{
				Callback(MeshNode);
			}
		}

		bool Initialize();

	private:
		FString Name;
		TArray<FAlDagNodePtr> MeshNodes;
		TAlObjectPtr<AlLayer> Layer;
		uint32 Hash;
		FString UniqueID;
		bool bInitialized = false;
	};

	class FBodyNode
	{
	public:
		FBodyNode(const FString& InName, TAlObjectPtr<AlLayer>& InLayer, int32 Count)
			: Name(InName)
			, Layer(InLayer)
		{
			DagNodes.Reserve(Count);
		}

		bool HasContent()
		{
			return Initialize() && !DagNodes.IsEmpty();
		}
		 
		bool HasSingleContent() const
		{
			return DagNodes.Num() == 1;
		}

		bool GetSingleContent(FAlDagNodePtr& OutDagNode) const
		{
			if (HasSingleContent())
			{
				FAlDagNodePtr&  DagNode = DagNodes.Last();
				OutDagNode = MoveTemp(DagNode);
				DagNodes.SetNum(0, EAllowShrinking::No);
				return true;
			}

			return false;
		}

		const FString& GetName() const { return Name; }

		uint32 GetHash() const { return Hash; }

		const FString& GetUniqueID() const { return UniqueID; }

		const TAlObjectPtr<AlLayer>& GetLayer() const { return Layer; }

		bool AddNode(FAlDagNodePtr& DagNode);

		void IterateOnDagNodes(const TFunction<void(const FAlDagNodePtr&)>& Callback) const
		{
			for (const FAlDagNodePtr& DagNode : DagNodes)
			{
				if (DagNode)
				{
					Callback(DagNode);
				}
			}
		}

		void IterateOnSlotIndices(const TFunction<void(int SlotIndex, const TAlObjectPtr<AlShader>& Shader)>& Callback) const
		{
			for (const TPair<int, TAlObjectPtr<AlShader>>& Entry : SlotIndexToShader)
			{
				Callback(Entry.Key, Entry.Value);
			}
		}

		bool Initialize();

		int32 GetSlotIndex(const FAlDagNodePtr& DagNode);

	private:
		FString Name;
		mutable TArray<FAlDagNodePtr> DagNodes;
		TAlObjectPtr<AlLayer> Layer;
		TMap<FString, int> ShaderNameToSlotIndex;
		TMap<int, TAlObjectPtr<AlShader>> SlotIndexToShader;
		uint32 Hash;
		FString UniqueID;
		bool bInitialized = false;
	};

	enum class ECADModelGeometryType : int32
	{
		DagNode,
		MeshNode,
		BodyNode,
		PatchMesh,
	};

	enum class EAliasObjectReference
	{
		LocalReference,  
		ParentReference, 
		WorldReference, 
	};
	
	struct FAliasGeometry : public CADLibrary::FCADModelGeometry
	{
		EAliasObjectReference Reference = EAliasObjectReference::LocalReference;
	};

	struct FDagNodeGeometry : public FAliasGeometry
	{
		const FAlDagNodePtr& DagNode;

		FDagNodeGeometry(int32 InType, EAliasObjectReference InReference, const FAlDagNodePtr& InDagNode)
			: DagNode(InDagNode)
		{
			Type = InType;
			Reference = InReference;
		}
	};

	struct FBodyNodeGeometry : public FAliasGeometry
	{
		TSharedPtr<FBodyNode> BodyNode;

		FBodyNodeGeometry(int32 InType, EAliasObjectReference InReference, const TSharedPtr<FBodyNode>& InBodyNode)
		{
			Type = InType;
			Reference = InReference;
			BodyNode = InBodyNode;
		}
	};

	namespace OpenModelUtils
	{
		/** Following layer hierarchy, get list of layers an actor would be in as a csv string*/
		bool GetCsvLayerString(const TAlObjectPtr<AlLayer>& Layer, FString& CsvString);

		bool ActorHasContent(const TSharedPtr<IDatasmithActorElement>& ActorElement);

		bool IsValidActor(const TSharedPtr<IDatasmithActorElement>& ActorElement);

		inline FString UuidToString(const uint32& Uuid)
		{
			return FString::Printf(TEXT("0x%08x"), Uuid);
		}

		inline uint32 GetTypeHash(AlPersistentID& GroupNodeId)
		{
			int IdA, IdB, IdC, IdD;
			GroupNodeId.id(IdA, IdB, IdC, IdD);
			return HashCombine(IdA, HashCombine(IdB, HashCombine(IdC, IdD)));
		}

		inline uint32 GetAlDagNodeUuid(AlDagNode& DagNode)
		{
			if (DagNode.hasPersistentID() == sSuccess)
			{
				AlPersistentID* PersistentID;
				DagNode.persistentID(PersistentID);
				return GetTypeHash(*PersistentID);
			}
			FString Label(StringCast<TCHAR>(DagNode.name()).Get());
			return GetTypeHash(Label);
		}

		bool TransferAlMeshToMeshDescription(const AlMesh& Mesh, const TCHAR* SlotMaterialName, FMeshDescription& MeshDescription, CADLibrary::FMeshParameters& SymmetricParameters, const bool bMerge = false);

		FAlDagNodePtr TesselateDagLeaf(const AlDagNode& DagLeaf, ETesselatorType TessType, double Tolerance);

		CADLibrary::FMeshParameters GetMeshParameters(const TAlObjectPtr<AlLayer>& Layer);
	}

}

#endif


