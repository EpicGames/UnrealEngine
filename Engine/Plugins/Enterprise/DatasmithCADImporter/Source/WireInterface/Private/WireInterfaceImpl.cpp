// Copyright Epic Games, Inc. All Rights Reserved.

#include "WireInterfaceImpl.h"

#include "Modules/ModuleManager.h"

#ifdef USE_OPENMODEL
#include "CADOptions.h"
#include "Containers/List.h"
#include "DatasmithImportOptions.h"
#include "DatasmithPayload.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "GenericPlatform/GenericPlatformTLS.h"
#include "HAL/ConsoleManager.h"
#include "IDatasmithSceneElements.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Utility/DatasmithMeshHelper.h"

#include "StaticMeshDescription.h"
#include "StaticMeshOperations.h"

#include "AliasModelToCADKernelConverter.h"
#include "AliasModelToTechSoftConverter.h" // requires Techsoft as public dependency
#include "CADInterfacesModule.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <AlChannel.h>
#include <AlDagNode.h>
#include <AlGroupNode.h>
#include <AlLayer.h>
#include <AlLinkItem.h>
#include <AlList.h>
#include <AlMesh.h>
#include <AlMeshNode.h>
#include <AlPersistentID.h>
#include <AlRetrieveOptions.h>
#include <AlSet.h>
#include <AlSetMember.h>
#include <AlShader.h>
#include <AlShadingFieldItem.h>
#include <AlShell.h>
#include <AlShellNode.h>
#include <AlSurface.h>
#include <AlSurfaceNode.h>
#include <AlTesselate.h>
#include <AlTrimRegion.h>
#include <AlTM.h>
#include <AlUniverse.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define LOCTEXT_NAMESPACE "WireInterface"

#define WRONG_VERSION_TEXT "Unsupported version of Alias detected. Please upgrade to Alias 2021.3 (or later version)."

#define TRACK_MESHELEMENT 0
#define MAKE_VISIBLE 0

#if MAKE_VISIBLE
#include "CompGeom/FitOrientedBox3.h"
#endif

namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{
#if TRACK_MESHELEMENT
	//static const FString ToTrack(TEXT("shell_30181"));
	//static const FString ToTrack(TEXT("Blend_srf_1395"));
	//static const FString ToTrack(TEXT("M0047942"));
	//static const FString ToTrack(TEXT("C5_31495_surf")); // G6
	static const FString ToTrack(TEXT("Object__TRANSFORM_copy_2_surf")); // refrigerator

	void MakeMeshVisible(FMeshDescription& MeshDescription);
#endif

#if WIRE_MEMORY_CHECK
	int32 AllocatedObjects = 0;
	int32 MaxAllocatedObjects = 0;
	TSet<AlDagNode*> DagNodeSet;
	TSet<AlObject*> ObjectSet;
#endif

	static const FColor DefaultColor = FColor(200, 200, 200);

	const uint64 LibAliasNext_Version = 0xffffffffffffffffull;
	const uint64 LibAlias2027_0_0_Version = 9288674231451972;
	const uint64 LibAlias2026_0_0_Version = 9007199254741154;
	const uint64 LibAlias2025_0_0_Version = 8725724278030572;
	const uint64 LibAlias2024_1_0_Version = 8444253596292024;
	const uint64 LibAlias2023_1_0_Version = 8162778619576619;
	const uint64 LibAlias2023_0_0_Version = 8162774324609149;
	const uint64 LibAlias2022_2_0_Version = 7881307937833405;
	const uint64 LibAlias2022_1_0_Version = 7881303642865885;
	const uint64 LibAlias2022_0_1_Version = 7881299347964005;
	const uint64 LibAlias2021_3_2_Version = 7599833027117059;
	const uint64 LibAlias2021_3_1_Version = 7599824433840131;
	const uint64 LibAlias2021_3_0_Version = 7599824424206339;
	const uint64 LibAlias2021_Version = 7599824377020416;
	const uint64 LibAlias2020_Version = 7318349414924288;
	const uint64 LibAlias2019_Version = 5000000000000000;

#if defined(OPEN_MODEL_2020)
	const uint64 LibAliasVersionMin = LibAlias2019_Version;
	const uint64 LibAliasVersionMax = LibAlias2021_3_0_Version;
	const FString AliasSdkVersion   = TEXT("2020");
#elif defined(OPEN_MODEL_2021_3)
	const uint64 LibAliasVersionMin = LibAlias2021_3_0_Version;
	const uint64 LibAliasVersionMax = LibAlias2022_0_1_Version;
	const FString AliasSdkVersion   = TEXT("2021.3");
#elif defined(OPEN_MODEL_2022)
	const uint64 LibAliasVersionMin = LibAlias2022_0_1_Version;
	const uint64 LibAliasVersionMax = LibAlias2022_1_0_Version;
	const FString AliasSdkVersion   = TEXT("2022");
#elif defined(OPEN_MODEL_2022_1)
	const uint64 LibAliasVersionMin = LibAlias2022_1_0_Version;
	const uint64 LibAliasVersionMax = LibAlias2022_2_0_Version;
	const FString AliasSdkVersion   = TEXT("2022.1");
#elif defined(OPEN_MODEL_2022_2)
	const uint64 LibAliasVersionMin = LibAlias2022_2_0_Version;
	const uint64 LibAliasVersionMax = LibAlias2023_0_0_Version;
	const FString AliasSdkVersion   = TEXT("2022.2");
#elif defined(OPEN_MODEL_2023_0)
	const uint64 LibAliasVersionMin = LibAlias2023_0_0_Version;
	const uint64 LibAliasVersionMax = LibAlias2023_1_0_Version;
	const FString AliasSdkVersion   = TEXT("2023.0");
#elif defined(OPEN_MODEL_2023_1)      
	const uint64 LibAliasVersionMin = LibAlias2023_1_0_Version;
	const uint64 LibAliasVersionMax = LibAlias2024_1_0_Version;
	const FString AliasSdkVersion = TEXT("2023.1");
#elif defined(OPEN_MODEL_2024_1)      
	const uint64 LibAliasVersionMin = LibAlias2024_1_0_Version;
	const uint64 LibAliasVersionMax = LibAlias2025_0_0_Version;
	const FString AliasSdkVersion = TEXT("2024.1");
#elif defined(OPEN_MODEL_2025_0)      
	const uint64 LibAliasVersionMin = LibAlias2025_0_0_Version;
	const uint64 LibAliasVersionMax = LibAlias2026_0_0_Version;
	const FString AliasSdkVersion = TEXT("2025.0");
#elif defined(OPEN_MODEL_2026_0)      
	const uint64 LibAliasVersionMin = LibAlias2026_0_0_Version;
	const uint64 LibAliasVersionMax = LibAlias2027_0_0_Version;
	const FString AliasSdkVersion = TEXT("2026.0");
#endif

	// Alias material management (to allow sew of BReps of different materials):
	// To be compatible with "Retessellate" function, Alias material management as to be the same than CAD (TechSoft) import. 
	// As a reminder: the name and slot of UE Material from CAD is based on CAD material/color data i.e. RGBA Color components => "UE Material slot" (int32) and "UE Material name" (FString = FString::FromInt("UE Material slot"))
	// UE Material Label is free.
	// 
	// During Retessellate step, Color/Material of each CAD Faces are known, so "UE Material slot" can be deduced.
	// 
	// For Alias import:
	// Alias BRep is exported in CAD modeler (CADKernel, TechSoft, ...)
	// Material is build in UE, 
	// From an Alias Material, a unique Color is generated.
	// This Color is associated to the BRep Shell/face in the CAD modeler
	// The name and slot of the associated UE Material is defined from this color
	// So at the retessellate step, nothing changes from the CAD Retessellate process
	// 
	// The unique Color of an Alias Material is defined as follows:
	// TypeHash(Alias Material Name) => uint24 == 3 uint8 => RGB components of the color  

	FColor CreateShaderColorFromShaderName(const FString& ShaderName)
	{
		const uint32 ShaderHash = FCrc::Strihash_DEPRECATED(*ShaderName);
		const uint32 Red = (ShaderHash & 0xff000000) >> 24;
		const uint32 Green = (ShaderHash & 0x00ff0000) >> 16;
		const uint32 Blue = (ShaderHash & 0x0000ff00) >> 8;
		return FColor(Red, Green, Blue);
	}

	int32 CreateShaderId(const FColor& ShaderColor)
	{
		return FMath::Abs((int32)GetTypeHash(ShaderColor));
	}

	uint32 GetSceneFileHash(const FString& FullPath, const FString& FileName)
	{
		FFileStatData FileStatData = IFileManager::Get().GetStatData(*FullPath);

		int64 FileSize = FileStatData.FileSize;
		FDateTime ModificationTime = FileStatData.ModificationTime;

		uint32 FileHash = GetTypeHash(FileName);
		FileHash = HashCombine(FileHash, GetTypeHash(FileSize));
		FileHash = HashCombine(FileHash, GetTypeHash(ModificationTime));

		return FileHash;
	}

	bool GetConsoleBoolValue(const TCHAR* CVarName, bool bDefault)
	{
		const IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(CVarName);
		return ConsoleVariable ? ConsoleVariable->GetBool() : bDefault;
	}

	bool FWireTranslatorImpl::Initialize(const TCHAR* InSceneFullName)
	{
		if (InSceneFullName)
		{
			statusCode Status = AlUniverse::initialize();
			if (Status != sSuccess && Status != sAlreadyCreated)
			{
				UE_LOGF(LogWireInterface, Error, "Cannot initialize OpenModel SDK. Import is aborted.");
				return false;
			}

			const TStringConversion Result = StringCast<ANSICHAR>(InSceneFullName);
			char OpenModelVersion[10];
			if (bool(AlUniverse::isWireFile(Result.Get(), OpenModelVersion)))
			{
				SceneFullPath = InSceneFullName;
				SceneVersion = StringCast<TCHAR>(OpenModelVersion).Get();
			}
			else
			{
				UE_LOGF(LogWireInterface, Error, "Cannot load %ls with the selected OpenModel SDK.", InSceneFullName);
				return false;
			}
		}

		return true;
	}

	FWireTranslatorImpl::~FWireTranslatorImpl()
	{
		if (bSceneLoaded)
		{
			FLayerContainer::Reset();
			MeshElementToParametricNode.Reset();
			MeshElementToMeshNode.Reset();
			MeshElementToBodyNode.Reset();
			MeshElementToPatchMesh.Reset();
			EncounteredNodes.Reset();
#if WIRE_MEMORY_CHECK
			if (!ensure(AllocatedObjects == 0))
			{
				for (AlObject* Object : ObjectSet)
				{
					if (AlIsValid(Object))
					{
						AlObjectType Type = Object->type();
						printf(">>> %d\n", Type);
					}
					else
					{
						ensure(false);
					}
				}
			}
#endif
		}
		bSceneLoaded = false;
	}

	void FWireTranslatorImpl::SetImportSettings(const FWireSettings& Settings)
	{
		WireSettings = Settings;
		WireSettings.bUseCADKernel = !CADLibrary::FImportParameters::bGDisableCADKernelTessellation;
		if (CADModelConverter)
		{
			CADModelConverter->SetImportParameters(Settings.ChordTolerance, Settings.MaxEdgeLength, Settings.NormalTolerance, (CADLibrary::EStitchingTechnique)Settings.StitchingTechnique);
		}
	}

	// Wire file parsing
	bool FWireTranslatorImpl::Load(TSharedPtr<IDatasmithScene> InScene)
	{
		UE_LOGF(LogWireInterface, Display, "CAD translation [%ls].", *SceneFullPath);
		UE_LOGF(LogWireInterface, Display, " - File version:         Alias %ls", *SceneVersion);
		UE_LOGF(LogWireInterface, Display, " - Parsing Library:      Alias %ls", *AliasSdkVersion);
		UE_LOGF(LogWireInterface, Display, " - Tessellation Library: %ls", WireSettings.bUseCADKernel ? TEXT("CADKernel") : TEXT("TechSoft"));

		UE_LOGF(LogWireInterface, Display, " - Translation parameters:");
		UE_LOGF(LogWireInterface, Display, "     - Merge By Group:      %ls", WireSettings.bMergeGeometryByGroup ? TEXT("True") : TEXT("False"));
		UE_LOGF(LogWireInterface, Display, "     - Layer As actor:      %ls", WireSettings.bUseLayerAsActor ? TEXT("True") : TEXT("False"));

		UE_LOGF(LogWireInterface, Display, " - Tessellation parameters:");
		UE_LOGF(LogWireInterface, Display, "     - ChordTolerance:      %lf", WireSettings.ChordTolerance);
		UE_LOGF(LogWireInterface, Display, "     - MaxEdgeLength:       %lf", WireSettings.MaxEdgeLength);
		UE_LOGF(LogWireInterface, Display, "     - MaxNormalAngle:      %lf", WireSettings.NormalTolerance);

		FString StitchingTechnique;
		switch (WireSettings.StitchingTechnique)
		{
		case EDatasmithCADStitchingTechnique::StitchingHeal:
			StitchingTechnique = TEXT("Heal");
			break;
		case EDatasmithCADStitchingTechnique::StitchingSew:
			StitchingTechnique = TEXT("Sew");
			break;
		default:
			StitchingTechnique = TEXT("None");
			break;
		}
		UE_LOGF(LogWireInterface, Display, "     - StitchingTechnique:  %ls", *StitchingTechnique);
		UE_LOGF(LogWireInterface, Display, "     - GeometricTolerance:  %lf", WireSettings.GetGeometricTolerance());
		UE_LOGF(LogWireInterface, Display, "     - Stitching Tolerance: %lf", WireSettings.GetStitchingTolerance());

		// #wire_import: TODO - Revisit stitching extension when using CADKernel
		//if (WireSettings.bUseCADKernel)
		//{
		//	UE_LOGF(LogWireInterface, Display, "     - Stitching Options:");
		//	UE_LOGF(LogWireInterface, Display, "         - ForceSew:              %ls", CADLibrary::FImportParameters::bGStitchingForceSew ? TEXT("True") : TEXT("False"));
		//	UE_LOGF(LogWireInterface, Display, "         - RemoveThinFaces:       %ls", CADLibrary::FImportParameters::bGStitchingRemoveThinFaces ? TEXT("True") : TEXT("False"));
		//	UE_LOGF(LogWireInterface, Display, "         - RemoveDuplicatedFaces: %ls", CADLibrary::FImportParameters::bGStitchingRemoveDuplicatedFaces ? TEXT("True") : TEXT("False"));
		//	UE_LOGF(LogWireInterface, Display, "         - ForceFactor:           %f", CADLibrary::FImportParameters::GStitchingForceFactor);
		//}

		DatasmithScene = InScene;

		const FString AliasProductVersion = FString::Printf(TEXT("Alias %s"), *AliasSdkVersion);
		DatasmithScene->SetHost(TEXT("Alias"));
		DatasmithScene->SetVendor(TEXT("Autodesk"));
		DatasmithScene->SetProductName(TEXT("Alias Tools"));
		DatasmithScene->SetExporterSDKVersion(*AliasSdkVersion);
		DatasmithScene->SetProductVersion(*AliasProductVersion);

		WireSettings.bAliasUseNative = GetConsoleBoolValue(TEXT("ds.Wiretranslator.UseNative"), false);
		if (!WireSettings.bAliasUseNative)
		{
			CADLibrary::FImportParameters ImportParameters;
			if (CADLibrary::FImportParameters::bGDisableCADKernelTessellation)
			{
				CADModelConverter = MakeShared<FAliasModelToTechSoftConverter>(ImportParameters);
			}
			else
			{
				CADModelConverter = MakeShared<FAliasModelToCADKernelConverter>(WireSettings, ImportParameters);
			}
		}
		else
		{
			// Merge by group when using Alias' tessellator
			WireSettings.bMergeGeometryByGroup = false;
		}

		// Initialize Alias.
		statusCode Status = AlUniverse::initialize();
		if (Status != sSuccess && Status != sAlreadyCreated)
		{
			UE_LOGF(LogWireInterface, Error, "Cannot initialize OpenModel SDK. Import is aborted.");
			return false;
		}

		if (AlUniverse::retrieve(TCHAR_TO_UTF8(*SceneFullPath)) != sSuccess)
		{
			return false;
		}

		bSceneLoaded = true;

		return TraverseModel();
	}

	bool FWireTranslatorImpl::TraverseModel()
	{
#if WIRE_MEMORY_CHECK
		AllocatedObjects = 0;
		MaxAllocatedObjects = 0;
		ObjectSet.Reset();
		DagNodeSet.Reset();
#endif
		FAlDagNodePtr DagNode = FindOrAddDagNode(AlUniverse::firstDagNode());
		while (DagNode)
		{
			TSharedPtr<IDatasmithActorElement> ActorElement = TraverseDag(DagNode);
			if (ActorElement)
			{
				DatasmithScene->AddActor(ActorElement);
			}

			DagNode = FindOrAddDagNode(DagNode->nextNode());
		}

		TAlObjectPtr<AlSet> Set(AlUniverse::firstSet());
		statusCode Status = Set ? sSuccess : sFailure;
		while (Status == sSuccess)
		{
			// #wire_import: Add an actor to represent the set.
			TAlObjectPtr<AlSetMember> SetMember(Set->firstMember());
			statusCode MemberStatus = SetMember ? sSuccess : sFailure;
			while (MemberStatus == sSuccess)
			{
				FAlDagNodePtr DagNodeInSet(SetMember->object() ? SetMember->object()->asDagNodePtr() : nullptr);
				if (DagNodeInSet)
				{
					TSharedPtr<IDatasmithActorElement> ActorElement = TraverseDag(DagNodeInSet);
					if (ActorElement)
					{
						DatasmithScene->AddActor(ActorElement);
					}
				}

				MemberStatus = SetMember->nextSetMemberD();
			}

			Status = Set->nextSetD();
		}

		return true;
	}

	TSharedPtr<IDatasmithActorElement> FWireTranslatorImpl::TraverseDag(const FAlDagNodePtr& RootNode)
	{
		if (RootNode.IsAGroup())
		{
			return WireSettings.bMergeGeometryByGroup ? ProcessGroupNode(RootNode) : TraverseGroupNode(RootNode);
		}
		else if (RootNode.HasGeometry())
		{
			return ProcessGeometryNode(RootNode);
		}

		return TSharedPtr<IDatasmithActorElement>();
	}

	TSharedPtr<IDatasmithActorElement> FWireTranslatorImpl::TraverseGroupNode(const FAlDagNodePtr& GroupNode, const TAlObjectPtr<AlLayer>& ParentLayer)
	{
		if (!GroupNode.IsAGroup())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TArray<FAlDagNodePtr> Children;
		
		FAlDagNodePtr ChildNode = FindOrAddDagNode(GroupNode->asGroupNodePtr()->childNode());
		while (ChildNode)
		{
			Children.Add(ChildNode);
			ChildNode = FindOrAddDagNode(ChildNode->nextNode());
		}

		const int32 ChildrenCount = Children.Num();
		if (ChildrenCount == 0)
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TArray<TSharedPtr<IDatasmithActorElement>> ChildActors;
		ChildActors.Reserve(ChildrenCount);

		TAlObjectPtr<AlLayer> GroupLayer = FLayerContainer::FindOrAdd(GroupNode->layer());
		for (FAlDagNodePtr& Child : Children)
		{
			if (TSharedPtr<IDatasmithActorElement> ActorElement = TraverseGroupNode(Child, GroupLayer))
			{
				ChildActors.Emplace(MoveTemp(ActorElement));
			}
			else if (Child.IsAMesh() || Child.IsASurface())
			{
				if (TSharedPtr<IDatasmithActorElement> ChildActorElement = ProcessGeometryNode(Child, ParentLayer))
				{
					ChildActors.Emplace(MoveTemp(ChildActorElement));
				}
			}
		}

		if (ChildActors.Num() == 0)
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TSharedPtr<IDatasmithActorElement> ActorElement = FDatasmithSceneFactory::CreateActor(*GroupNode.GetUniqueID(GROUPNODE_TYPE));
		if (!ActorElement.IsValid())
		{
			return ActorElement;
		}

		FString Label = GroupNode.GetName();
		ActorElement->SetLabel(Label.Len() > 0 ? *Label : TEXT("UnnamedGroup"));

		FString CsvLayerString;
		if (OpenModelUtils::GetCsvLayerString(GroupLayer, CsvLayerString))
		{
			ActorElement->SetLayer(*CsvLayerString);
		}

		GroupNode.SetActorTransform(*ActorElement);

		for (TSharedPtr<IDatasmithActorElement>& ChildActor : ChildActors)
		{
			if (OpenModelUtils::ActorHasContent(ChildActor))
			{
				ActorElement->AddChild(ChildActor);
			}
		}

		if (WireSettings.bUseLayerAsActor && GroupLayer != ParentLayer)
		{
			if (OpenModelUtils::ActorHasContent(ActorElement))
			{
				if (TSharedPtr<IDatasmithActorElement> LayerActor = FindOrAddLayerActor(GroupLayer))
				{
					LayerActor->AddChild(ActorElement);
				}
			}

			return TSharedPtr<IDatasmithActorElement>();
		}

		return ActorElement;
	}

	TSharedPtr<IDatasmithActorElement> FWireTranslatorImpl::ProcessGroupNode(const FAlDagNodePtr& GroupNode, const TAlObjectPtr<AlLayer>& ParentLayer)
	{
		if (!GroupNode.IsAGroup())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TArray<FAlDagNodePtr> Children;
		FAlDagNodePtr ChildNode = FindOrAddDagNode(GroupNode->asGroupNodePtr()->childNode());
		while (ChildNode)
		{
			Children.Add(ChildNode);
			AlDagNode* NextNode = ChildNode->nextNode();
#if WIRE_MEMORY_CHECK
			ensure(NextNode != ChildNode.Get());
#endif
			ChildNode = FindOrAddDagNode(NextNode);
		}

		const int32 ChildrenCount = Children.Num();
		if (ChildrenCount == 0)
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TArray<TSharedPtr<IDatasmithActorElement>> ChildActors;
		ChildActors.Reserve(ChildrenCount);
		
		TAlObjectPtr<AlLayer> GroupLayer = FLayerContainer::FindOrAdd(GroupNode->layer());

		TSharedPtr<FBodyNode> BodyNode = MakeShared<FBodyNode>(GroupNode.GetName() + TEXT("_surf"), GroupLayer, ChildrenCount);
		TSharedPtr<FPatchMesh> PatchMesh = MakeShared<FPatchMesh>(GroupNode.GetName() + TEXT("_mesh"), GroupLayer, ChildrenCount);

		for(FAlDagNodePtr& Child : Children)
		{
			if (TSharedPtr<IDatasmithActorElement> ActorElement = ProcessGroupNode(Child, GroupLayer))
			{
				ChildActors.Emplace(MoveTemp(ActorElement));
			}
			else if (Child.IsAMesh())
			{
				PatchMesh->AddMeshNode(Child);
			}
			else if (Child.IsASurface() || Child.IsAShell())
			{
				BodyNode->AddNode(Child);
			}
		}

		if (ChildActors.IsEmpty() && !BodyNode->Initialize() && !PatchMesh->Initialize())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		if (TSharedPtr<IDatasmithActorElement> ActorElement = ProcessBodyNode(BodyNode, GroupNode, GroupLayer))
		{
			if (OpenModelUtils::ActorHasContent(ActorElement))
			{
				ChildActors.Emplace(MoveTemp(ActorElement));
			}
		}

		if (TSharedPtr<IDatasmithActorElement> ActorElement = ProcessPatchMesh(PatchMesh, GroupNode, GroupLayer))
		{
			if (OpenModelUtils::ActorHasContent(ActorElement))
			{
				ChildActors.Emplace(MoveTemp(ActorElement));
			}
		}

		if (ChildActors.Num() == 0)
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TSharedPtr<IDatasmithActorElement> ActorElement;
		if (WireSettings.bMergeGeometryByGroup && ChildActors.Num() == 1)
		{
			ActorElement = ChildActors[0];
		}
		else
		{
			ActorElement = FDatasmithSceneFactory::CreateActor(*GroupNode.GetUniqueID(GROUPNODE_TYPE));
		}

		if (!ActorElement.IsValid())
		{
			return ActorElement;
		}

		FString Label = GroupNode.GetName();
		ActorElement->SetLabel(Label.Len() > 0 ? *Label : TEXT("UnnamedGroup"));

		FString CsvLayerString;
		if (OpenModelUtils::GetCsvLayerString(GroupLayer, CsvLayerString))
		{
			ActorElement->SetLayer(*CsvLayerString);
		}

		if (WireSettings.bMergeGeometryByGroup && ChildActors.Num() > 1)
		{
			GroupNode.SetActorTransform(*ActorElement);

			for (TSharedPtr<IDatasmithActorElement>& ChildActor : ChildActors)
			{
				if (OpenModelUtils::ActorHasContent(ChildActor))
				{
					ActorElement->AddChild(ChildActor);
				}
			}

		}

		if (WireSettings.bUseLayerAsActor && GroupLayer != ParentLayer)
		{
			if (OpenModelUtils::ActorHasContent(ActorElement))
			{
				if (TSharedPtr<IDatasmithActorElement> LayerActor = FindOrAddLayerActor(GroupLayer))
				{
					LayerActor->AddChild(ActorElement);
				}
			}

			return TSharedPtr<IDatasmithActorElement>();
		}

		return ActorElement;
	}

	TSharedPtr<IDatasmithActorElement> FWireTranslatorImpl::ProcessGeometryNode(const FAlDagNodePtr& GeomNode, const TAlObjectPtr<AlLayer>& ParentLayer)
	{
		const TAlObjectPtr<AlLayer>& Layer = GeomNode.GetLayer();
		if (Layer && Layer->invisible())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TSharedPtr<IDatasmithMeshElement> MeshElement = FindOrAddMeshElement(GeomNode);
		if (!MeshElement.IsValid())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TSharedPtr<IDatasmithMeshActorElement> ActorElement = FDatasmithSceneFactory::CreateMeshActor(*GeomNode.GetUniqueID(MESHNODE_TYPE));
		if (!ActorElement.IsValid())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		FString Label = GeomNode.GetName();
		ActorElement->SetLabel(Label.Len() > 0 ? *Label : TEXT("NoName"));
		ActorElement->SetStaticMeshPathName(MeshElement->GetName());

		FString CsvLayerString;
		if (OpenModelUtils::GetCsvLayerString(Layer, CsvLayerString))
		{
			ActorElement->SetLayer(*CsvLayerString);
		}

		GeomNode.SetActorTransform(*ActorElement);

		if (WireSettings.bUseLayerAsActor && Layer != ParentLayer)
		{
			if (TSharedPtr<IDatasmithActorElement> LayerActor = FindOrAddLayerActor(Layer))
			{
				LayerActor->AddChild(ActorElement);
				return TSharedPtr<IDatasmithActorElement>();
			}

			ensureWire(false);
		}

		return ActorElement;
	}

	TSharedPtr<IDatasmithActorElement> FWireTranslatorImpl::ProcessBodyNode(TSharedPtr<FBodyNode>& BodyNode, const FAlDagNodePtr& GroupNode, const TAlObjectPtr<AlLayer>& ParentLayer)
	{
		if (!BodyNode->HasContent())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		FAlDagNodePtr Singleton;
		if (BodyNode->GetSingleContent(Singleton))
		{
			return ProcessGeometryNode(Singleton, ParentLayer);
		}

		TSharedPtr<IDatasmithMeshElement> MeshElement = FindOrAddMeshElement(BodyNode);
		if (!MeshElement.IsValid())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TSharedPtr<IDatasmithMeshActorElement> ActorElement = FDatasmithSceneFactory::CreateMeshActor(*BodyNode->GetUniqueID());
		if (!ActorElement.IsValid())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		FString Label = BodyNode->GetName();
		ActorElement->SetLabel(Label.Len() > 0 ? *Label : TEXT("NoName"));
		ActorElement->SetStaticMeshPathName(MeshElement->GetName());

		FString CsvLayerString;
		if (OpenModelUtils::GetCsvLayerString(BodyNode->GetLayer(), CsvLayerString))
		{
			ActorElement->SetLayer(*CsvLayerString);
		}

		if (BodyNode->GetLayer() && !BodyNode->GetLayer()->isSymmetric())
		{
			GroupNode.SetActorTransform(*ActorElement);
		}

		if (WireSettings.bUseLayerAsActor && BodyNode->GetLayer() != ParentLayer)
		{
			if (TSharedPtr<IDatasmithActorElement> LayerActor = FindOrAddLayerActor(BodyNode->GetLayer()))
			{
				LayerActor->AddChild(ActorElement);
				return TSharedPtr<IDatasmithActorElement>();
			}

			ensureWire(false);
		}

		return ActorElement;
	}

	TSharedPtr<IDatasmithActorElement> FWireTranslatorImpl::ProcessPatchMesh(TSharedPtr<FPatchMesh>& PatchMesh, const FAlDagNodePtr& GroupNode, const TAlObjectPtr<AlLayer>& ParentLayer)
	{
		if (!PatchMesh->HasContent())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		if (!GroupNode.IsVisible())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		FAlDagNodePtr Singleton;
		if (PatchMesh->GetSingleContent(Singleton))
		{
			return ProcessGeometryNode(Singleton, ParentLayer);
		}

		TSharedPtr<IDatasmithMeshElement> MeshElement = FindOrAddMeshElement(PatchMesh);
		if (!MeshElement.IsValid())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TSharedPtr<IDatasmithMeshActorElement> ActorElement = FDatasmithSceneFactory::CreateMeshActor(*PatchMesh->GetUniqueID());
		if (!ActorElement.IsValid())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		FString Label = PatchMesh->GetName();
		ActorElement->SetLabel(Label.Len() > 0 ? *Label : TEXT("NoName"));
		ActorElement->SetStaticMeshPathName(MeshElement->GetName());

		FString CsvLayerString;
		if (OpenModelUtils::GetCsvLayerString(PatchMesh->GetLayer(), CsvLayerString))
		{
			ActorElement->SetLayer(*CsvLayerString);
		}

		GroupNode.SetActorTransform(*ActorElement);

		if (WireSettings.bUseLayerAsActor && PatchMesh->GetLayer() != ParentLayer)
		{
			if (TSharedPtr<IDatasmithActorElement> LayerActor = FindOrAddLayerActor(PatchMesh->GetLayer()))
			{
				LayerActor->AddChild(ActorElement);
				return TSharedPtr<IDatasmithActorElement>();
			}

			ensureWire(false);
		}

		return ActorElement;
	}

	TSharedPtr<IDatasmithMeshElement> FWireTranslatorImpl::FindOrAddMeshElement(const FAlDagNodePtr& GeomNode)
	{
		// Look if geometry has not been already processed, return it if found
		if (TSharedPtr<IDatasmithMeshElement>* MeshElementPtr = GeomNodeToMeshElement.Find(GeomNode.GetHash()))
		{
			return *MeshElementPtr;
		}

		if (!GeomNode.HasGeometry())
		{
			// #wire_import: Log an error
			return TSharedPtr<IDatasmithMeshElement>();
		}

		TSharedPtr<IDatasmithMeshElement> MeshElement = FDatasmithSceneFactory::CreateMesh(*GeomNode.GetUniqueID(MESH_TYPE));

		MeshElement->SetLabel(*GeomNode.GetName());
		MeshElement->SetLightmapSourceUV(-1);
#if TRACK_MESHELEMENT
		{
			if (!ToTrack.Equals(MeshElement->GetLabel()))
			{
				return TSharedPtr<IDatasmithMeshElement>();
			}
		}
#endif
		auto ApplyMaterial = [this, &MeshElement](const TAlObjectPtr<AlShader>& Shader, int32 SlotIndex)
			{
				if (TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = this->FindOrAddMaterial(Shader))
				{
					MeshElement->SetMaterial(MaterialElement->GetName(), SlotIndex);
				}
			};


		TAlObjectPtr<AlShell> Shell;
		if (GeomNode.GetShell(Shell))
		{
			TAlObjectPtr<AlShader> Shader(Shell->firstShader());
			int32 SlotIndex = 0;
			while (Shader)
			{
				ApplyMaterial(Shader, SlotIndex++);
				Shader = Shell->nextShader(Shader.Get());
			}
			// #wire_import: Check There are as many shaders as trim regions
			MeshElementToParametricNode.Add(MeshElement, GeomNode);
		}
		else
		{
			TAlObjectPtr<AlSurface> Surface;
			if (GeomNode.GetSurface(Surface))
			{
				// #wire_import: Check for trim regions
				ApplyMaterial(Surface->firstShader(), 0);
				MeshElementToParametricNode.Add(MeshElement, GeomNode);
			}
			else
			{
				TAlObjectPtr<AlMesh> Mesh;
				if (GeomNode.GetMesh(Mesh))
				{
					ApplyMaterial(Mesh->firstShader(), 0);
					MeshElementToMeshNode.Add(MeshElement, GeomNode);
				}
				else
				{
					// #wire_import: Log an error
					return TSharedPtr<IDatasmithMeshElement>();
				}
			}
		}

		DatasmithScene->AddMesh(MeshElement);
		GeomNodeToMeshElement.Add(GeomNode.GetHash(), MeshElement);

		return MeshElement;
	}

	TSharedPtr<IDatasmithMeshElement> FWireTranslatorImpl::FindOrAddMeshElement(TSharedPtr<FBodyNode>& BodyNode)
	{
		// Look if geometry has not been already processed, return it if found
		if (TSharedPtr<IDatasmithMeshElement>* MeshElementPtr = BodyNodeToMeshElement.Find(BodyNode->GetHash()))
		{
			return *MeshElementPtr;
		}

		if (!BodyNode->HasContent())
		{
			// #wire_import: Log an error
			return TSharedPtr<IDatasmithMeshElement>();
		}

		TSharedPtr<IDatasmithMeshElement> MeshElement = FDatasmithSceneFactory::CreateMesh(*BodyNode->GetUniqueID());

		MeshElement->SetLabel(*BodyNode->GetName());
		MeshElement->SetLightmapSourceUV(-1);
#if TRACK_MESHELEMENT
		{
			if (!ToTrack.Equals(MeshElement->GetLabel()))
			{
				return TSharedPtr<IDatasmithMeshElement>();
			}
		}
#endif

		auto ApplyMaterial = [this, &MeshElement](int32 SlotIndex, const TAlObjectPtr<AlShader>& Shader)
			{
				if (TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = this->FindOrAddMaterial(Shader))
				{
					MeshElement->SetMaterial(MaterialElement->GetName(), SlotIndex);
				}
			};


		BodyNode->IterateOnSlotIndices(ApplyMaterial);

		DatasmithScene->AddMesh(MeshElement);
		BodyNodeToMeshElement.Add(BodyNode->GetHash(), MeshElement);
		MeshElementToBodyNode.Add(MeshElement, BodyNode);

		return MeshElement;
	}

	TSharedPtr<IDatasmithMeshElement> FWireTranslatorImpl::FindOrAddMeshElement(TSharedPtr<FPatchMesh>& PatchMesh)
	{
		// Look if geometry has not been already processed, return it if found
		if (TSharedPtr<IDatasmithMeshElement>* MeshElementPtr = PatchMeshToMeshElement.Find(PatchMesh->GetHash()))
		{
			return *MeshElementPtr;
		}

		if (!PatchMesh->HasContent())
		{
			// #wire_import: Log an error
			return TSharedPtr<IDatasmithMeshElement>();
		}

		TSharedPtr<IDatasmithMeshElement> MeshElement = FDatasmithSceneFactory::CreateMesh(*PatchMesh->GetUniqueID());

		MeshElement->SetLabel(*PatchMesh->GetName());
		MeshElement->SetLightmapSourceUV(-1);
#if TRACK_MESHELEMENT
		{
			if (!ToTrack.Equals(MeshElement->GetLabel()))
			{
				return TSharedPtr<IDatasmithMeshElement>();
			}
		}
#endif

		auto ApplyMaterial = [this, &MeshElement](const TAlObjectPtr<AlShader>& Shader, int32 SlotIndex)
			{
				if (TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = this->FindOrAddMaterial(Shader))
				{
					MeshElement->SetMaterial(MaterialElement->GetName(), SlotIndex);
				}
			};


		int32 SlotIndex = 0;
		PatchMesh->IterateOnMeshNodes([&ApplyMaterial, &SlotIndex](const FAlDagNodePtr& MeshNode)
			{
				TAlObjectPtr<AlMesh> Mesh;
				if (MeshNode.GetMesh(Mesh))
				{
					ApplyMaterial(Mesh->firstShader(), SlotIndex++);
				}
			});

		DatasmithScene->AddMesh(MeshElement);
		BodyNodeToMeshElement.Add(PatchMesh->GetHash(), MeshElement);
		MeshElementToPatchMesh.Add(MeshElement, PatchMesh);

		return MeshElement;
	}

	TSharedPtr<IDatasmithActorElement> FWireTranslatorImpl::FindOrAddLayerActor(const TAlObjectPtr<AlLayer>& Layer)
	{
		if (!WireSettings.bUseLayerAsActor || !Layer || Layer.GetName().IsEmpty())
		{
			return TSharedPtr<IDatasmithActorElement>();
		}

		TSharedPtr<IDatasmithActorElement>* LayerActorPtr = LayerToActor.Find(Layer.GetHash());
		if (LayerActorPtr)
		{
			return *LayerActorPtr;
		}

		TSharedPtr<IDatasmithActorElement> ParentLayerActor;
		TAlObjectPtr<AlLayer> ParentLayer = FLayerContainer::FindOrAdd(Layer->parentLayer());
		if (ParentLayer)
		{
			ParentLayerActor = FindOrAddLayerActor(ParentLayer);
		}

		TSharedPtr<IDatasmithActorElement> LayerActor = FDatasmithSceneFactory::CreateActor(*Layer.GetUniqueID(LAYER_TYPE));

		FString LayerName = Layer.GetName();
		LayerActor->SetLabel(*LayerName);

		FString CsvLayerString;
		if (OpenModelUtils::GetCsvLayerString(Layer, CsvLayerString))
		{
			LayerActor->SetLayer(*CsvLayerString);
		}

		if (ParentLayerActor)
		{
			ParentLayerActor->AddChild(LayerActor);
		}
		else
		{
			DatasmithScene->AddActor(LayerActor);
		}

		LayerToActor.Add(Layer.GetHash(), LayerActor);

		return LayerActor;
	}

	// Geometry retrieval

	bool FWireTranslatorImpl::LoadStaticMesh(const TSharedPtr<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload, const FDatasmithTessellationOptions& InTessellationOptions)
	{
		CADLibrary::FMeshParameters MeshParameters;

#if WIRE_MEMORY_CHECK
		int32 PrevAllocatedObjects = AllocatedObjects;
#endif
		if (TOptional<FMeshDescription> Mesh = GetMeshDescription(MeshElement, MeshParameters))
		{
#if TRACK_MESHELEMENT
			FMeshDescription& MeshDescription = OutMeshPayload.LodMeshes.Add_GetRef(MoveTemp(Mesh.GetValue()));
			MakeMeshVisible(MeshDescription);
#else
			OutMeshPayload.LodMeshes.Add(MoveTemp(Mesh.GetValue()));
#endif
			const TCHAR* MeshFilename = MeshElement->GetFile();
			if (!WireSettings.bAliasUseNative && FPaths::FileExists(MeshFilename))
			{
				CADModelConverter->AddSurfaceDataForMesh(MeshFilename, MeshParameters, InTessellationOptions, OutMeshPayload);

				// Remove the file because it is temporary since caching is disabled.
				if (!CADLibrary::FImportParameters::bGEnableCADCache)
				{
					IFileManager::Get().Delete(MeshFilename);
				}
			}

#if WIRE_MEMORY_CHECK
			ensure(PrevAllocatedObjects == AllocatedObjects);
#endif
			return true;
		}

		return false;
	}

	TOptional<FMeshDescription> FWireTranslatorImpl::GetMeshDescription(TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& OutMeshParameters)
	{
		if (WireSettings.bAliasUseNative)
		{
			if (FAlDagNodePtr* GeomNodePtr = MeshElementToParametricNode.Find(MeshElement))
			{
				FAlDagNodePtr& GeomNode = *GeomNodePtr;

				// #wire_import: Check whether parametric geometry with symmetry keeps the symmetry
				// #wire_import: the best way, should be to don't have to apply inverse global transform to the generated mesh
				FAlDagNodePtr MeshNode = OpenModelUtils::TesselateDagLeaf(*GeomNode, ETesselatorType::Fast, WireSettings.ChordTolerance);

				TAlObjectPtr<AlMesh> Mesh;
				if (MeshNode.GetMesh(Mesh))
				{
					AlMatrix4x4 AlMatrix;
					GeomNode->inverseGlobalTransformationMatrix(AlMatrix);
					Mesh->transform(AlMatrix);

					// Get the meshes from the dag nodes. Note that removing the mesh's DAG.
					// will also removes the meshes, so we have to do it later.
					return GetMeshDescriptionFromMeshNode(MeshNode, MeshElement, OutMeshParameters);
				}
			}
			
			if (FAlDagNodePtr* MeshNodePtr = MeshElementToMeshNode.Find(MeshElement))
			{
				FAlDagNodePtr MeshNode = *MeshNodePtr;
				if (MeshNode)
				{
					return GetMeshDescriptionFromMeshNode(MeshNode, MeshElement, OutMeshParameters);
				}
			}

			return TOptional<FMeshDescription>();
		}

		if (TSharedPtr<FBodyNode>* BodyNodePtr = MeshElementToBodyNode.Find(MeshElement))
		{
			return GetMeshDescriptionFromBodyNode(*BodyNodePtr, MeshElement, OutMeshParameters);
		}

		if (FAlDagNodePtr* GeomNodePtr = MeshElementToParametricNode.Find(MeshElement))
		{
			return GetMeshDescriptionFromParametricNode(*GeomNodePtr, MeshElement, OutMeshParameters);
		}

		if (TSharedPtr<FPatchMesh>* PatchMeshPtr = MeshElementToPatchMesh.Find(MeshElement))
		{
			return GetMeshDescriptionFromPatchMesh(*PatchMeshPtr, MeshElement, OutMeshParameters);
		}

		if (FAlDagNodePtr* MeshNodePtr = MeshElementToMeshNode.Find(MeshElement))
		{
			return GetMeshDescriptionFromMeshNode(*MeshNodePtr, MeshElement, OutMeshParameters);
		}

		return TOptional<FMeshDescription>();
	}

	TOptional<FMeshDescription> FWireTranslatorImpl::GetMeshDescriptionFromBodyNode(TSharedPtr<FBodyNode>& BodyNode, TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& OutMeshParameters)
	{
		OutMeshParameters = OpenModelUtils::GetMeshParameters(BodyNode->GetLayer());

		TSharedPtr<CADLibrary::ICADModelConverter> ModelConverter = GetModelConverter();

		ModelConverter->InitializeProcess();

		EAliasObjectReference ObjectReference = EAliasObjectReference::LocalReference;
		if (OutMeshParameters.bIsSymmetric)
		{
			// All actors of a Alias symmetric layer are defined in the world Reference i.e. they have identity transform. So Mesh actor has to be defined in the world reference.
			ObjectReference = EAliasObjectReference::WorldReference;
		}
		else if (WireSettings.bMergeGeometryByGroup)
		{
			// In the case of StitchingSew, AlDagNode children of a GroupNode are merged together. To be merged, they have to be defined in the reference of parent GroupNode.
			ObjectReference = EAliasObjectReference::ParentReference;
		}

		const FBodyNodeGeometry BodyNodeGeometry( (int32)ECADModelGeometryType::BodyNode, ObjectReference, BodyNode);
		ModelConverter->AddGeometry(BodyNodeGeometry);

		ModelConverter->RepairTopology();

		ModelConverter->SaveModel(*OutputPath, MeshElement);

		FMeshDescription MeshDescription;
		DatasmithMeshHelper::PrepareAttributeForStaticMesh(MeshDescription);

		if (ModelConverter->Tessellate(OutMeshParameters, MeshDescription))
		{
			return TOptional<FMeshDescription>(MoveTemp(MeshDescription));
		}

		const TCHAR* StaticMeshLabel = MeshElement->GetLabel();
		const TCHAR* StaticMeshName = MeshElement->GetName();
		UE_LOGF(LogWireInterface, Warning, "Failed to generate the mesh of \"%ls\" (%ls) StaticMesh.", StaticMeshLabel, StaticMeshName);

		return TOptional<FMeshDescription>();
	}

	TOptional<FMeshDescription> FWireTranslatorImpl::GetMeshDescriptionFromPatchMesh(TSharedPtr<FPatchMesh>& PatchMesh, TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& OutMeshParameters)
	{
		OutMeshParameters = OpenModelUtils::GetMeshParameters(PatchMesh->GetLayer());

		FMeshDescription MeshDescription;
		DatasmithMeshHelper::PrepareAttributeForStaticMesh(MeshDescription);
		MeshDescription.Empty();

		constexpr bool bMerge = true;
		int32 SlotIndex = 0;
		PatchMesh->IterateOnMeshNodes([&](const FAlDagNodePtr& MeshNode)
			{
				TAlObjectPtr<AlMesh> Mesh;
				if (MeshNode.GetMesh(Mesh))
				{
					AlMatrix4x4 AlMatrix;
					if (OutMeshParameters.bIsSymmetric)
					{
						MeshNode->globalTransformationMatrix(AlMatrix);
					}
					else
					{
						MeshNode->localTransformationMatrix(AlMatrix);
					}

					Mesh->transform(AlMatrix);

					const FString SlotMaterialName = DatasmithMeshHelper::DefaultSlotName(SlotIndex++).ToString();

					OpenModelUtils::TransferAlMeshToMeshDescription(*Mesh, *SlotMaterialName, MeshDescription, OutMeshParameters, bMerge);
				}
			});

		// Build edge meta data
		FStaticMeshOperations::DetermineEdgeHardnessesFromVertexInstanceNormals(MeshDescription);

		return MoveTemp(MeshDescription);
	}

	// #wire_import: AlSurfaceNode can have trim regions. This should be handle at this stage
	TOptional<FMeshDescription> FWireTranslatorImpl::GetMeshDescriptionFromParametricNode(const FAlDagNodePtr& DagNode, TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& OutMeshParameters)
	{
		OutMeshParameters = OpenModelUtils::GetMeshParameters(DagNode.GetLayer());

		TSharedPtr<CADLibrary::ICADModelConverter> ModelConverter = GetModelConverter();

		ModelConverter->InitializeProcess();

		EAliasObjectReference ObjectReference = EAliasObjectReference::LocalReference;

		// All geometry are processed in world space if layers are converted to actors
		// All actors of a Alias symmetric layer are defined in the world Reference
		// i.e. they have identity transform. So Mesh actor has to be defined in the world reference.
		if (OutMeshParameters.bIsSymmetric)
		{
			ObjectReference = EAliasObjectReference::WorldReference;
		}

		ensureWire(MeshElement->GetMaterialSlotCount() == 1);

		const FDagNodeGeometry DagNodeGeometry( (int32)ECADModelGeometryType::DagNode, ObjectReference, DagNode );
		if (!ModelConverter->AddGeometry(DagNodeGeometry))
		{
			return TOptional<FMeshDescription>();
		}

		ModelConverter->RepairTopology();

		ModelConverter->SaveModel(*OutputPath, MeshElement);

		FMeshDescription MeshDescription;
		DatasmithMeshHelper::PrepareAttributeForStaticMesh(MeshDescription);

		OutMeshParameters = DagNode.GetMeshParameters();

		if (ModelConverter->Tessellate(OutMeshParameters, MeshDescription))
		{
			return TOptional<FMeshDescription>(MoveTemp(MeshDescription));
		}

		const TCHAR* StaticMeshLabel = MeshElement->GetLabel();
		const TCHAR* StaticMeshName = MeshElement->GetName();
		UE_LOGF(LogWireInterface, Warning, "Failed to generate the mesh of \"%ls\" (%ls) StaticMesh.", StaticMeshLabel, StaticMeshName);

		return TOptional<FMeshDescription>();
	}

	TOptional<FMeshDescription> FWireTranslatorImpl::GetMeshDescriptionFromMeshNode(const FAlDagNodePtr& MeshNode, TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& OutMeshParameters)
	{
		if (!MeshNode)
		{
			return TOptional<FMeshDescription>();
		}

		TAlObjectPtr<AlMesh> Mesh;
		if (!MeshNode.GetMesh(Mesh))
		{
			return TOptional<FMeshDescription>();
		}

		OutMeshParameters = MeshNode.GetMeshParameters();

		if (OutMeshParameters.bIsSymmetric)
		{
			AlMatrix4x4 AlGlobalMatrix;
			MeshNode->globalTransformationMatrix(AlGlobalMatrix);
			Mesh->transform(AlGlobalMatrix);
		}

		FMeshDescription MeshDescription;
		DatasmithMeshHelper::PrepareAttributeForStaticMesh(MeshDescription);

		const bool bMerge = false;
		OpenModelUtils::TransferAlMeshToMeshDescription(*Mesh, TEXT("0"), MeshDescription, OutMeshParameters, bMerge);

		// Build edge meta data
		FStaticMeshOperations::DetermineEdgeHardnessesFromVertexInstanceNormals(MeshDescription);

		return MoveTemp(MeshDescription);
	}

	TSharedPtr<CADLibrary::ICADModelConverter> FWireTranslatorImpl::GetModelConverter() const
	{
		TSharedPtr<CADLibrary::ICADModelConverter> ModelConverter;

		CADLibrary::FImportParameters ImportParameters;
		if (CADLibrary::FImportParameters::bGDisableCADKernelTessellation)
		{
			ModelConverter = MakeShared<FAliasModelToTechSoftConverter>(ImportParameters);
		}
		else
		{
			ModelConverter = MakeShared<FAliasModelToCADKernelConverter>(WireSettings, ImportParameters);
		}

		if (ModelConverter)
		{
			ModelConverter->SetImportParameters(WireSettings.ChordTolerance, WireSettings.MaxEdgeLength, WireSettings.NormalTolerance, (CADLibrary::EStitchingTechnique)WireSettings.StitchingTechnique);
		}

		return ModelConverter;
	}

	// Material creation

	TSharedPtr<IDatasmithBaseMaterialElement> FWireTranslatorImpl::FindOrAddMaterial(const TAlObjectPtr<AlShader>& Shader)
	{
		const FString ShaderName = Shader.GetName();

		if (TSharedPtr<IDatasmithBaseMaterialElement>* MaterialElementPtr = ShaderNameToMaterial.Find(ShaderName))
		{
			return *MaterialElementPtr;
		}

		const FString ShaderModelName = Shader->shadingModel();

		TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement = FDatasmithSceneFactory::CreateUEPbrMaterial(*Shader.GetUniqueID(SHADER_TYPE));
		MaterialElement->SetLabel(*ShaderName);

		if (ShaderModelName.Equals(TEXT("BLINN")))
		{
			AddAlBlinnParameters(Shader, MaterialElement);
		}
		else if (ShaderModelName.Equals(TEXT("LAMBERT")))
		{
			AddAlLambertParameters(Shader, MaterialElement);
		}
		else if (ShaderModelName.Equals(TEXT("LIGHTSOURCE")))
		{
			AddAlLightSourceParameters(Shader, MaterialElement);
		}
		else if (ShaderModelName.Equals(TEXT("PHONG")))
		{
			AddAlPhongParameters(Shader, MaterialElement);
		}

		DatasmithScene->AddMaterial(MaterialElement);
		ShaderNameToMaterial.Add(*ShaderName, MaterialElement);

		return MaterialElement;
	}

	bool FWireTranslatorImpl::GetCommonParameters(int32 Field, double Value, FColor& Color, FColor& TransparencyColor, FColor& IncandescenceColor, double GlowIntensity)
	{
		switch (AlShadingFields(Field))
		{
		case AlShadingFields::kFLD_SHADING_COMMON_COLOR_R:
			Color.R = (uint8)Value;
			return true;
		case AlShadingFields::kFLD_SHADING_COMMON_COLOR_G:
			Color.G = (uint8)Value;
			return true;
		case  AlShadingFields::kFLD_SHADING_COMMON_COLOR_B:
			Color.B = (uint8)Value;
			return true;
		case AlShadingFields::kFLD_SHADING_COMMON_INCANDESCENCE_R:
			IncandescenceColor.R = (uint8)Value;
			return true;
		case AlShadingFields::kFLD_SHADING_COMMON_INCANDESCENCE_G:
			IncandescenceColor.G = (uint8)Value;
			return true;
		case AlShadingFields::kFLD_SHADING_COMMON_INCANDESCENCE_B:
			IncandescenceColor.B = (uint8)Value;
			return true;
		case  AlShadingFields::kFLD_SHADING_COMMON_TRANSPARENCY_R:
			TransparencyColor.R = (uint8)Value;
			return true;
		case  AlShadingFields::kFLD_SHADING_COMMON_TRANSPARENCY_G:
			TransparencyColor.G = (uint8)Value;
			return true;
		case  AlShadingFields::kFLD_SHADING_COMMON_TRANSPARENCY_B:
			TransparencyColor.B = (uint8)Value;
			return true;
		case AlShadingFields::kFLD_SHADING_COMMON_GLOW_INTENSITY:
			GlowIntensity = Value;
			return true;
		default:
			return false;
		}
	}

	void FWireTranslatorImpl::AddAlBlinnParameters(const TAlObjectPtr<AlShader>& Shader, TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement)
	{
		// Default values for a Blinn material
		FColor Color(145, 148, 153);
		FColor TransparencyColor(0, 0, 0);
		FColor IncandescenceColor(0, 0, 0);
		FColor SpecularColor(38, 38, 38);
		double Diffuse = 1.0;
		double GlowIntensity = 0.0;
		double Gloss = 0.8;
		double Eccentricity = 0.35;
		double Specularity = 1.0;
		double Reflectivity = 0.5;
		double SpecularRolloff = 0.5;

		AlList* List = Shader->fields();
		for (AlShadingFieldItem* Item = static_cast<AlShadingFieldItem*>(List->first()); Item; Item = Item->nextField())
		{
			double Value = 0.0f;
			statusCode ErrorCode = Shader->parameter(Item->field(), Value);
			if (ErrorCode != 0)
			{
				continue;
			}

			if (GetCommonParameters(Item->field(), Value, Color, TransparencyColor, IncandescenceColor, GlowIntensity))
			{
				continue;
			}

			switch (Item->field())
			{
			case AlShadingFields::kFLD_SHADING_BLINN_DIFFUSE:
				Diffuse = Value;
				break;
			case AlShadingFields::kFLD_SHADING_BLINN_GLOSS_:
				Gloss = Value;
				break;
			case AlShadingFields::kFLD_SHADING_BLINN_SPECULAR_R:
				SpecularColor.R = (uint8)(255.f * Value);
				break;
			case AlShadingFields::kFLD_SHADING_BLINN_SPECULAR_G:
				SpecularColor.G = (uint8)(255.f * Value);;
				break;
			case AlShadingFields::kFLD_SHADING_BLINN_SPECULAR_B:
				SpecularColor.B = (uint8)(255.f * Value);;
				break;
			case AlShadingFields::kFLD_SHADING_BLINN_SPECULARITY_:
				Specularity = Value;
				break;
			case AlShadingFields::kFLD_SHADING_BLINN_SPECULAR_ROLLOFF:
				SpecularRolloff = Value;
				break;
			case AlShadingFields::kFLD_SHADING_BLINN_ECCENTRICITY:
				Eccentricity = Value;
				break;
			case AlShadingFields::kFLD_SHADING_BLINN_REFLECTIVITY:
				Reflectivity = Value;
				break;
			}
		}

		bool bIsTransparent = IsTransparent(TransparencyColor);

		// Construct parameter expressions
		IDatasmithMaterialExpressionScalar* DiffuseExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseExpression->GetScalar() = Diffuse;
		DiffuseExpression->SetName(TEXT("Diffuse"));

		IDatasmithMaterialExpressionScalar* GlossExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		GlossExpression->GetScalar() = Gloss;
		GlossExpression->SetName(TEXT("Gloss"));

		IDatasmithMaterialExpressionColor* SpecularColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		SpecularColorExpression->SetName(TEXT("SpecularColor"));
		SpecularColorExpression->GetColor() = FLinearColor::FromSRGBColor(SpecularColor);

		IDatasmithMaterialExpressionScalar* SpecularityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		SpecularityExpression->GetScalar() = Specularity * 0.3;
		SpecularityExpression->SetName(TEXT("Specularity"));

		IDatasmithMaterialExpressionScalar* SpecularRolloffExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		SpecularRolloffExpression->GetScalar() = SpecularRolloff;
		SpecularRolloffExpression->SetName(TEXT("SpecularRolloff"));

		IDatasmithMaterialExpressionScalar* EccentricityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		EccentricityExpression->GetScalar() = Eccentricity;
		EccentricityExpression->SetName(TEXT("Eccentricity"));

		IDatasmithMaterialExpressionScalar* ReflectivityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		ReflectivityExpression->GetScalar() = Reflectivity;
		ReflectivityExpression->SetName(TEXT("Reflectivity"));

		IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		ColorExpression->SetName(TEXT("Color"));
		ColorExpression->GetColor() = FLinearColor::FromSRGBColor(Color);

		IDatasmithMaterialExpressionColor* IncandescenceColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		IncandescenceColorExpression->SetName(TEXT("IncandescenceColor"));
		IncandescenceColorExpression->GetColor() = FLinearColor::FromSRGBColor(IncandescenceColor);

		IDatasmithMaterialExpressionColor* TransparencyColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		TransparencyColorExpression->SetName(TEXT("TransparencyColor"));
		TransparencyColorExpression->GetColor() = FLinearColor::FromSRGBColor(TransparencyColor);

		IDatasmithMaterialExpressionScalar* GlowIntensityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		GlowIntensityExpression->GetScalar() = GlowIntensity;
		GlowIntensityExpression->SetName(TEXT("GlowIntensity"));

		// Create aux expressions
		IDatasmithMaterialExpressionGeneric* ColorSpecLerp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		ColorSpecLerp->SetExpressionName(TEXT("LinearInterpolate"));

		IDatasmithMaterialExpressionScalar* ColorSpecLerpValue = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		ColorSpecLerpValue->GetScalar() = 0.96f;

		IDatasmithMaterialExpressionGeneric* ColorMetallicLerp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		ColorMetallicLerp->SetExpressionName(TEXT("LinearInterpolate"));

		IDatasmithMaterialExpressionGeneric* DiffuseLerp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		DiffuseLerp->SetExpressionName(TEXT("LinearInterpolate"));

		IDatasmithMaterialExpressionScalar* DiffuseLerpA = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseLerpA->GetScalar() = 0.04f;

		IDatasmithMaterialExpressionScalar* DiffuseLerpB = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseLerpB->GetScalar() = 1.0f;

		IDatasmithMaterialExpressionGeneric* BaseColorMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* BaseColorAdd = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorAdd->SetExpressionName(TEXT("Add"));

		IDatasmithMaterialExpressionGeneric* BaseColorTransparencyMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorTransparencyMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* IncandescenceMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		IncandescenceMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* IncandescenceScaleMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		IncandescenceScaleMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionScalar* IncandescenceScale = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		IncandescenceScale->GetScalar() = 100.0f;

		IDatasmithMaterialExpressionGeneric* EccentricityMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		EccentricityMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* EccentricityOneMinus = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		EccentricityOneMinus->SetExpressionName(TEXT("OneMinus"));

		IDatasmithMaterialExpressionGeneric* RoughnessOneMinus = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		RoughnessOneMinus->SetExpressionName(TEXT("OneMinus"));

		IDatasmithMaterialExpressionScalar* FresnelExponent = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		FresnelExponent->GetScalar() = 4.0f;

		IDatasmithMaterialExpressionFunctionCall* FresnelFunc = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
		FresnelFunc->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Fresnel_Function.Fresnel_Function"));

		IDatasmithMaterialExpressionGeneric* FresnelLerp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		FresnelLerp->SetExpressionName(TEXT("LinearInterpolate"));

		IDatasmithMaterialExpressionScalar* FresnelLerpA = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		FresnelLerpA->GetScalar() = 1.0f;

		IDatasmithMaterialExpressionScalar* SpecularPowerExp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		SpecularPowerExp->GetScalar() = 0.5f;

		IDatasmithMaterialExpressionGeneric* Power = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		Power->SetExpressionName(TEXT("Power"));

		IDatasmithMaterialExpressionGeneric* FresnelMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		FresnelMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* TransparencyOneMinus = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		TransparencyOneMinus->SetExpressionName(TEXT("OneMinus"));

		IDatasmithMaterialExpressionFunctionCall* BreakFloat3 = nullptr;
		IDatasmithMaterialExpressionGeneric* AddRG = nullptr;
		IDatasmithMaterialExpressionGeneric* AddRGB = nullptr;
		IDatasmithMaterialExpressionGeneric* Divide = nullptr;
		IDatasmithMaterialExpressionScalar* DivideConstant = nullptr;
		if (bIsTransparent)
		{
			BreakFloat3 = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
			BreakFloat3->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakFloat3Components.BreakFloat3Components"));

			AddRG = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AddRG->SetExpressionName(TEXT("Add"));

			AddRGB = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AddRGB->SetExpressionName(TEXT("Add"));

			Divide = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			Divide->SetExpressionName(TEXT("Divide"));

			DivideConstant = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
			DivideConstant->GetScalar() = 3.0f;
		}

		// Connect expressions
		SpecularColorExpression->ConnectExpression(*ColorSpecLerp->GetInput(0));
		ColorExpression->ConnectExpression(*ColorSpecLerp->GetInput(1));
		ColorSpecLerpValue->ConnectExpression(*ColorSpecLerp->GetInput(2));

		ColorExpression->ConnectExpression(*ColorMetallicLerp->GetInput(0));
		ColorSpecLerp->ConnectExpression(*ColorMetallicLerp->GetInput(1));
		GlossExpression->ConnectExpression(*ColorMetallicLerp->GetInput(2));

		DiffuseLerpA->ConnectExpression(*DiffuseLerp->GetInput(0));
		DiffuseLerpB->ConnectExpression(*DiffuseLerp->GetInput(1));
		DiffuseExpression->ConnectExpression(*DiffuseLerp->GetInput(2));

		ColorMetallicLerp->ConnectExpression(*BaseColorMultiply->GetInput(0));
		DiffuseLerp->ConnectExpression(*BaseColorMultiply->GetInput(1));

		BaseColorMultiply->ConnectExpression(*BaseColorAdd->GetInput(0));
		IncandescenceColorExpression->ConnectExpression(*BaseColorAdd->GetInput(1));

		BaseColorAdd->ConnectExpression(*BaseColorTransparencyMultiply->GetInput(0));
		TransparencyOneMinus->ConnectExpression(*BaseColorTransparencyMultiply->GetInput(1));

		GlowIntensityExpression->ConnectExpression(*IncandescenceScaleMultiply->GetInput(0));
		IncandescenceScale->ConnectExpression(*IncandescenceScaleMultiply->GetInput(1));

		BaseColorTransparencyMultiply->ConnectExpression(*IncandescenceMultiply->GetInput(0));
		IncandescenceScaleMultiply->ConnectExpression(*IncandescenceMultiply->GetInput(1));

		EccentricityExpression->ConnectExpression(*EccentricityOneMinus->GetInput(0));

		EccentricityOneMinus->ConnectExpression(*EccentricityMultiply->GetInput(0));
		SpecularityExpression->ConnectExpression(*EccentricityMultiply->GetInput(1));

		EccentricityMultiply->ConnectExpression(*RoughnessOneMinus->GetInput(0));

		FresnelExponent->ConnectExpression(*FresnelFunc->GetInput(3));

		SpecularRolloffExpression->ConnectExpression(*Power->GetInput(0));
		SpecularPowerExp->ConnectExpression(*Power->GetInput(1));

		FresnelLerpA->ConnectExpression(*FresnelLerp->GetInput(0));
		FresnelFunc->ConnectExpression(*FresnelLerp->GetInput(1));
		Power->ConnectExpression(*FresnelLerp->GetInput(2));

		FresnelLerp->ConnectExpression(*FresnelMultiply->GetInput(0));
		ReflectivityExpression->ConnectExpression(*FresnelMultiply->GetInput(1));

		TransparencyColorExpression->ConnectExpression(*TransparencyOneMinus->GetInput(0));

		if (bIsTransparent)
		{
			TransparencyOneMinus->ConnectExpression(*BreakFloat3->GetInput(0));

			BreakFloat3->ConnectExpression(*AddRG->GetInput(0), 0);
			BreakFloat3->ConnectExpression(*AddRG->GetInput(1), 1);

			AddRG->ConnectExpression(*AddRGB->GetInput(0));
			BreakFloat3->ConnectExpression(*AddRGB->GetInput(1), 2);

			AddRGB->ConnectExpression(*Divide->GetInput(0));
			DivideConstant->ConnectExpression(*Divide->GetInput(1));
		}

		// Connect material outputs
		MaterialElement->GetBaseColor().SetExpression(BaseColorTransparencyMultiply);
		MaterialElement->GetMetallic().SetExpression(GlossExpression);
		MaterialElement->GetSpecular().SetExpression(FresnelMultiply);
		MaterialElement->GetRoughness().SetExpression(RoughnessOneMinus);
		MaterialElement->GetEmissiveColor().SetExpression(IncandescenceMultiply);

		if (bIsTransparent)
		{
			MaterialElement->GetOpacity().SetExpression(Divide);
			MaterialElement->SetParentLabel(TEXT("M_DatasmithAliasBlinnTransparent"));
		}
		else
		{
			MaterialElement->SetParentLabel(TEXT("M_DatasmithAliasBlinn"));
		}

	}

	void FWireTranslatorImpl::AddAlLambertParameters(const TAlObjectPtr<AlShader>& Shader, TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement)
	{
		// Default values for a Lambert material
		FColor Color(145, 148, 153);
		FColor TransparencyColor(0, 0, 0);
		FColor IncandescenceColor(0, 0, 0);
		double Diffuse = 1.0;
		double GlowIntensity = 0.0;

		AlList* List = Shader->fields();
		for (AlShadingFieldItem* Item = static_cast<AlShadingFieldItem*>(List->first()); Item; Item = Item->nextField())
		{
			double Value = 0.0f;
			statusCode ErrorCode = Shader->parameter(Item->field(), Value);
			if (ErrorCode != 0)
			{
				continue;
			}

			if (GetCommonParameters(Item->field(), Value, Color, TransparencyColor, IncandescenceColor, GlowIntensity))
			{
				continue;
			}

			switch (Item->field())
			{
			case AlShadingFields::kFLD_SHADING_LAMBERT_DIFFUSE:
				Diffuse = Value;
				break;
			}
		}

		bool bIsTransparent = IsTransparent(TransparencyColor);

		// Construct parameter expressions
		IDatasmithMaterialExpressionScalar* DiffuseExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseExpression->GetScalar() = Diffuse;
		DiffuseExpression->SetName(TEXT("Diffuse"));

		IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		ColorExpression->SetName(TEXT("Color"));
		ColorExpression->GetColor() = FLinearColor::FromSRGBColor(Color);

		IDatasmithMaterialExpressionColor* IncandescenceColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		IncandescenceColorExpression->SetName(TEXT("IncandescenceColor"));
		IncandescenceColorExpression->GetColor() = FLinearColor::FromSRGBColor(IncandescenceColor);

		IDatasmithMaterialExpressionColor* TransparencyColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		TransparencyColorExpression->SetName(TEXT("TransparencyColor"));
		TransparencyColorExpression->GetColor() = FLinearColor::FromSRGBColor(TransparencyColor);

		IDatasmithMaterialExpressionScalar* GlowIntensityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		GlowIntensityExpression->GetScalar() = GlowIntensity;
		GlowIntensityExpression->SetName(TEXT("GlowIntensity"));

		// Create aux expressions
		IDatasmithMaterialExpressionGeneric* DiffuseLerp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		DiffuseLerp->SetExpressionName(TEXT("LinearInterpolate"));

		IDatasmithMaterialExpressionScalar* DiffuseLerpA = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseLerpA->GetScalar() = 0.04f;

		IDatasmithMaterialExpressionScalar* DiffuseLerpB = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseLerpB->GetScalar() = 1.0f;

		IDatasmithMaterialExpressionGeneric* BaseColorMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* BaseColorAdd = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorAdd->SetExpressionName(TEXT("Add"));

		IDatasmithMaterialExpressionGeneric* BaseColorTransparencyMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorTransparencyMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* IncandescenceMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		IncandescenceMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* IncandescenceScaleMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		IncandescenceScaleMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionScalar* IncandescenceScale = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		IncandescenceScale->GetScalar() = 100.0f;

		IDatasmithMaterialExpressionGeneric* TransparencyOneMinus = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		TransparencyOneMinus->SetExpressionName(TEXT("OneMinus"));

		IDatasmithMaterialExpressionFunctionCall* BreakFloat3 = nullptr;
		IDatasmithMaterialExpressionGeneric* AddRG = nullptr;
		IDatasmithMaterialExpressionGeneric* AddRGB = nullptr;
		IDatasmithMaterialExpressionGeneric* Divide = nullptr;
		IDatasmithMaterialExpressionScalar* DivideConstant = nullptr;
		if (bIsTransparent)
		{
			BreakFloat3 = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
			BreakFloat3->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakFloat3Components.BreakFloat3Components"));

			AddRG = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AddRG->SetExpressionName(TEXT("Add"));

			AddRGB = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AddRGB->SetExpressionName(TEXT("Add"));

			Divide = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			Divide->SetExpressionName(TEXT("Divide"));

			DivideConstant = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
			DivideConstant->GetScalar() = 3.0f;
		}

		// Connect expressions
		DiffuseLerpA->ConnectExpression(*DiffuseLerp->GetInput(0));
		DiffuseLerpB->ConnectExpression(*DiffuseLerp->GetInput(1));
		DiffuseExpression->ConnectExpression(*DiffuseLerp->GetInput(2));

		ColorExpression->ConnectExpression(*BaseColorMultiply->GetInput(0));
		DiffuseLerp->ConnectExpression(*BaseColorMultiply->GetInput(1));

		BaseColorMultiply->ConnectExpression(*BaseColorAdd->GetInput(0));
		IncandescenceColorExpression->ConnectExpression(*BaseColorAdd->GetInput(1));

		BaseColorAdd->ConnectExpression(*BaseColorTransparencyMultiply->GetInput(0));
		TransparencyOneMinus->ConnectExpression(*BaseColorTransparencyMultiply->GetInput(1));

		GlowIntensityExpression->ConnectExpression(*IncandescenceScaleMultiply->GetInput(0));
		IncandescenceScale->ConnectExpression(*IncandescenceScaleMultiply->GetInput(1));

		BaseColorTransparencyMultiply->ConnectExpression(*IncandescenceMultiply->GetInput(0));
		IncandescenceScaleMultiply->ConnectExpression(*IncandescenceMultiply->GetInput(1));

		TransparencyColorExpression->ConnectExpression(*TransparencyOneMinus->GetInput(0));

		if (bIsTransparent)
		{
			TransparencyOneMinus->ConnectExpression(*BreakFloat3->GetInput(0));

			BreakFloat3->ConnectExpression(*AddRG->GetInput(0), 0);
			BreakFloat3->ConnectExpression(*AddRG->GetInput(1), 1);

			AddRG->ConnectExpression(*AddRGB->GetInput(0));
			BreakFloat3->ConnectExpression(*AddRGB->GetInput(1), 2);

			AddRGB->ConnectExpression(*Divide->GetInput(0));
			DivideConstant->ConnectExpression(*Divide->GetInput(1));
		}

		// Connect material outputs
		MaterialElement->GetBaseColor().SetExpression(BaseColorTransparencyMultiply);
		MaterialElement->GetEmissiveColor().SetExpression(IncandescenceMultiply);
		if (bIsTransparent)
		{
			MaterialElement->GetOpacity().SetExpression(Divide);
			MaterialElement->SetParentLabel(TEXT("M_DatasmithAliasLambertTransparent"));
		}
		else {
			MaterialElement->SetParentLabel(TEXT("M_DatasmithAliasLambert"));
		}

	}

	void FWireTranslatorImpl::AddAlLightSourceParameters(const TAlObjectPtr<AlShader>& Shader, TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement)
	{
		// Default values for a LightSource material
		FColor Color(145, 148, 153);
		FColor TransparencyColor(0, 0, 0);
		FColor IncandescenceColor(0, 0, 0);
		double GlowIntensity = 0.0;

		AlList* List = Shader->fields();
		for (AlShadingFieldItem* Item = static_cast<AlShadingFieldItem*>(List->first()); Item; Item = Item->nextField())
		{
			double Value = 0.0f;
			statusCode ErrorCode = Shader->parameter(Item->field(), Value);
			if (ErrorCode != 0)
			{
				continue;
			}

			GetCommonParameters(Item->field(), Value, Color, TransparencyColor, IncandescenceColor, GlowIntensity);
		}

		bool bIsTransparent = IsTransparent(TransparencyColor);

		// Construct parameter expressions
		IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		ColorExpression->SetName(TEXT("Color"));
		ColorExpression->GetColor() = FLinearColor::FromSRGBColor(Color);

		IDatasmithMaterialExpressionColor* IncandescenceColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		IncandescenceColorExpression->SetName(TEXT("IncandescenceColor"));
		IncandescenceColorExpression->GetColor() = FLinearColor::FromSRGBColor(IncandescenceColor);

		IDatasmithMaterialExpressionColor* TransparencyColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		TransparencyColorExpression->SetName(TEXT("TransparencyColor"));
		TransparencyColorExpression->GetColor() = FLinearColor::FromSRGBColor(TransparencyColor);

		IDatasmithMaterialExpressionScalar* GlowIntensityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		GlowIntensityExpression->GetScalar() = GlowIntensity;
		GlowIntensityExpression->SetName(TEXT("GlowIntensity"));

		// Create aux expressions
		IDatasmithMaterialExpressionGeneric* BaseColorAdd = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorAdd->SetExpressionName(TEXT("Add"));

		IDatasmithMaterialExpressionGeneric* BaseColorTransparencyMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorTransparencyMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* IncandescenceMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		IncandescenceMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* IncandescenceScaleMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		IncandescenceScaleMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionScalar* IncandescenceScale = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		IncandescenceScale->GetScalar() = 100.0f;

		IDatasmithMaterialExpressionGeneric* TransparencyOneMinus = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		TransparencyOneMinus->SetExpressionName(TEXT("OneMinus"));

		IDatasmithMaterialExpressionFunctionCall* BreakFloat3 = nullptr;
		IDatasmithMaterialExpressionGeneric* AddRG = nullptr;
		IDatasmithMaterialExpressionGeneric* AddRGB = nullptr;
		IDatasmithMaterialExpressionGeneric* Divide = nullptr;
		IDatasmithMaterialExpressionScalar* DivideConstant = nullptr;
		if (bIsTransparent)
		{
			BreakFloat3 = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
			BreakFloat3->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakFloat3Components.BreakFloat3Components"));

			AddRG = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AddRG->SetExpressionName(TEXT("Add"));

			AddRGB = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AddRGB->SetExpressionName(TEXT("Add"));

			Divide = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			Divide->SetExpressionName(TEXT("Divide"));

			DivideConstant = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
			DivideConstant->GetScalar() = 3.0f;
		}

		// Connect expressions
		ColorExpression->ConnectExpression(*BaseColorAdd->GetInput(0));
		IncandescenceColorExpression->ConnectExpression(*BaseColorAdd->GetInput(1));

		BaseColorAdd->ConnectExpression(*BaseColorTransparencyMultiply->GetInput(0));
		TransparencyOneMinus->ConnectExpression(*BaseColorTransparencyMultiply->GetInput(1));

		GlowIntensityExpression->ConnectExpression(*IncandescenceScaleMultiply->GetInput(0));
		IncandescenceScale->ConnectExpression(*IncandescenceScaleMultiply->GetInput(1));

		BaseColorTransparencyMultiply->ConnectExpression(*IncandescenceMultiply->GetInput(0));
		IncandescenceScaleMultiply->ConnectExpression(*IncandescenceMultiply->GetInput(1));

		TransparencyColorExpression->ConnectExpression(*TransparencyOneMinus->GetInput(0));

		if (bIsTransparent)
		{
			TransparencyOneMinus->ConnectExpression(*BreakFloat3->GetInput(0));

			BreakFloat3->ConnectExpression(*AddRG->GetInput(0), 0);
			BreakFloat3->ConnectExpression(*AddRG->GetInput(1), 1);

			AddRG->ConnectExpression(*AddRGB->GetInput(0));
			BreakFloat3->ConnectExpression(*AddRGB->GetInput(1), 2);

			AddRGB->ConnectExpression(*Divide->GetInput(0));
			DivideConstant->ConnectExpression(*Divide->GetInput(1));
		}

		// Connect material outputs
		MaterialElement->GetBaseColor().SetExpression(BaseColorTransparencyMultiply);
		MaterialElement->GetEmissiveColor().SetExpression(IncandescenceMultiply);

		if (bIsTransparent)
		{
			MaterialElement->GetOpacity().SetExpression(Divide);
			MaterialElement->SetParentLabel(TEXT("M_DatasmithAliasLightSourceTransparent"));
		}
		else {
			MaterialElement->SetParentLabel(TEXT("M_DatasmithAliasLightSource"));
		}
	}

	void FWireTranslatorImpl::AddAlPhongParameters(const TAlObjectPtr<AlShader>& Shader, TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement)
	{
		// Default values for a Phong material
		FColor Color(145, 148, 153);
		FColor TransparencyColor(0, 0, 0);
		FColor IncandescenceColor(0, 0, 0);
		FColor SpecularColor(38, 38, 38);
		double Diffuse = 1.0;
		double GlowIntensity = 0.0;
		double Gloss = 0.8;
		double Shinyness = 20.0;
		double Specularity = 1.0;
		double Reflectivity = 0.5;

		AlList* List = Shader->fields();
		for (AlShadingFieldItem* Item = static_cast<AlShadingFieldItem*>(List->first()); Item; Item = Item->nextField())
		{
			double Value = 0.0f;
			statusCode ErrorCode = Shader->parameter(Item->field(), Value);
			if (ErrorCode != 0)
			{
				continue;
			}

			if (GetCommonParameters(Item->field(), Value, Color, TransparencyColor, IncandescenceColor, GlowIntensity))
			{
				continue;
			}

			switch (Item->field())
			{
			case AlShadingFields::kFLD_SHADING_PHONG_DIFFUSE:
				Diffuse = Value;
				break;
			case AlShadingFields::kFLD_SHADING_PHONG_GLOSS_:
				Gloss = Value;
				break;
			case AlShadingFields::kFLD_SHADING_PHONG_SPECULAR_R:
				SpecularColor.R = (uint8)(255.f * Value);;
				break;
			case AlShadingFields::kFLD_SHADING_PHONG_SPECULAR_G:
				SpecularColor.G = (uint8)(255.f * Value);;
				break;
			case AlShadingFields::kFLD_SHADING_PHONG_SPECULAR_B:
				SpecularColor.B = (uint8)(255.f * Value);;
				break;
			case AlShadingFields::kFLD_SHADING_PHONG_SPECULARITY_:
				Specularity = Value;
				break;
			case AlShadingFields::kFLD_SHADING_PHONG_SHINYNESS:
				Shinyness = Value;
				break;
			case AlShadingFields::kFLD_SHADING_PHONG_REFLECTIVITY:
				Reflectivity = Value;
				break;
			}
		}

		bool bIsTransparent = IsTransparent(TransparencyColor);

		// Construct parameter expressions
		IDatasmithMaterialExpressionScalar* DiffuseExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseExpression->GetScalar() = Diffuse;
		DiffuseExpression->SetName(TEXT("Diffuse"));

		IDatasmithMaterialExpressionScalar* GlossExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		GlossExpression->GetScalar() = Gloss;
		GlossExpression->SetName(TEXT("Gloss"));

		IDatasmithMaterialExpressionColor* SpecularColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		SpecularColorExpression->SetName(TEXT("SpecularColor"));
		SpecularColorExpression->GetColor() = FLinearColor::FromSRGBColor(SpecularColor);

		IDatasmithMaterialExpressionScalar* SpecularityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		SpecularityExpression->GetScalar() = Specularity * 0.3;
		SpecularityExpression->SetName(TEXT("Specularity"));

		IDatasmithMaterialExpressionScalar* ShinynessExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		ShinynessExpression->GetScalar() = Shinyness;
		ShinynessExpression->SetName(TEXT("Shinyness"));

		IDatasmithMaterialExpressionScalar* ReflectivityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		ReflectivityExpression->GetScalar() = Reflectivity;
		ReflectivityExpression->SetName(TEXT("Reflectivity"));

		IDatasmithMaterialExpressionColor* ColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		ColorExpression->SetName(TEXT("Color"));
		ColorExpression->GetColor() = FLinearColor::FromSRGBColor(Color);

		IDatasmithMaterialExpressionColor* IncandescenceColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		IncandescenceColorExpression->SetName(TEXT("IncandescenceColor"));
		IncandescenceColorExpression->GetColor() = FLinearColor::FromSRGBColor(IncandescenceColor);

		IDatasmithMaterialExpressionColor* TransparencyColorExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionColor>();
		TransparencyColorExpression->SetName(TEXT("TransparencyColor"));
		TransparencyColorExpression->GetColor() = FLinearColor::FromSRGBColor(TransparencyColor);

		IDatasmithMaterialExpressionScalar* GlowIntensityExpression = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		GlowIntensityExpression->GetScalar() = GlowIntensity;
		GlowIntensityExpression->SetName(TEXT("GlowIntensity"));

		// Create aux expressions
		IDatasmithMaterialExpressionGeneric* ColorSpecLerp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		ColorSpecLerp->SetExpressionName(TEXT("LinearInterpolate"));

		IDatasmithMaterialExpressionScalar* ColorSpecLerpValue = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		ColorSpecLerpValue->GetScalar() = 0.96f;

		IDatasmithMaterialExpressionGeneric* ColorMetallicLerp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		ColorMetallicLerp->SetExpressionName(TEXT("LinearInterpolate"));

		IDatasmithMaterialExpressionGeneric* DiffuseLerp = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		DiffuseLerp->SetExpressionName(TEXT("LinearInterpolate"));

		IDatasmithMaterialExpressionScalar* DiffuseLerpA = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseLerpA->GetScalar() = 0.04f;

		IDatasmithMaterialExpressionScalar* DiffuseLerpB = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		DiffuseLerpB->GetScalar() = 1.0f;

		IDatasmithMaterialExpressionGeneric* BaseColorMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* BaseColorAdd = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorAdd->SetExpressionName(TEXT("Add"));

		IDatasmithMaterialExpressionGeneric* BaseColorTransparencyMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		BaseColorTransparencyMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* IncandescenceMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		IncandescenceMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* IncandescenceScaleMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		IncandescenceScaleMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionScalar* IncandescenceScale = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		IncandescenceScale->GetScalar() = 100.0f;

		IDatasmithMaterialExpressionGeneric* ShinynessSubtract = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		ShinynessSubtract->SetExpressionName(TEXT("Subtract"));

		IDatasmithMaterialExpressionScalar* ShinynessSubtract2 = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		ShinynessSubtract2->GetScalar() = 2.0f;

		IDatasmithMaterialExpressionGeneric* ShinynessDivide = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		ShinynessDivide->SetExpressionName(TEXT("Divide"));

		IDatasmithMaterialExpressionScalar* ShinynessDivide98 = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
		ShinynessDivide98->GetScalar() = 98.0f;

		IDatasmithMaterialExpressionGeneric* SpecularityMultiply = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		SpecularityMultiply->SetExpressionName(TEXT("Multiply"));

		IDatasmithMaterialExpressionGeneric* RoughnessOneMinus = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		RoughnessOneMinus->SetExpressionName(TEXT("OneMinus"));

		IDatasmithMaterialExpressionGeneric* TransparencyOneMinus = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
		TransparencyOneMinus->SetExpressionName(TEXT("OneMinus"));

		IDatasmithMaterialExpressionFunctionCall* BreakFloat3 = nullptr;
		IDatasmithMaterialExpressionGeneric* AddRG = nullptr;
		IDatasmithMaterialExpressionGeneric* AddRGB = nullptr;
		IDatasmithMaterialExpressionGeneric* Divide = nullptr;
		IDatasmithMaterialExpressionScalar* DivideConstant = nullptr;
		if (bIsTransparent)
		{
			BreakFloat3 = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionFunctionCall>();
			BreakFloat3->SetFunctionPathName(TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakFloat3Components.BreakFloat3Components"));

			AddRG = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AddRG->SetExpressionName(TEXT("Add"));

			AddRGB = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			AddRGB->SetExpressionName(TEXT("Add"));

			Divide = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionGeneric>();
			Divide->SetExpressionName(TEXT("Divide"));

			DivideConstant = MaterialElement->AddMaterialExpression<IDatasmithMaterialExpressionScalar>();
			DivideConstant->GetScalar() = 3.0f;
		}

		// Connect expressions
		SpecularColorExpression->ConnectExpression(*ColorSpecLerp->GetInput(0));
		ColorExpression->ConnectExpression(*ColorSpecLerp->GetInput(1));
		ColorSpecLerpValue->ConnectExpression(*ColorSpecLerp->GetInput(2));

		ColorExpression->ConnectExpression(*ColorMetallicLerp->GetInput(0));
		ColorSpecLerp->ConnectExpression(*ColorMetallicLerp->GetInput(1));
		GlossExpression->ConnectExpression(*ColorMetallicLerp->GetInput(2));

		DiffuseLerpA->ConnectExpression(*DiffuseLerp->GetInput(0));
		DiffuseLerpB->ConnectExpression(*DiffuseLerp->GetInput(1));
		DiffuseExpression->ConnectExpression(*DiffuseLerp->GetInput(2));

		ColorMetallicLerp->ConnectExpression(*BaseColorMultiply->GetInput(0));
		DiffuseLerp->ConnectExpression(*BaseColorMultiply->GetInput(1));

		BaseColorMultiply->ConnectExpression(*BaseColorAdd->GetInput(0));
		IncandescenceColorExpression->ConnectExpression(*BaseColorAdd->GetInput(1));

		BaseColorAdd->ConnectExpression(*BaseColorTransparencyMultiply->GetInput(0));
		TransparencyOneMinus->ConnectExpression(*BaseColorTransparencyMultiply->GetInput(1));

		GlowIntensityExpression->ConnectExpression(*IncandescenceScaleMultiply->GetInput(0));
		IncandescenceScale->ConnectExpression(*IncandescenceScaleMultiply->GetInput(1));

		BaseColorTransparencyMultiply->ConnectExpression(*IncandescenceMultiply->GetInput(0));
		IncandescenceScaleMultiply->ConnectExpression(*IncandescenceMultiply->GetInput(1));

		ShinynessExpression->ConnectExpression(*ShinynessSubtract->GetInput(0));
		ShinynessSubtract2->ConnectExpression(*ShinynessSubtract->GetInput(1));

		ShinynessSubtract->ConnectExpression(*ShinynessDivide->GetInput(0));
		ShinynessDivide98->ConnectExpression(*ShinynessDivide->GetInput(1));

		ShinynessDivide->ConnectExpression(*SpecularityMultiply->GetInput(0));
		SpecularityExpression->ConnectExpression(*SpecularityMultiply->GetInput(1));

		SpecularityMultiply->ConnectExpression(*RoughnessOneMinus->GetInput(0));

		TransparencyColorExpression->ConnectExpression(*TransparencyOneMinus->GetInput(0));

		if (bIsTransparent)
		{
			TransparencyOneMinus->ConnectExpression(*BreakFloat3->GetInput(0));

			BreakFloat3->ConnectExpression(*AddRG->GetInput(0), 0);
			BreakFloat3->ConnectExpression(*AddRG->GetInput(1), 1);

			AddRG->ConnectExpression(*AddRGB->GetInput(0));
			BreakFloat3->ConnectExpression(*AddRGB->GetInput(1), 2);

			AddRGB->ConnectExpression(*Divide->GetInput(0));
			DivideConstant->ConnectExpression(*Divide->GetInput(1));
		}

		// Connect material outputs
		MaterialElement->GetBaseColor().SetExpression(BaseColorTransparencyMultiply);
		MaterialElement->GetMetallic().SetExpression(GlossExpression);
		MaterialElement->GetSpecular().SetExpression(ReflectivityExpression);
		MaterialElement->GetRoughness().SetExpression(RoughnessOneMinus);
		MaterialElement->GetEmissiveColor().SetExpression(IncandescenceMultiply);
		if (bIsTransparent)
		{
			MaterialElement->GetOpacity().SetExpression(Divide);
			MaterialElement->SetParentLabel(TEXT("M_DatasmithAliasPhongTransparent"));
		}
		else {
			MaterialElement->SetParentLabel(TEXT("M_DatasmithAliasPhong"));
		}
	}

	class FWireInterfaceModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			// AliasStudio 2027 and beyond is not supported
			uint64 AliasVersion = IWireInterface::GetRequiredAliasVersion(LibAlias2027_0_0_Version);

#ifdef OPEN_MODEL_2020
			// Check installed version of Alias Tools because binaries before 2021.3 are not compatible with Alias 2022
			if (LibAlias2020_Version < AliasVersion && AliasVersion < LibAlias2021_Version)
			{
				static const bool bIsDisplay = []() -> bool
					{
						UE_LOG(LogWireInterface, Warning, TEXT(WRONG_VERSION_TEXT));
						return true;
					}();
				return;
			}
#endif

			if (LibAliasVersionMin <= AliasVersion && AliasVersion < LibAliasVersionMax)
			{
				auto MakeInterfaceFunc = []() -> TSharedPtr<IWireInterface>
					{
						return MakeShared<FWireTranslatorImpl>();
					};
				IWireInterface::RegisterInterface(UE_OPENMODEL_MAJOR_VERSION, UE_OPENMODEL_MAJOR_VERSION, MoveTemp(MakeInterfaceFunc));
			}
		}

		virtual void ShutdownModule() override
		{
#if WIRE_MEMORY_CHECK
			// #wire_import: Need to investigate why this is crashing when enabled.
			if (bool(AlUniverse::isInitialized()))
			{
				AlUniverse::deleteAll();
			}
#endif
		}
	};

#if TRACK_MESHELEMENT
#pragma optimize("", off)
	void MakeMeshVisible(FMeshDescription& MeshDescription)
	{
#if MAKE_VISIBLE
		using namespace UE::Geometry;

		TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();
		
		TArrayView<FVector3f> Positions = VertexPositions.GetRawArray();
		TOrientedBox3<float> OBox = FitOrientedBox3Points<float>(Positions);
		
		FVertexArray& Vertices = MeshDescription.Vertices();
		const TMatrix3<float> Matrix = OBox.Frame.Rotation.ToRotationMatrix();

		constexpr float MinSize = 1.f;
		constexpr float MaxSize = 20.f;

		const float ScaleX = OBox.Extents.X < MinSize ? 20.f / OBox.Extents.X : OBox.Extents.X > MaxSize ? MaxSize / OBox.Extents.X : 10.f;
		const float ScaleY = OBox.Extents.Y < MinSize ? 20.f / OBox.Extents.Y : OBox.Extents.Y > MaxSize ? MaxSize / OBox.Extents.Y : 10.f;
		const float ScaleZ = OBox.Extents.Z < MinSize ? 20.f / OBox.Extents.Z : OBox.Extents.Z > MaxSize ? MaxSize / OBox.Extents.Z : 10.f;
		UE_LOGF(LogWireInterface, Warning, "Scaling factor: %.3f %.3f %.3f", ScaleX, ScaleY, ScaleZ);
		const FVector3f AxisX = OBox.AxisX();
		const FVector3f AxisY = OBox.AxisY();
		const FVector3f AxisZ = OBox.AxisZ();

		for (const FVertexID& VertexID : MeshDescription.Vertices().GetElementIDs())
		{
			FVector3f P = VertexPositions[VertexID] - OBox.Frame.Origin;
			VertexPositions[VertexID] = ((P | AxisX) * ScaleX) * AxisX + ((P | AxisY) * ScaleY) * AxisY + ((P | AxisZ) * ScaleZ) * AxisZ;
		}
#endif
	}
#endif

} // namespace

#undef LOCTEXT_NAMESPACE // "DatasmithWireTranslator"

#else // USE_OPENMODEL
namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{
	class FWireInterfaceModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
		}

		virtual void ShutdownModule() override
		{
		}
	};
}
#endif

// need this macro wrapper to expand the UE_DATASMITHWIRETRANSLATOR_MODULE_NAME macro and not create symbols with "UE_DATASMITHWIRETRANSLATOR_MODULE_NAME" in the token
#define IMPLEMENT_MODULE_WRAPPER(ModuleName) IMPLEMENT_MODULE(UE_DATASMITHWIRETRANSLATOR_NAMESPACE::FWireInterfaceModule, ModuleName);
IMPLEMENT_MODULE_WRAPPER(UE_DATASMITHWIRETRANSLATOR_MODULE_NAME)
