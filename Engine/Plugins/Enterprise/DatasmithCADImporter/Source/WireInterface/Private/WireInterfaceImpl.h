// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifdef USE_OPENMODEL

#include "OpenModelUtils.h"

#include "IWireInterface.h"

#include "Templates/SharedPointer.h"

class AlDagNode;
class AlLayer;
class AlMesh;
class AlShader;
class AlShaderNode;
class IDatasmithActorElement;
class IDatasmithBaseMaterialElement;
class IDatasmithMaterialIDElement;
class IDatasmithUEPbrMaterialElement;

struct FMeshDescription;

namespace CADLibrary
{
	struct FMeshParameters;
	class ICADModelConverter;
}

namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{
	class FBodyData;
	struct FDagNodeInfo;

	class FWireTranslatorImpl : public IWireInterface
	{
	public:
		FWireTranslatorImpl() {}

		~FWireTranslatorImpl();

		/** Begin IWrieInterface */
		virtual bool Initialize(const TCHAR* InSceneFullName) override;
		virtual bool Load(TSharedPtr<IDatasmithScene> InScene) override;
		virtual void SetImportSettings(const FWireSettings& Options) override;
		virtual void SetOutputPath(const FString& Path) { OutputPath = Path; }

		bool LoadStaticMesh(const TSharedPtr<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload, const FDatasmithTessellationOptions& InTessellationOptions);
		/** End IWrieInterface */

	private:
		/** Model traversal */
		bool TraverseModel();
		TSharedPtr<IDatasmithActorElement> TraverseDag(const FAlDagNodePtr& DagNode);

		TSharedPtr<IDatasmithActorElement> ProcessGeometryNode(const FAlDagNodePtr& GeomNode, const TAlObjectPtr<AlLayer>& ParentLayer = TAlObjectPtr<AlLayer>());
		TSharedPtr<IDatasmithActorElement> TraverseGroupNode(const FAlDagNodePtr& GroupNode, const TAlObjectPtr<AlLayer>& ParentLayer = TAlObjectPtr<AlLayer>());
		TSharedPtr<IDatasmithActorElement> ProcessGroupNode(const FAlDagNodePtr& GroupNode, const TAlObjectPtr<AlLayer>& ParentLayer = TAlObjectPtr<AlLayer>());
		TSharedPtr<IDatasmithActorElement> ProcessBodyNode(TSharedPtr<FBodyNode>& BodyNode, const FAlDagNodePtr& GroupNode, const TAlObjectPtr<AlLayer>& ParentLayer = TAlObjectPtr<AlLayer>());
		TSharedPtr<IDatasmithActorElement> ProcessPatchMesh(TSharedPtr<FPatchMesh>& PatchMesh, const FAlDagNodePtr& GroupNode, const TAlObjectPtr<AlLayer>& ParentLayer = TAlObjectPtr<AlLayer>());

		TSharedPtr<IDatasmithActorElement> FindOrAddLayerActor(const TAlObjectPtr<AlLayer>& Layer);
		TSharedPtr<IDatasmithMeshElement> FindOrAddMeshElement(const FAlDagNodePtr& GeomNode);
		TSharedPtr<IDatasmithMeshElement> FindOrAddMeshElement(TSharedPtr<FBodyNode>& BodyNode);
		TSharedPtr<IDatasmithMeshElement> FindOrAddMeshElement(TSharedPtr<FPatchMesh>& PatchMesh);

		/** Material creation */
		bool IsTransparent(FColor& TransparencyColor)
		{
			float Opacity = 1.0f - ((float)(TransparencyColor.R + TransparencyColor.G + TransparencyColor.B)) / 765.0f;
			return !FMath::IsNearlyEqual(Opacity, 1.0f);
		}

		bool GetCommonParameters(int32 Field, double Value, FColor& Color, FColor& TransparencyColor, FColor& IncandescenceColor, double GlowIntensity);

		void AddAlBlinnParameters(const TAlObjectPtr<AlShader>& Shader, TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement);
		void AddAlLambertParameters(const TAlObjectPtr<AlShader>& Shader, TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement);
		void AddAlLightSourceParameters(const TAlObjectPtr<AlShader>& Shader, TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement);
		void AddAlPhongParameters(const TAlObjectPtr<AlShader>& Shader, TSharedPtr<IDatasmithUEPbrMaterialElement> MaterialElement);
		
		TSharedPtr<IDatasmithBaseMaterialElement> FindOrAddMaterial(const TAlObjectPtr<AlShader>& Shader);

		/** Geometry retrieval */
		TOptional<FMeshDescription> GetMeshDescription(TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& OutMeshParameters);
		TSharedPtr<CADLibrary::ICADModelConverter> GetModelConverter() const;
		TOptional<FMeshDescription> GetMeshDescriptionFromBodyNode(TSharedPtr<FBodyNode>& BodyNode, TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& MeshParameters);
		TOptional<FMeshDescription> GetMeshDescriptionFromPatchMesh(TSharedPtr<FPatchMesh>& PatchMesh, TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& MeshParameters);
		TOptional<FMeshDescription> GetMeshDescriptionFromParametricNode(const FAlDagNodePtr& DagNode, TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& MeshParameters);
		TOptional<FMeshDescription> GetMeshDescriptionFromMeshNode(const FAlDagNodePtr& MeshNode, TSharedPtr<IDatasmithMeshElement> MeshElement, CADLibrary::FMeshParameters& MeshParameters);

		FAlDagNodePtr FindOrAddDagNode(AlDagNode* InDagNode)
		{
			if (!InDagNode)
			{
				return FAlDagNodePtr();
			}

			if (FAlDagNodePtr* DagNodePtr = EncounteredNodes.Find(InDagNode))
			{
#if WIRE_MEMORY_CHECK
				ensure(false);
#endif
				return *DagNodePtr;
			}

			FAlDagNodePtr& NewDagNode = EncounteredNodes.Add(InDagNode);
			NewDagNode = InDagNode;

			return NewDagNode;
		}
	private:
		TSharedPtr<IDatasmithScene> DatasmithScene;
		FString OutputPath;
		FString SceneFullPath;
		FString SceneVersion;

		FWireSettings WireSettings;

		TSharedPtr<CADLibrary::ICADModelConverter> CADModelConverter;

		bool bSceneLoaded = false;

		TMap<FString, TSharedPtr<IDatasmithBaseMaterialElement>> ShaderNameToMaterial;

		TMap<uint32, TSharedPtr<IDatasmithMeshElement>> GeomNodeToMeshElement;
		TMap<TSharedPtr<IDatasmithMeshElement>, FAlDagNodePtr> MeshElementToParametricNode;
		TMap<TSharedPtr<IDatasmithMeshElement>, FAlDagNodePtr> MeshElementToMeshNode;

		TMap<uint32, TSharedPtr<IDatasmithMeshElement>> BodyNodeToMeshElement;
		TMap<TSharedPtr<IDatasmithMeshElement>, TSharedPtr<FBodyNode>> MeshElementToBodyNode;

		TMap<uint32, TSharedPtr<IDatasmithMeshElement>> PatchMeshToMeshElement;
		TMap<TSharedPtr<IDatasmithMeshElement>, TSharedPtr<FPatchMesh>> MeshElementToPatchMesh;

		TMap<AlDagNode*, FAlDagNodePtr> EncounteredNodes;

		TMap<uint32, TSharedPtr<IDatasmithActorElement>> LayerToActor;

		// #cad_debug
		bool bTrackMesh = false;
	};
} // namespace
#endif
