// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE_DATASMITHWIRETRANSLATOR_NAMESPACE
{

/**
 * Datasmith Translator for .wire files.
 */
class FDatasmithWireTranslatorModule : public IModuleInterface
{
public:

    static FDatasmithWireTranslatorModule& Get()
    {
        return FModuleManager::LoadModuleChecked< FDatasmithWireTranslatorModule >(UE_STRINGIZE(UE_DATASMITHWIRETRANSLATOR_MODULE_NAME));
	}

    static bool IsAvailable()
    {
        return FModuleManager::Get().IsModuleLoaded(UE_STRINGIZE(UE_DATASMITHWIRETRANSLATOR_MODULE_NAME));
    }

	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	FString GetTempDir() const;

private:
	FString TempDir;
};

}