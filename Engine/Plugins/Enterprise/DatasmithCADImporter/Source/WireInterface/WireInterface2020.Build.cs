// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public abstract class WireInterfaceBase : ModuleRules
{
	public WireInterfaceBase(ReadOnlyTargetRules Target) : base(Target)
	{
		// TODO: investigate to remove that (Jira UETOOL-4975)
		bUseUnity = false;
		//OptimizeCode = CodeOptimization.Never;
		//PCHUsage = ModuleRules.PCHUsageMode.NoPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CADInterfaces",
				"CADKernel",
				"CADKernelSurface",
				"CADLibrary",
				"CADTools",
				"DatasmithContent",
				"DatasmithCore",
				"DatasmithTranslator",
				"DatasmithWireTranslator",
				"Engine",
				"MeshDescription",
				"GeometryCore",
				"ParametricSurface",
				"StaticMeshDescription",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
 					"MessageLog",
 					"UnrealEd",
				}
			);
		}

		PublicDefinitions.Add(GetAliasDefinition());
		PublicDefinitions.Add($"UE_DATASMITHWIRETRANSLATOR_NAMESPACE={GetAliasDefinition()}Namespace");
		PublicDefinitions.Add($"UE_DATASMITHWIRETRANSLATOR_MODULE_NAME={GetType().Name}");
		PublicDefinitions.Add($"UE_OPENMODEL_MAJOR_VERSION={GetMajorVersion()}");
		PublicDefinitions.Add($"UE_OPENMODEL_MINOR_VERSION={GetMinorVersion()}");
		PublicDefinitions.Add("WIRE_THINFACE_ENABLED=0");

		if (System.Type.GetType(GetAliasVersion()) != null)
		{
			PrivateDependencyModuleNames.Add(GetAliasVersion());
		}
	}

	public abstract string GetAliasVersion();
	public abstract string GetAliasDefinition();
	public abstract int GetMajorVersion();
	public abstract int GetMinorVersion();

}

public class WireInterface2020 : WireInterfaceBase
{
	public WireInterface2020(ReadOnlyTargetRules Target) 
		: base(Target)
	{
		PublicDefinitions.Add("IS_MAIN_MODULE");
	}

	public override string GetAliasVersion()
	{
		return "OpenModel2020";
	}
	
	public override string GetAliasDefinition()
	{
		return "OPEN_MODEL_2020";
	}

	public override int GetMajorVersion()
	{
		return 2020;
	}

	public override int GetMinorVersion()
	{
		return 0;
	}
}
