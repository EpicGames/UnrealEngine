// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class WireInterface2021_3 : WireInterfaceBase
{
	public WireInterface2021_3(ReadOnlyTargetRules Target) 
		: base(Target)
	{
	}

	public override string GetAliasVersion()
	{
		return "OpenModel2021_3";
	}
	
	public override string GetAliasDefinition()
	{
		return "OPEN_MODEL_2021_3";
	}

	public override int GetMajorVersion()
	{
		return 2021;
	}

	public override int GetMinorVersion()
	{
		return 3;
	}
}
