// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class WireInterface2022 : WireInterfaceBase
{
	public WireInterface2022(ReadOnlyTargetRules Target) 
		: base(Target)
	{
	}

	public override string GetAliasVersion()
	{
		return "OpenModel2022";
	}
	
	public override string GetAliasDefinition()
	{
	return "OPEN_MODEL_2022";
	}

	public override int GetMajorVersion()
	{
		return 2022;
	}

	public override int GetMinorVersion()
	{
		return 0;
	}
}