// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class WireInterface2023_1 : WireInterfaceBase
{
	public WireInterface2023_1(ReadOnlyTargetRules Target) 
		: base(Target)
	{
	}

	public override string GetAliasVersion()
	{
		return "OpenModel2023_1";
	}

	public override string GetAliasDefinition()
	{
		return "OPEN_MODEL_2023_1";
	}

	public override int GetMajorVersion()
	{
		return 2023;
	}

	public override int GetMinorVersion()
	{
		return 1;
	}
}