// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class WireInterface2024_1 : WireInterfaceBase
{
	public WireInterface2024_1(ReadOnlyTargetRules Target) 
		: base(Target)
	{
	}

	public override string GetAliasVersion()
	{
		return "OpenModel2024_1";
	}

	public override string GetAliasDefinition()
	{
		return "OPEN_MODEL_2024_1";
	}

	public override int GetMajorVersion()
	{
		return 2024;
	}

	public override int GetMinorVersion()
	{
		return 1;
	}
}