// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class WireInterface2025_0 : WireInterfaceBase
{
	public WireInterface2025_0(ReadOnlyTargetRules Target) 
		: base(Target)
	{
	}

	public override string GetAliasVersion()
	{
		return "OpenModel2025_0";
	}

	public override string GetAliasDefinition()
	{
		return "OPEN_MODEL_2025_0";
	}

	public override int GetMajorVersion()
	{
		return 2025;
	}

	public override int GetMinorVersion()
	{
		return 0;
	}
}