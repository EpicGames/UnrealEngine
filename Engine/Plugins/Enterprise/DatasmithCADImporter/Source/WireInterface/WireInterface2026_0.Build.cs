// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class WireInterface2026_0 : WireInterfaceBase
{
	public WireInterface2026_0(ReadOnlyTargetRules Target) 
		: base(Target)
	{
	}

	public override string GetAliasVersion()
	{
		return "OpenModel2026_0";
	}

	public override string GetAliasDefinition()
	{
		return "OPEN_MODEL_2026_0";
	}

	public override int GetMajorVersion()
	{
		return 2026;
	}

	public override int GetMinorVersion()
	{
		return 0;
	}
}